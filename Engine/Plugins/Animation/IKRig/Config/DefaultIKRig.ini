[CoreRedirects]
+ClassRedirects=(OldName="/Script/IKRig.IKRigPBIKSolver", NewName="/Script/IKRig.IKRigFBIKSolver")
+ClassRedirects=(OldName="/Script/IKRig.IKRig_PBIKBoneSettings", NewName="/Script/IKRig.IKRig_FBIKBoneSettings")
+ClassRedirects=(OldName="/Script/IKRig.AnimNode_IKRetargeter", NewName="/Script/IKRig.AnimNode_RetargetPoseFromMesh")
+FunctionRedirects=(OldName="/Script/IKRigEditor.IKRigController.SetRootBone",NewName="/Script/IKRigEditor.IKRigController.SetStartBone")
+FunctionRedirects=(OldName="/Script/IKRigEditor.IKRigController.GetRootBone",NewName="/Script/IKRigEditor.IKRigController.GetStartBone")

