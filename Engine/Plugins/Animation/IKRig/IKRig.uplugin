{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "IK Rig",
	"Description": "",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "FullBodyIK",
			"Enabled": true
		},
		{
			"Name": "UAF",
			"Enabled": false,
			"Optional": true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled": false,
			"Optional": true
		}
	],
	"Modules": [
		{
			"Name": "IKRig",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "IKRigDeveloper",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "IKRigEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "IKRigUAF",
			"Type": "Runtime",
			"LoadingPhase": "None"
		}
	]
}