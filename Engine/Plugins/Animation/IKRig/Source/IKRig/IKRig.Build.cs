// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IKRig : ModuleRules
{
	public IKRig(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// NOTE: this was disabled when we added a Rig Unit for IK Rig due to compiler warnings in Control Rig
		// We should renable this once those are addressed.
		//CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"AnimationCore",
			});

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"ControlRig",
				"RigVM",
				"Core",
				"PBIK"
			});

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			});
		
		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"ApplicationCore",
					"MessageLog",
				});

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"AnimationWidgets",
				});
		}
	}
}
