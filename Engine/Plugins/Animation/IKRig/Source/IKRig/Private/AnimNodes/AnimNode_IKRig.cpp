// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_IKRig.h"

#include "Engine/SkeletalMesh.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimTrace.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"

#include "Rig/IKRigProcessor.h"
#include "Rig/IKRigDefinition.h"

#include "Components/SkeletalMeshComponent.h"
#include "ActorComponents/IKRigInterface.h"

#include "PrimitiveDrawingUtils.h"
#include "Logging/LogVerbosity.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_IKRig)

namespace UE::Anim::IKRig
{
	static bool bEnableIKRig = true;
	FAutoConsoleVariableRef CVarEnableIKRigNode(TEXT("a.animnode.IKRig.Enable"), bEnableIKRig, TEXT("Enables/disables IK Rig Node in Anim Graph."));
}

FAnimNode_IKRig::FAnimNode_IKRig()
	: AlphaInputType(EAnimAlphaInputType::Float)
	, bAlphaBoolEnabled(true)
	, Alpha(1.0f)
	, ActualAlpha(0.f)
{}

void FAnimNode_IKRig::Evaluate_AnyThread(FPoseContext& Output) 
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(IKRig, !IsInGameThread());

	if (bStartFromRefPose)
	{
		Output.ResetToRefPose();
	}
	else if (Source.GetLinkNode())
	{
		Source.Evaluate(Output);
	}
	
	if (!FAnimWeight::IsRelevant(ActualAlpha))
	{
		return;
	}
	
	// initialize the IK Rig (will only try once on the current version of the rig asset)
	if (!IKRigProcessor.IsInitialized())
	{
		IKRigProcessor.Initialize(RigDefinitionAsset, Output.AnimInstanceProxy->GetSkelMeshComponent()->GetSkeletalMeshAsset(),FIKRigGoalContainer());
	}

	// bail out if uninitialized
	if (!IKRigProcessor.IsInitialized() || !UE::Anim::IKRig::bEnableIKRig)
	{
		return;
	}
	
	// copy input pose to solver stack
	CopyInputPoseToSolver(Output.Pose);
	// update target goal transforms
	AssignGoalTargets();
	// run stack of solvers,
	const FTransform WorldToComponent =  Output.AnimInstanceProxy->GetComponentTransform().Inverse();
	IKRigProcessor.Solve(WorldToComponent);
	// updates transforms with new pose
	CopyOutputPoseToAnimGraph(Output.Pose);
}

void FAnimNode_IKRig::CopyInputPoseToSolver(FCompactPose& InputPose)
{
	// start Solve() from REFERENCE pose
	if (bStartFromRefPose)
	{
		IKRigProcessor.SetInputPoseToRefPose();
		return;
	}
	
	// start Solve() from INPUT pose
	// copy local bone transforms into IKRigProcessor skeleton
	FIKRigSkeleton& IKRigSkeleton = IKRigProcessor.GetSkeletonWriteable();
	for (FCompactPoseBoneIndex CPIndex : InputPose.ForEachBoneIndex())
	{
		if (int32* Index = CompactPoseToRigIndices.Find(CPIndex))
		{
			// bones that were recorded with rig indices == -1 were not in the
			// Reference Skeleton that the IK Rig was initialized with and therefore
			// are not considered as part of the solve.
			if (IKRigSkeleton.CurrentPoseLocal.IsValidIndex(*Index))
			{
				IKRigSkeleton.CurrentPoseLocal[*Index] = InputPose[CPIndex];	
			}
		}
	}
	// update global pose in IK Rig
	IKRigSkeleton.UpdateAllGlobalTransformFromLocal();
}

void FAnimNode_IKRig::AssignGoalTargets()
{
	// update goal transforms before solve
	// these transforms can come from a few different sources, handled here...

	#if WITH_EDITOR
	// use the goal transforms from the source asset itself
	// this is used to live preview results from the IK Rig editor
	// NOTE: as the transaction when undoing/redoing can be called on the preview scene before the editor, the processor
	// might not have been reinitialized, resulting in data being desynchronized. Thus, we must wait until the
	// transaction has been fully processed.
	if (bDriveWithSourceAsset && !GIsTransacting)
	{
		IKRigProcessor.CopyAllSettingsFromAsset(RigDefinitionAsset);
		return;
	}
	#endif
	
	// copy transforms from this anim node's goal pins from blueprint
	for (const FIKRigGoal& Goal : Goals)
	{
		IKRigProcessor.SetIKGoal(Goal);
	}

	// override any goals that were manually set with goals from goal creator components (they take precedence)
	for (const TPair<FName, FIKRigGoal>& GoalPair : GoalsFromGoalCreators)
	{
		IKRigProcessor.SetIKGoal(GoalPair.Value);
	}
}

void FAnimNode_IKRig::CopyOutputPoseToAnimGraph(FCompactPose& OutputPose)
{
	FIKRigSkeleton& IKRigSkeleton = IKRigProcessor.GetSkeletonWriteable();
	
	// update local transforms of current IKRig pose
	IKRigSkeleton.UpdateAllLocalTransformFromGlobal();

	// copy local transforms to output pose
	for (FCompactPoseBoneIndex CPIndex : OutputPose.ForEachBoneIndex())
	{
		if (int32* Index = CompactPoseToRigIndices.Find(CPIndex))
		{
			// bones that were recorded with rig indices == -1 were not in the
			// Reference Skeleton that the IK Rig was initialized with and therefore
			// are not considered as part of the solve. These transforms are left at their
			// input pose (in local space).
			if (*Index != -1)
			{
				OutputPose[CPIndex].BlendWith(IKRigSkeleton.CurrentPoseLocal[*Index], ActualAlpha);
			}
		}
	}
}

void FAnimNode_IKRig::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()

	DebugData.AddDebugItem(FString::Printf(TEXT("%s IK Rig evaluated with %d Goals."), *DebugData.GetNodeName(this), Goals.Num()));
		
	for (const TPair<FName, FIKRigGoal>& GoalPair : GoalsFromGoalCreators)
	{
		DebugData.AddDebugItem(FString::Printf(TEXT("Goal supplied by actor component: %s"), *GoalPair.Value.ToString()));
	}

	for (const FIKRigGoal& Goal : Goals)
	{
		if (GoalsFromGoalCreators.Contains(Goal.Name))
		{
			continue;
		}
		
		DebugData.AddDebugItem(FString::Printf(TEXT("Goal supplied by node pin: %s"), *Goal.ToString()));
	}

	Source.GatherDebugData(DebugData);
}

void FAnimNode_IKRig::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	FAnimNode_Base::Initialize_AnyThread(Context);
	Source.Initialize(Context);

	// Initial update of the node, so we dont have a frame-delay on setup
	GetEvaluateGraphExposedInputs().Execute(Context);

	// initialize all the FProperty pointers to copy dynamic anim node inputs
	InitializeProperties(Context.GetAnimInstanceObject(), GetTargetClass());

	// trigger another search for goal creators on the main thread
	bSearchedForGoalCreators = false;
}

void FAnimNode_IKRig::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	GetEvaluateGraphExposedInputs().Execute(Context);

	// update actual alpha
	{
		ActualAlpha = 0.f;

		switch (AlphaInputType)
		{
		case EAnimAlphaInputType::Float:
			ActualAlpha = AlphaScaleBias.ApplyTo(AlphaScaleBiasClamp.ApplyTo(Alpha, Context.GetDeltaTime()));
			break;
		case EAnimAlphaInputType::Bool:
			ActualAlpha = AlphaBoolBlend.ApplyTo(bAlphaBoolEnabled, Context.GetDeltaTime());
			break;
		case EAnimAlphaInputType::Curve:
			if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(Context.AnimInstanceProxy->GetAnimInstanceObject()))
			{
				ActualAlpha = AlphaScaleBiasClamp.ApplyTo(AnimInstance->GetCurveValue(AlphaCurveName), Context.GetDeltaTime());
			}
			break;
		};
		
		ActualAlpha = FMath::Clamp<float>(ActualAlpha, 0.f, 1.f);
	}

	PropagateInputProperties(Context.AnimInstanceProxy->GetAnimInstanceObject());

	FAnimNode_Base::Update_AnyThread(Context);
	Source.Update(Context);

#if ENABLE_VISUAL_LOG
#if WITH_EDITORONLY_DATA
	// is node setup?
	if (IKRigProcessor.IsInitialized() && FVisualLogger::IsRecording())
	{
		static const FBox UnitBox(FVector(-1, -1, -1), FVector(1, 1, 1));

		const TArray<FIKRigGoal>& ProcessorGoals = IKRigProcessor.GetGoalContainer().GetGoalArray();
		for (const FIKRigGoal& Goal : ProcessorGoals)
		{
			FTransform GoalTransform(Goal.FinalBlendedRotation, Goal.FinalBlendedPosition, FVector(DebugScale, DebugScale, DebugScale));
			GoalTransform = GoalTransform * Context.AnimInstanceProxy->GetComponentTransform();
			UE_VLOG_OBOX(Context.AnimInstanceProxy->GetAnimInstanceObject(), "IKRig", Display, UnitBox, GoalTransform.ToMatrixWithScale(), FColor::Yellow, TEXT(""));
			UE_VLOG_LOCATION(Context.AnimInstanceProxy->GetAnimInstanceObject(), "IKRig", Verbose, GoalTransform.GetTranslation(), 0, FColor::White, TEXT("%ls"), ToCStr(Goal.Name.ToString()));
		}
	}
#endif
#endif

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Name"), RigDefinitionAsset ? ToCStr(RigDefinitionAsset->GetName()) : TEXT(""));
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Asset"), RigDefinitionAsset);
}

void FAnimNode_IKRig::PreUpdate(const UAnimInstance* InAnimInstance)
{
	// cache list of goal creator components on the actor (this is only done once)
	// NOTE: tried doing this in Initialize_AnyThread but it would miss some GoalCreator components
	// so it was moved here to be more robust
	if (!bSearchedForGoalCreators)
	{
		bSearchedForGoalCreators = true;

		GoalCreators.Reset();
		USkeletalMeshComponent* SkelMeshComponent = InAnimInstance->GetSkelMeshComponent();
		AActor* OwningActor = SkelMeshComponent->GetOwner();
		TArray<UActorComponent*> GoalCreatorComponents =  OwningActor->GetComponentsByInterface( UIKGoalCreatorInterface::StaticClass() );
		for (UActorComponent* GoalCreatorComponent : GoalCreatorComponents)
		{
			IIKGoalCreatorInterface* GoalCreator = Cast<IIKGoalCreatorInterface>(GoalCreatorComponent);
			if (!ensureMsgf(GoalCreator, TEXT("Goal creator component failed cast to IIKGoalCreatorInterface.")))
			{
				continue;
			}
			GoalCreators.Add(GoalCreatorComponent);
		}
	}
	
	// pull all the goals out of any goal creators on the owning actor
	// this is done on the main thread because we're talking to actor components here
	GoalsFromGoalCreators.Reset();
	for (UActorComponent* GoalCreatorComponent : GoalCreators)
    {
    	if (IIKGoalCreatorInterface* GoalCreator = Cast<IIKGoalCreatorInterface>(GoalCreatorComponent))
    	{
    		GoalCreator->AddIKGoals_Implementation(GoalsFromGoalCreators);
    	}
    }
}

void FAnimNode_IKRig::OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance)
{
	Super::OnInitializeAnimInstance(InProxy, InAnimInstance);
	
	InitializeProperties(InAnimInstance, GetTargetClass());
}

void FAnimNode_IKRig::SetProcessorNeedsInitialized()
{
	IKRigProcessor.SetNeedsInitialized();
}

FIKRigProcessor* FAnimNode_IKRig::GetIKRigProcessor()
{
	return &IKRigProcessor;
}

void FAnimNode_IKRig::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	FAnimNode_Base::CacheBones_AnyThread(Context);
	Source.CacheBones(Context);
	
	const FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();
	if (!RequiredBones.IsValid())
	{
		return;
	}
	
	// fill up node names, mapping the anim graph bone indices to the indices used by the IK Rig
	CompactPoseToRigIndices.Reset();
	const TArray<FBoneIndexType>& RequiredBonesArray = RequiredBones.GetBoneIndicesArray();
	const USkeletalMesh* SkeletalMesh = RequiredBones.GetSkeletalMeshAsset();
	if (!ensure(SkeletalMesh))
	{
		return;
	}
	const FReferenceSkeleton& MeshRefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 NumBones = RequiredBonesArray.Num();
	for (uint16 Index = 0; Index < NumBones; ++Index)
	{
		const int32 MeshBone = RequiredBonesArray[Index];
		if (!ensure(MeshBone != INDEX_NONE))
		{
			continue;
		}
		
		FCompactPoseBoneIndex CPIndex = RequiredBones.MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshBone));
		const FName Name = MeshRefSkeleton.GetBoneName(MeshBone);
		CompactPoseToRigIndices.Add(CPIndex) = MeshRefSkeleton.FindBoneIndex(Name);
	}

	// must reinitialize if bone count changes
	IKRigProcessor.SetNeedsInitialized();
}

void FAnimNode_IKRig::InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass)
{
	static const FString AlphaPosPropStr = GET_MEMBER_NAME_CHECKED(FIKRigGoal, PositionAlpha).ToString();
	static const FString AlphaRotPropStr = GET_MEMBER_NAME_CHECKED(FIKRigGoal, RotationAlpha).ToString();
	static const FString PositionPropStr = GET_MEMBER_NAME_CHECKED(FIKRigGoal, Position).ToString();
	static const FString RotationPropStr = GET_MEMBER_NAME_CHECKED(FIKRigGoal, Rotation).ToString();
	
	SourceProperties.Reset(SourcePropertyNames.Num());
	DestProperties.Reset(SourcePropertyNames.Num());
	UpdateFunctions.Reset(SourcePropertyNames.Num());

	check(SourcePropertyNames.Num() == DestPropertyNames.Num());

	const UClass* SourceClass = InSourceInstance->GetClass();
	for (int32 PropIndex = 0; PropIndex < SourcePropertyNames.Num(); ++PropIndex)
	{
		const FName& SourceName = SourcePropertyNames[PropIndex];
		
		FProperty* SourceProperty = FindFProperty<FProperty>(SourceClass, SourceName);
		SourceProperties.Add(SourceProperty);
		DestProperties.Add(nullptr);

		// property not found
		if (!SourceProperty)
		{
			continue;
		}
		
		// store update functions for later use in PropagateInputProperties to avoid looking for properties
		// while evaluating
		const FName& GoalPropertyName = DestPropertyNames[PropIndex];

		// find the right goal
		const int32 GoalIndex = Goals.IndexOfByPredicate([&GoalPropertyName](const FIKRigGoal& InGoal)
		{
			return GoalPropertyName.ToString().EndsWith(InGoal.Name.ToString());
		});

		// goal not found?
		if (!Goals.IsValidIndex(GoalIndex))
		{
			continue;
		}
		
		FIKRigGoal& Goal = Goals[GoalIndex];

		const FString GoalPropStr = GoalPropertyName.ToString();

		if (GoalPropStr.StartsWith(AlphaPosPropStr))
		{
			UpdateFunctions.Add([&Goal, SourceProperty](const UObject* InSourceInstance)
			{
				static_assert(std::is_same_v<decltype(FIKRigGoal::PositionAlpha), float>);
				const double* AlphaValue = SourceProperty->ContainerPtrToValuePtr<double>(InSourceInstance);
				Goal.PositionAlpha = FMath::Clamp<float>(static_cast<float>(*AlphaValue), 0.f, 1.f);
			});
		}
		else if (GoalPropStr.StartsWith(AlphaRotPropStr))
		{
			UpdateFunctions.Add([&Goal, SourceProperty](const UObject* InSourceInstance)
			{
				static_assert(std::is_same_v<decltype(FIKRigGoal::RotationAlpha), float>);
				const double* AlphaValue = SourceProperty->ContainerPtrToValuePtr<double>(InSourceInstance);
				Goal.RotationAlpha = FMath::Clamp<float>(static_cast<float>(*AlphaValue), 0.f, 1.f);
			});
		}
		else if (GoalPropStr.StartsWith(PositionPropStr))
		{
			UpdateFunctions.Add([&Goal, SourceProperty](const UObject* InSourceInstance)
			{
				static_assert(std::is_same_v<decltype(FIKRigGoal::Position), FVector>);
				Goal.Position = *SourceProperty->ContainerPtrToValuePtr<FVector>(InSourceInstance);
			});
		}
		else if (GoalPropStr.StartsWith(RotationPropStr))
		{
			UpdateFunctions.Add([&Goal, SourceProperty](const UObject* InSourceInstance)
			{
				static_assert(std::is_same_v<decltype(FIKRigGoal::Rotation), FRotator>);
				Goal.Rotation = *SourceProperty->ContainerPtrToValuePtr<FRotator>(InSourceInstance);
			});
		}
	}
}

void FAnimNode_IKRig::PropagateInputProperties(const UObject* InSourceInstance)
{
	if (!InSourceInstance)
	{
		return;
	}

	for (const TFunction<void(const UObject*)>& UpdateFunc : UpdateFunctions)
	{
		UpdateFunc(InSourceInstance);
	}
}

#if WITH_EDITOR
void FAnimNode_IKRig::ConditionalDebugDraw(
	FPrimitiveDrawInterface* PDI,
	USkeletalMeshComponent* PreviewSkelMeshComp) const
{
	// is anim graph setup?
	if (!(bEnableDebugDraw && PreviewSkelMeshComp && PreviewSkelMeshComp->GetWorld()))
	{
		return;
	}

	// is processor running?
	if (!IKRigProcessor.IsInitialized())
	{
		return;
	}

	// draw oriented box for each goal
	const TArray<FIKRigGoal>& ProcessorGoals = IKRigProcessor.GetGoalContainer().GetGoalArray();
	for (const FIKRigGoal& Goal : ProcessorGoals)
	{
		DrawOrientedWireBox(PDI, Goal.FinalBlendedPosition, FVector::XAxisVector, FVector::YAxisVector, FVector::ZAxisVector, FVector::One() * DebugScale, FLinearColor::Yellow, SDPG_World);
		DrawCoordinateSystem(PDI, Goal.Position, Goal.FinalBlendedRotation.Rotator(), DebugScale, SDPG_World);
	}
}
#endif
