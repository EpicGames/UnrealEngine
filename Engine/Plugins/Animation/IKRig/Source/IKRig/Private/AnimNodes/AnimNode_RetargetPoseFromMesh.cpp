// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/AnimNode_RetargetPoseFromMesh.h"

#include "IKRigObjectVersion.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Retargeter/RetargetOps/CurveRemapOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RetargetPoseFromMesh)


void FAnimNode_RetargetPoseFromMesh::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)

	FAnimNode_Base::Initialize_AnyThread(Context);

	GetEvaluateGraphExposedInputs().Execute(Context);

	if (RetargetFrom == ERetargetSourceMode::SourcePosePin)
	{
		Source.Initialize(Context);
	}
}

void FAnimNode_RetargetPoseFromMesh::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)

	FAnimNode_Base::CacheBones_AnyThread(Context);

	if (RetargetFrom == ERetargetSourceMode::SourcePosePin)
	{
		Source.CacheBones(Context);
	}

	// rebuild mapping of compact index to full skeleton bone index
	CompactToTargetBoneIndexMap.Reset();
	const TArray<FBoneIndexType>& RequiredBonesArray = Context.AnimInstanceProxy->GetRequiredBones().GetBoneIndicesArray();
	for (FBoneIndexType CompactIndex = 0; CompactIndex < RequiredBonesArray.Num(); ++CompactIndex)
	{
		CompactToTargetBoneIndexMap.Emplace(CompactIndex, RequiredBonesArray[CompactIndex]);
	}
}

void FAnimNode_RetargetPoseFromMesh::Update_AnyThread(const FAnimationUpdateContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)

	FAnimNode_Base::Update_AnyThread(Context);

	// this introduces a frame of latency in setting the pin-driven source component,
    // but we cannot do the work to extract transforms on a worker thread as it is not thread safe.
    GetEvaluateGraphExposedInputs().Execute(Context);

	// delta time stored here and passed to retargeter
	// NOTE: this must be accumulated as Update can be called multiple times
	DeltaTime += Context.GetDeltaTime();

	if (RetargetFrom == ERetargetSourceMode::SourcePosePin)
	{
		// update pose input
		Source.Update(Context);
	}

	// get the variable values from the input pins
	CopyInputPropertiesToVariables(Context.AnimInstanceProxy->GetAnimInstanceObject());
}

void FAnimNode_RetargetPoseFromMesh::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	SCOPE_CYCLE_COUNTER(STAT_IKRetarget);

	// editor profiling
#ifdef WITH_EDITOR
	double StartTime = FPlatformTime::Seconds();
#endif

	auto IsReadyToRun = [this, &Output]()
	{
		if (!IKRetargeterAsset)
		{
			return false; // must have retarget asset
		}
		
		// ensure the processor is initialized
		if (!EnsureProcessorIsInitialized(Output.AnimInstanceProxy->GetSkelMeshComponent()))
		{
			return false; // must be initialized
		}

		// validate source data from other component
		const bool bCopyingFromOtherComponent = RetargetFrom != ERetargetSourceMode::SourcePosePin;
		if (bCopyingFromOtherComponent)
		{
			if (!SourceMeshComponent.IsValid())
			{
				return false; // copying from another mesh but no source component was found
			}

			if (!SourceMeshComponent->GetSkeletalMeshAsset())
			{
				return false; // if copying from another mesh, must have valid pointer to it
			}
			
			if (PoseToRetargetFromComponentSpace.IsEmpty())
			{
				return false; // if copying from another mesh, pose must be copied already from the PreUpdate()
			}

			const int32 NumBonesInSource = Processor.GetSkeleton(ERetargetSourceOrTarget::Source).BoneNames.Num();
			if (NumBonesInSource != PoseToRetargetFromComponentSpace.Num())
			{
				return false; // the source pose has not been copied yet (can happen if Evaluate() is called before PreUpdate() with a different mesh)	
			}
		}

		if (!bCopyingFromOtherComponent && !Source.GetLinkNode())
		{
			return false; // if copying from input pin, must be connected
		}

		if (!IsLODEnabled(Output.AnimInstanceProxy))
		{
			return false; // LOD'd out
		}

		// phew! ready to run
		return true;
	};

	auto FillPoseToRetargetFromInputPin = [this, &Output]()
	{
		// NOTE: in this case, we are copying the input pose from the anim graph pin, so the source and the target skeleton are the same!
		// Retargeting between the same mesh allows retarget operations to modify or reinterpret a pose in some way.
		// But the input pose is local and may be compacted, so we need to reconstruct the full component space pose to pass to the retargeter...

		// start with the full ref pose
		const USkeletalMesh* TargetMesh = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetSkeletalMeshAsset();
		const FReferenceSkeleton& TargetRefSkeleton = TargetMesh->GetRefSkeleton();
		const TArray<FTransform>& TargetLocalRefPose = TargetRefSkeleton.GetRefBonePose();;

		// start with full local ref pose
		InputLocalTransforms.Reset(TargetLocalRefPose.Num());
		InputLocalTransforms = TargetRefSkeleton.GetRefBonePose();
		
		// use the input pose from the anim graph as the pose to retarget from
		if (Source.GetLinkNode())
		{
			// evaluate the input pose
			Source.Evaluate(Output);

			// overwrite required bones with input pose from the anim graph
			for (const TPair<int32, int32>& Pair : CompactToTargetBoneIndexMap)
			{
				const FCompactPoseBoneIndex CompactBoneIndex(Pair.Key);
				if (Output.Pose.IsValidIndex(CompactBoneIndex))
				{
					const int32 TargetBoneIndex = Pair.Value;
					InputLocalTransforms[TargetBoneIndex] = Output.Pose[CompactBoneIndex];
				}
			}
		}
		
		// convert to component space
		FAnimationRuntime::FillUpComponentSpaceTransforms(TargetRefSkeleton, InputLocalTransforms, PoseToRetargetFromComponentSpace);

		// scale it according to the scale source op (if used)
		Processor.ApplySourceScaleToPose(PoseToRetargetFromComponentSpace);
	};
	
	auto ApplyRetargetedPose = [this, &Output](const TArray<FTransform>& RetargetedPose)
	{
		// convert pose to local space and apply to output
		FCSPose<FCompactPose> ComponentPose;
		ComponentPose.InitPose(Output.Pose);
		const FCompactPose& CompactPose = ComponentPose.GetPose();
		for (const TPair<int32, int32>& Pair : CompactToTargetBoneIndexMap)
		{
			const FCompactPoseBoneIndex CompactBoneIndex(Pair.Key);
			if (CompactPose.IsValidIndex(CompactBoneIndex))
			{
				const int32 TargetBoneIndex = Pair.Value;
				ComponentPose.SetComponentSpaceTransform(CompactBoneIndex, RetargetedPose[TargetBoneIndex]);
			}
		}

		// convert to local space
		FCSPose<FCompactPose>::ConvertComponentPosesToLocalPoses(ComponentPose, Output.Pose);

		// once converted back to local space, we copy scale values back
		// (retargeter strips scale values and deals with translation only in component space)
		const TObjectPtr<USkeletalMesh> TargetMesh = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetSkeletalMeshAsset();
		const TArray<FTransform>& RefPose = TargetMesh->GetRefSkeleton().GetRefBonePose();
		for (const TPair<int32, int32>& Pair : CompactToTargetBoneIndexMap)
		{
			const FCompactPoseBoneIndex CompactBoneIndex(Pair.Key);
			if (Output.Pose.IsValidIndex(CompactBoneIndex))
			{
				const FVector ScaleFromRetarget = Output.Pose[CompactBoneIndex].GetScale3D();
				const FVector ScaleFromSkeletalMesh = RefPose[Pair.Value].GetScale3D() - FVector::OneVector;
				const FVector NewScale = ScaleFromRetarget + ScaleFromSkeletalMesh;
				Output.Pose[CompactBoneIndex].SetScale3D(NewScale);
			}
		}
	};

	// check the configuration of the node and output the reference pose unless it's ready to retarget
	if (!IsReadyToRun())
	{
		Output.ResetToRefPose();
		return;
	}

	// if we're retargeting the connected input pose from the anim graph, then evaluate it and get the pose to retarget
	if (RetargetFrom == ERetargetSourceMode::SourcePosePin)
	{
		FillPoseToRetargetFromInputPin(); // (also gets speed curves from anim graph)
	}

	// LOD off the IK pass
	const int32 CurrentLODLevel = Output.AnimInstanceProxy->GetLODLevel();
	bool bForceIKOff = LODThresholdForIK != INDEX_NONE && CurrentLODLevel > LODThresholdForIK;
	FRetargetProfile RetargetProfileToUse = GetMergedRetargetProfile(bForceIKOff);

	// get the property overrides
	const TArray<FName>* OverridesToUse = bUseCustomOverrideSets ? &OverrideSetsToApply : &IKRetargeterAsset->GetOverrideSetsToApply();

	// give retarget ops a chance to access data from the anim graph before running
	Processor.OnAnimGraphEvaluateAnyThread(Output);

	// run the retargeter
	FRetargetRunParameters Params;
	Params.SourceGlobalPose = &PoseToRetargetFromComponentSpace;
	Params.Profile = &RetargetProfileToUse;
	Params.OverrideSetsToApply = *OverridesToUse;
	Params.Variables = &RuntimeVariables;
	Params.BoundCurveValues = &BoundCurveValues;
	Params.DeltaTime = DeltaTime;
	Params.LOD = CurrentLODLevel;
	const TArray<FTransform>& RetargetedPose = Processor.RunRetargeter(Params);

	// reset delta time (it's accumulated)
	DeltaTime = 0.f;

	// apply the retargeted pose to the output pin
	ApplyRetargetedPose(RetargetedPose);

	// editor profiling
#if WITH_EDITOR
	constexpr double FrameWindow = 30.0;
	constexpr double Alpha = 2.0 / (FrameWindow + 1.0);
	AverageExecutionTime += Alpha * ((FPlatformTime::Seconds() - StartTime) - AverageExecutionTime);
#endif
}

void FAnimNode_RetargetPoseFromMesh::PreUpdate(const UAnimInstance* InAnimInstance)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(PreUpdate)

	auto GetPoseToRetargetFromSourceMesh = [this](const USkeletalMeshComponent* TargetMeshComponent)
	{
		// skip if copying from a source pin
		const bool bCopyingFromOtherComponent = RetargetFrom != ERetargetSourceMode::SourcePosePin;
		if (!bCopyingFromOtherComponent)
		{
			return;
		}

		// source mesh not connected or not found
		if (!SourceMeshComponent.IsValid())
		{
			return;
		}

		// if our source is running under leader-pose, then get bone data from there
		USkeletalMeshComponent* ComponentToCopyFrom =  SourceMeshComponent.Get();
		if(USkeletalMeshComponent* LeaderPoseComponent = Cast<USkeletalMeshComponent>(SourceMeshComponent->LeaderPoseComponent.Get()))
		{
			ComponentToCopyFrom = LeaderPoseComponent;
		}
	
		// skip copying pose when component is no longer ticking
		if (!ComponentToCopyFrom->IsRegistered())
		{
			return; 
		}
	
		const bool bUROInSync =
			ComponentToCopyFrom->ShouldUseUpdateRateOptimizations() &&
			ComponentToCopyFrom->AnimUpdateRateParams != nullptr &&
			SourceMeshComponent->AnimUpdateRateParams == TargetMeshComponent->AnimUpdateRateParams;
		const bool bUsingExternalInterpolation = ComponentToCopyFrom->IsUsingExternalInterpolation();
		const TArray<FTransform>& CachedComponentSpaceTransforms = ComponentToCopyFrom->GetCachedComponentSpaceTransforms();
		const bool bArraySizesMatch = CachedComponentSpaceTransforms.Num() == ComponentToCopyFrom->GetComponentSpaceTransforms().Num();

		// copy source array from the appropriate location
		PoseToRetargetFromComponentSpace.Reset();
		if ((bUROInSync || bUsingExternalInterpolation) && bArraySizesMatch)
		{
			// copy from source's cache
			PoseToRetargetFromComponentSpace.Append(CachedComponentSpaceTransforms);
		}
		else
		{
			// copy directly
			PoseToRetargetFromComponentSpace.Append(ComponentToCopyFrom->GetComponentSpaceTransforms());
		}
	
		// strip all scale out of the pose values, the translation of a component-space pose has incorporated scale values
		for (FTransform& Transform : PoseToRetargetFromComponentSpace)
		{
			Transform.SetScale3D(FVector::OneVector);
		}

		Processor.ApplySourceScaleToPose(PoseToRetargetFromComponentSpace);
	};

	// copy all the data from the source component
	USkeletalMeshComponent* TargetMeshComponent = InAnimInstance->GetSkelMeshComponent();
	if (Processor.IsInitialized() && RetargetFrom != ERetargetSourceMode::SourcePosePin)
	{
		GetPoseToRetargetFromSourceMesh(TargetMeshComponent);

		if (SourceMeshComponent.IsValid() && TargetMeshComponent)
		{
			Processor.OnAnimGraphPreUpdateMainThread(*SourceMeshComponent.Get(), *TargetMeshComponent);
		}
	}

	// get the curve values used by property overrides that are bound to curves
	const bool bCopyingFromOtherComponent = RetargetFrom != ERetargetSourceMode::SourcePosePin;
	TWeakObjectPtr<USkeletalMeshComponent> ComponentWithCurves = bCopyingFromOtherComponent ? SourceMeshComponent : TargetMeshComponent;
	if (ComponentWithCurves.IsValid())
	{
		CopyBoundCurveValues(ComponentWithCurves->GetAnimInstance());
	}
}

FIKRetargetProcessor* FAnimNode_RetargetPoseFromMesh::GetRetargetProcessor()
{
	return &Processor;
}

bool FAnimNode_RetargetPoseFromMesh::EnsureProcessorIsInitialized(const USkeletalMeshComponent* TargetMeshComponent)
{
	if (!ensure(TargetMeshComponent))
	{
		return false;
	}
	
	// has user supplied a retargeter asset?
	if (!IKRetargeterAsset)
	{
		return false;
	}
	
	// if user hasn't explicitly connected a source mesh, optionally use the parent mesh component (if there is one)
	if (RetargetFrom == ERetargetSourceMode::ParentSkeletalMeshComponent && !bSearchedForParentComponent)
	{
		bSearchedForParentComponent = true;
		SourceMeshComponent.Reset();
		
		// walk up component hierarchy until we find a skeletal mesh component
		for (USceneComponent* AttachParentComp = TargetMeshComponent->GetAttachParent(); AttachParentComp != nullptr; AttachParentComp = AttachParentComp->GetAttachParent())
		{
			SourceMeshComponent = Cast<USkeletalMeshComponent>(AttachParentComp);
			if (SourceMeshComponent.IsValid())
			{
				break;
			}
		}
	}
	else
	{
		bSearchedForParentComponent = false;
	}
	
	// has a source mesh been plugged in or found?
	const bool bCopyingFromOtherComponent = RetargetFrom != ERetargetSourceMode::SourcePosePin;
	if (bCopyingFromOtherComponent && !SourceMeshComponent.IsValid())
	{
		if (!SourceMeshComponent.IsValid())
		{
			return false; // can't do anything if we don't have a source mesh component
		}

		if (!ensureMsgf(SourceMeshComponent != TargetMeshComponent, TEXT("Cannot use target component as source.")))
		{
			return false; // we do not support retargeting between the same component
		}
	}

	// check that both a source and target mesh exist
	const USkeletalMesh* TargetMesh = TargetMeshComponent->GetSkeletalMeshAsset();
	const USkeletalMesh* SourceMesh = bCopyingFromOtherComponent ? SourceMeshComponent->GetSkeletalMeshAsset() : TargetMesh;
	if (!(SourceMesh && TargetMesh))
	{
		return false; // cannot initialize if components are missing skeletal mesh references
	}
	
	// try initializing the processor
	if (!Processor.WasInitializedWithTheseAssets(SourceMesh, TargetMesh, IKRetargeterAsset))
	{
		constexpr bool bForceIKOff = false;
		FRetargetProfile RetargetProfileToUse = GetMergedRetargetProfile(bForceIKOff);
		
		// initialize retarget processor with source and target skeletal meshes
		FRetargetInitParameters Params;
		Params.SourceSkeletalMesh = SourceMesh;
		Params.TargetSkeletalMesh = TargetMesh;
		Params.RetargeterAsset = IKRetargeterAsset;
		Params.CustomProfile = &RetargetProfileToUse;
		Params.bSuppressWarnings = false;
		Processor.Initialize(Params);
	}

	return Processor.IsInitialized();
}

FRetargetProfile FAnimNode_RetargetPoseFromMesh::GetMergedRetargetProfile(bool bForceIKOff) const
{
	// collect settings to retarget with starting with asset settings and overriding with custom profile
	FRetargetProfile Profile;
	Profile.FillProfileWithAssetSettings(IKRetargeterAsset);
	// load custom profile plugged into the anim node
	Profile.MergeWithOtherProfile(CustomRetargetProfile);
	// force all IK off (skips IK solve)
	Profile.bForceAllIKOff = bForceIKOff;
	return MoveTemp(Profile);
}

bool FAnimNode_RetargetPoseFromMesh::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);
	// doesn't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FAnimNode_RetargetPoseFromMesh::PostSerialize(const FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FIKRigObjectVersion::GUID) < FIKRigObjectVersion::UseAttachedParentDeprecated)
		{
			RetargetFrom = bUseAttachedParent_DEPRECATED ? ERetargetSourceMode::ParentSkeletalMeshComponent : ERetargetSourceMode::CustomSkeletalMeshComponent;
		}
	}
	
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
double FAnimNode_RetargetPoseFromMesh::GetAverageExecutionTime() const
{
	return AverageExecutionTime;
}
#endif

void FAnimNode_RetargetPoseFromMesh::CopyInputPropertiesToVariables(const UObject* InSourceInstance)
{
	if (!InSourceInstance || !IKRetargeterAsset)
	{
		return;
	}

	// start the bag with the hardcoded values from the asset (possibly overridden by pin values)
	RuntimeVariables = IKRetargeterAsset->GetVariables().Bag;

	const UPropertyBag* BagStruct = RuntimeVariables.GetPropertyBagStruct();
	if (!BagStruct)
	{
		return;
	}

	UClass* SourceClass = InSourceInstance->GetClass();

	// get the mutable memory address of the Property Bag so we can write to it
	uint8* BagMemory = RuntimeVariables.GetMutableValue().GetMemory();
	if (!BagMemory)
	{
		return;
	}

	if (!ensure(SourcePropertyNames.Num() == DestPropertyNames.Num()))
	{
		return; // should not happen
	}

	// iterate through the compiler-generated mappings and copy pin values to the variables property bag
	for (int32 PropIdx = 0; PropIdx < SourcePropertyNames.Num(); ++PropIdx)
	{
		const FName& SourceName = SourcePropertyNames[PropIdx];
		const FName& DestName = DestPropertyNames[PropIdx]; 

		FProperty* SourceProp = SourceClass->FindPropertyByName(SourceName);
		FProperty* BagProp = BagStruct->FindPropertyByName(DestName);
		if (!SourceProp || !BagProp)
		{
			continue;
		}
		
		const void* SourceValuePtr = SourceProp->ContainerPtrToValuePtr<void>(InSourceInstance);
		void* BagValuePtr = BagProp->ContainerPtrToValuePtr<void>(BagMemory);

		// EXACT MATCH: fast mem copy (ie, bool to bool, int32 to int32)
		if (SourceProp->SameType(BagProp))
		{
			SourceProp->CopyCompleteValue(BagValuePtr, SourceValuePtr);
		}
		// NUMERIC MISMATCH: safely coerce numbers...
		else if (FNumericProperty* SourceNum = CastField<FNumericProperty>(SourceProp))
		{
			if (FNumericProperty* BagNum = CastField<FNumericProperty>(BagProp))
			{
				// handle float type bridging (double, float)
				if (SourceNum->IsFloatingPoint() && BagNum->IsFloatingPoint())
				{
					double Val = SourceNum->GetFloatingPointPropertyValue(SourceValuePtr);
					BagNum->SetFloatingPointPropertyValue(BagValuePtr, Val);
				}
				// handle integer type bridging (int64, int32, byte)
				else if (SourceNum->IsInteger() && BagNum->IsInteger())
				{
					int64 Val = SourceNum->GetSignedIntPropertyValue(SourceValuePtr);
					BagNum->SetIntPropertyValue(BagValuePtr, Val);
				}
			}
		}
		// COMPLEX MISMATCH: handle struct bridging (ie, FVector3d -> FVector3f)
		else
		{
			// string export/import handles struct precision differences (ie, LWC vector to float vector etc)
			FString TempStr;
			SourceProp->ExportTextItem_Direct(TempStr, SourceValuePtr, nullptr, nullptr, PPF_None);
			BagProp->ImportText_Direct(*TempStr, BagValuePtr, nullptr, PPF_None);
		}
	}
}

void FAnimNode_RetargetPoseFromMesh::CopyBoundCurveValues(const UObject* InSourceInstance)
{
	const UAnimInstance* AnimInstance = Cast<UAnimInstance>(InSourceInstance);
	if (!AnimInstance)
	{
		return;
	}
	
	if (!IKRetargeterAsset)
	{
		return;
	}

	// rebuild the TMap with the latest bound curve names
	if (CachedOverrideVersion != IKRetargeterAsset->GetOverrideVersion())
	{
		BoundCurveValues.Reset();
		
		const TMap<FName, FRetargetOverrideSet>& AllOverrideSets = IKRetargeterAsset->GetOverrideSets();
		for (const TTuple<FName, FRetargetOverrideSet>& OverrideSet : AllOverrideSets)
		{
			for (const FRetargetOpOverrides& OpOverrides : OverrideSet.Value.OpOverrides)
			{
				for (const FRetargetOpPropertyOverride& PropertyOverride : OpOverrides.PropertyOverrides)
				{
					const FName BoundCurveName = PropertyOverride.GetBoundCurveName();
					if (BoundCurveName.IsNone())
					{
						continue;
					}
					BoundCurveValues.Emplace(BoundCurveName, 0.0f);
				}
			}
		}
		
		CachedOverrideVersion = IKRetargeterAsset->GetOverrideVersion();
	}

	// fill the map with values from the curves
	const TMap<FName, float>& AnimCurveList = AnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve);
	for (TTuple<FName, float>& BoundCurve : BoundCurveValues)
	{
		if (const float* AnimCurveValue = AnimCurveList.Find(BoundCurve.Key))
		{
			BoundCurve.Value = *AnimCurveValue;
		}
	}
}

