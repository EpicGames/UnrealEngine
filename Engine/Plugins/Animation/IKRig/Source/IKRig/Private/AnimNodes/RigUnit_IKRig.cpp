// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNodes/RigUnit_IKRig.h"

#include "ControlRigComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_IKRig)

FRigUnit_IKRig_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT()
	LLM_SCOPE_BYNAME(TEXT("Animation/IKRig"));

	URigHierarchy* Hierarchy = ExecuteContext.Hierarchy;
	if (Hierarchy == nullptr)
	{
		return;
	}

	auto GetSkeletalMesh = [&ExecuteContext]() ->USkeletalMesh*
	{
		const USkeletalMeshComponent* OwningComponent = Cast<USkeletalMeshComponent>(ExecuteContext.GetOwningComponent());
		if (!OwningComponent)
		{
			return nullptr;
		}

		return OwningComponent->GetSkeletalMeshAsset();
	};

	FIKRigProcessor& Processor = WorkData.IKRigProcessor;

	if (!Processor.IsInitialized())
	{
		Processor.Initialize(IKRigAsset, GetSkeletalMesh(), FIKRigGoalContainer());
		if (Processor.IsInitialized())
		{
			CacheRigElements(WorkData.CachedRigElements, Processor.GetSkeleton().BoneNames, Hierarchy);
		}
	}
	if (!Processor.IsInitialized())
	{
		return;
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		// live preview source asset settings in the retarget editor
		// NOTE: this copies solver settings and goal.PositionAlpha and goal.RotationAlpha
		Processor.CopyAllSettingsFromAsset(IKRigAsset);
	}
#endif

	// set the goals
	FIKRigGoalContainer& GoalContainer = Processor.GetGoalContainer();
	
	// default all goals to 0 pos/rot alpha (overridden by user supplied goals)
	for (FIKRigGoal& Goal : GoalContainer.GetGoalArray())
	{
		Goal.PositionAlpha = 0.0; 
		Goal.RotationAlpha = 0.0;
	}

	// apply input goals to the processor
	for (const FIKRigGoalInput& GoalInput : Goals)
	{
		if (FIKRigGoal* Goal = GoalContainer.FindGoalByName(GoalInput.GoalName))
		{
			Goal->bEnabled = true;
			Goal->Position = GoalInput.Transform.GetLocation();
			Goal->Rotation = GoalInput.Transform.GetRotation().Rotator();
			Goal->PositionAlpha = GoalInput.PositionAlpha;
			Goal->RotationAlpha = GoalInput.RotationAlpha;
			Goal->PositionSpace = EIKRigGoalSpace::Component;
			Goal->RotationSpace = EIKRigGoalSpace::Component;
		}
	}
	
	// copy input pose to start IK solve from
	for (int32 BoneIndex=0; BoneIndex<WorkData.CachedRigElements.Num(); BoneIndex++)
	{
		const FCachedRigElement& CachedRigElement = WorkData.CachedRigElements[BoneIndex];
		if (!ensure(CachedRigElement.IsValid()))
		{
			continue;
		}
		const FTransform BoneGlobalTransform = Hierarchy->GetGlobalTransform(CachedRigElement.GetIndex());
		Processor.SetInputBoneGlobal(BoneIndex, BoneGlobalTransform);
	}
	
	// run IK solve
	Processor.Solve();
	
	// copy results of solve
	const TArray<FTransform>& OutputPose = Processor.GetSkeleton().CurrentPoseGlobal;
	for (int32 BoneIndex=0; BoneIndex<OutputPose.Num(); BoneIndex++)
	{
		const FCachedRigElement& CachedRigElement = WorkData.CachedRigElements[BoneIndex];
		if (!CachedRigElement.IsValid())
		{
			continue;
		}

		static bool bInitial = false;
		static bool bAffectChildren = false;
		static bool bSetupUndo = false;
		Hierarchy->SetGlobalTransform(CachedRigElement.GetKey(), OutputPose[BoneIndex], bInitial, bAffectChildren, bSetupUndo);
	}
}

void FRigUnit_IKRig::CacheRigElements(
	TArray<FCachedRigElement>& OutMap,
	const TArray<FName>& InBoneNames,
	URigHierarchy* InHierarchy)
{
	OutMap.Reset(InBoneNames.Num());
    
	// cache a rig element for each bone in the skeletal mesh
	for (const FName& BoneName : InBoneNames)
	{
		FRigElementKey RigElementKey = FRigElementKey(BoneName, ERigElementType::Bone);
		OutMap.Emplace(RigElementKey, InHierarchy);
	}
}
