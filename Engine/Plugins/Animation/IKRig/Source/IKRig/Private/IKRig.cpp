// Copyright Epic Games, Inc. All Rights Reserved.

#include "IKRig.h"

#include "IKRigObjectVersion.h"
#include "UObject/DevObjectVersion.h"

#define LOCTEXT_NAMESPACE "FIKRigModule"

IMPLEMENT_MODULE(FIKRigModule, IKRig)

// Unique IK Rig Object version id
const FGuid FIKRigObjectVersion::GUID(0xF6DFBB78, 0xBB50A0E4, 0x4018B84D, 0x60CBAF23);
// Register custom version with Core
FDevVersionRegistration GRegisterIKRigObjectVersion(FIKRigObjectVersion::GUID, FIKRigObjectVersion::LatestVersion, TEXT("Dev-IKRig"));

void FIKRigModule::StartupModule()
{
	// If UAF is loaded, then let's enable our UAF-compatible module.
	if (FModuleManager::Get().IsModuleLoaded("UAF"))
	{
		FModuleManager::Get().LoadModule("IKRigUAF");
	}
}

void FIKRigModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE
