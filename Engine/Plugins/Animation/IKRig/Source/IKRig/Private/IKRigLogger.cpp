// Copyright Epic Games, Inc. All Rights Reserved.

#include "IKRigLogger.h"
#include "Logging/MessageLog.h"
#include "Retargeter/IKRetargeter.h"
#include "Rig/IKRigDefinition.h"

#if WITH_EDITOR
#include "Modules/ModuleManager.h"
#include "MessageLogModule.h"
#endif

#define LOCTEXT_NAMESPACE "FIKRigLogger"

DEFINE_LOG_CATEGORY(LogIKRig);

void FIKRigLogger::SetLogTarget(const UObject* InAsset)
{
	FText LogLabel;
	if (const UIKRigDefinition* IKRig = Cast<UIKRigDefinition>(InAsset))
	{
		LogName = FName("IKRig_", IKRig->GetUniqueID());
		LogLabel =  LOCTEXT("IKRigLogName", "IK Rig Log");
	}
	else if (const UIKRetargeter* Retargeter = Cast<UIKRetargeter>(InAsset))
	{
		LogName = FName("IKRetarget_", Retargeter->GetUniqueID());
		LogLabel = LOCTEXT("IKRetargetLogName", "IK Retarget Log");
	}
	else
	{
		checkNoEntry();
		return;
	}

#if WITH_EDITOR
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if (!MessageLogModule.IsRegisteredLogListing(LogName))
	{
		FMessageLogInitializationOptions InitOptions;
		InitOptions.bShowFilters = false;
		InitOptions.bShowPages = false;
		InitOptions.bAllowClear = false;
		InitOptions.bShowInLogWindow = false;
		InitOptions.bDiscardDuplicates = true;
		MessageLogModule.RegisterLogListing(LogName, LogLabel, InitOptions);
	}
#endif
	
}

FName FIKRigLogger::GetLogTarget() const
{
	return LogName;
}

void FIKRigLogger::LogError(const FText& Message) const
{
	if (LogName != NAME_None)
	{
		FMessageLog(LogName).SuppressLoggingToOutputLog(true).Error(Message);	
	}
	
	Errors.Add(Message);
}

void FIKRigLogger::LogWarning(const FText& Message) const
{
	if (LogName != NAME_None)
	{
		FMessageLog(LogName).SuppressLoggingToOutputLog(true).Warning(Message);	
	}
	
	Warnings.Add(Message);
}

void FIKRigLogger::LogInfo(const FText& Message) const
{
	if (LogName != NAME_None)
	{
		FMessageLog(LogName).SuppressLoggingToOutputLog(true).Info(Message);
	}
	
	Messages.Add(Message);
}

void FIKRigLogger::Clear() const
{
	Errors.Empty();
	Warnings.Empty();
	Messages.Empty();
}

#undef LOCTEXT_NAMESPACE
