// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargetChainMapping.h"

#include "Algo/LevenshteinDistance.h"
#include "Retargeter/IKRetargetDeprecated.h"
#include "Retargeter/IKRetargetSettings.h"
#include "Rig/IKRigDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetChainMapping)

void FRetargetChainMapping::ReinitializeWithIKRigs(const UIKRigDefinition* InSourceIKRig, const UIKRigDefinition* InTargetIKRig)
{
	SourceIKRig = InSourceIKRig;
	TargetIKRig = InTargetIKRig;
	
	if (!ChainMap.IsEmpty() && !(IsValid(SourceIKRig) && IsValid(TargetIKRig)))
	{
		// don't clean data until we have a replacement
		// we want to preserve existing mappings in case they are still relevant once the op is configured with both rigs
		return;
	}

	if (!IsValid(TargetIKRig))
	{
		// need at least a target before we can add target chains
		return;
	}
	
	// remove null target chains
	ChainMap.RemoveAll([](const FRetargetChainPair& ChainPair)
	{
		return ChainPair.TargetChainName == NAME_None;
	});
	
	// remove chains in the map that are not in the IK Rig
	TArray<FName> TargetChainNames = TargetIKRig->GetRetargetChainNames();
	ChainMap.RemoveAll([&TargetChainNames](const FRetargetChainPair& ChainPair)
	{
		return !TargetChainNames.Contains(ChainPair.TargetChainName);
	});

	// add any chains in the rig that are not in the mapping
	for (const FName& RequiredChain : TargetChainNames)
	{
		if (!HasChain(RequiredChain, ERetargetSourceOrTarget::Target))
		{
			SetChainMapping(RequiredChain, NAME_None);
		}
	}
	
	// reset any sources that are no longer present to "None"
	TArray<FName> SourceChainNames = SourceIKRig ? SourceIKRig->GetRetargetChainNames() : TArray<FName>();
	for (FRetargetChainPair& Pair : ChainMap)
	{
		if (!SourceChainNames.Contains(Pair.SourceChainName))
		{
			Pair.SourceChainName = NAME_None;
		}
	}

	// enforce the chain order based on the StartBone index
	SortMapping();
}

bool FRetargetChainMapping::IsReady() const
{
	return SourceIKRig && TargetIKRig && !ChainMap.IsEmpty();
}

bool FRetargetChainMapping::HasAnyChains() const
{
	return !ChainMap.IsEmpty();
}

const UIKRigDefinition* FRetargetChainMapping::GetIKRig(ERetargetSourceOrTarget SourceOrTarget)
{
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceIKRig : TargetIKRig;
}

bool FRetargetChainMapping::HasChain(const FName& InChainName, ERetargetSourceOrTarget InSourceOrTarget) const
{
	if (InSourceOrTarget == ERetargetSourceOrTarget::Target)
	{
		return ChainMap.ContainsByPredicate([InChainName](const FRetargetChainPair& InChainPair)
		{
			return InChainPair.TargetChainName == InChainName;
		});
	}
	else
	{
		return ChainMap.ContainsByPredicate([InChainName](const FRetargetChainPair& InChainPair)
		{
			return InChainPair.SourceChainName == InChainName;
		});
	}
}

FName FRetargetChainMapping::GetChainMappedTo(const FName InChainName, ERetargetSourceOrTarget InSourceOrTarget) const
{
	if (const FRetargetChainPair* FoundChainPair = FindChainPair(InChainName, InSourceOrTarget))
	{
		return InSourceOrTarget == ERetargetSourceOrTarget::Source ? FoundChainPair->TargetChainName : FoundChainPair->SourceChainName;
	}
	
	return NAME_None;
}

FRetargetChainPair* FRetargetChainMapping::FindChainPair(const FName& InChainName, ERetargetSourceOrTarget InSourceOrTarget) const
{
	return const_cast<FRetargetChainPair*>(FindChainPairConst(InChainName, InSourceOrTarget));
}

const FRetargetChainPair* FRetargetChainMapping::FindChainPairConst(const FName& InChainName, ERetargetSourceOrTarget InSourceOrTarget) const
{
	for (const FRetargetChainPair& ChainPair : ChainMap)
	{
		const FName& NameToCompare = InSourceOrTarget == ERetargetSourceOrTarget::Source ? ChainPair.SourceChainName : ChainPair.TargetChainName;
		if (NameToCompare == InChainName)
		{
			return &ChainPair;
		}
	}
	return nullptr;
}

TArray<FRetargetChainPair>& FRetargetChainMapping::GetChainPairs()
{
	return ChainMap;
}

const TArray<FRetargetChainPair>& FRetargetChainMapping::GetChainPairs() const
{
	return ChainMap;
}

TArray<FName> FRetargetChainMapping::GetChainNames(ERetargetSourceOrTarget InSourceOrTarget) const
{
	const UIKRigDefinition* IKRig = InSourceOrTarget == ERetargetSourceOrTarget::Source ?  SourceIKRig : TargetIKRig;
	if (!IKRig)
	{
		return TArray<FName>();
	}
	
	TArray<FName> ChainNames = IKRig->GetRetargetChainNames();
	return MoveTemp(ChainNames);
}

void FRetargetChainMapping::SetChainMapping(const FName InTargetChainName, const FName InSourceChainName)
{
	if (FRetargetChainPair* FoundChainPair = FindChainPair(InTargetChainName, ERetargetSourceOrTarget::Target))
	{
		FoundChainPair->SourceChainName = InSourceChainName;
		return;
	}

	ChainMap.Add(FRetargetChainPair(InTargetChainName, InSourceChainName));
}

void FRetargetChainMapping::SortMapping()
{
	if (!IsValid(TargetIKRig))
	{
		return;
	}
	
	ChainMap.Sort([this](const FRetargetChainPair& A, const FRetargetChainPair& B)
	{
		const TArray<FBoneChain>& BoneChains = TargetIKRig->GetRetargetChains();
		const FIKRigSkeleton& TargetSkeleton = TargetIKRig->GetSkeleton();

		// look for chains
		const int32 IndexA = BoneChains.IndexOfByPredicate([&A](const FBoneChain& Chain)
		{
			return A.TargetChainName == Chain.ChainName;
		});

		const int32 IndexB = BoneChains.IndexOfByPredicate([&B](const FBoneChain& Chain)
		{
			return B.TargetChainName == Chain.ChainName;
		});

		// compare their StartBone Index 
		if (IndexA > INDEX_NONE && IndexB > INDEX_NONE)
		{
			const int32 StartBoneIndexA = TargetSkeleton.GetBoneIndexFromName(BoneChains[IndexA].StartBone.BoneName);
			const int32 StartBoneIndexB = TargetSkeleton.GetBoneIndexFromName(BoneChains[IndexB].StartBone.BoneName);

			if (StartBoneIndexA == StartBoneIndexB)
			{
				// fallback to sorting alphabetically
				return BoneChains[IndexA].ChainName.LexicalLess(BoneChains[IndexB].ChainName);
			}
				
			return StartBoneIndexA < StartBoneIndexB;	
		}

		// sort them according to the target ik rig if previously failed 
		return IndexA < IndexB;
	});
}

void FRetargetChainMapping::AutoMapChains(const EAutoMapChainType AutoMapType, const bool bForceRemap)
{
	// first clean the map (removes stale/invalid chains)
	ReinitializeWithIKRigs(SourceIKRig, TargetIKRig);
	
	// iterate over all the chain pairs and find matching source chain
	TArray<FName> SourceChainNames = GetChainNames(ERetargetSourceOrTarget::Source);
	TArray<FRetargetChainPair>& ChainPairs = GetChainPairs();
	for (FRetargetChainPair& ChainPair : ChainPairs)
	{
		const bool bIsMappedAlready = ChainPair.SourceChainName != NAME_None;
		if (bIsMappedAlready && !bForceRemap)
		{
			continue; // already set by user
		}

		// find a source chain to map to
		int32 SourceChainIndexToMapTo = -1;

		switch (AutoMapType)
		{
		case EAutoMapChainType::Fuzzy:
			{
				// auto-map chains using a fuzzy string comparison
				FString TargetNameLowerCase = ChainPair.TargetChainName.ToString().ToLower();
				float HighestScore = 0.2f;
				for (int32 ChainIndex=0; ChainIndex<SourceChainNames.Num(); ++ChainIndex)
				{
					FString SourceNameLowerCase = SourceChainNames[ChainIndex].ToString().ToLower();
					float WorstCase = static_cast<float>(TargetNameLowerCase.Len() + SourceNameLowerCase.Len());
					WorstCase = WorstCase < 1.0f ? 1.0f : WorstCase;
					const float Score = 1.0f - (static_cast<float>(Algo::LevenshteinDistance(TargetNameLowerCase, SourceNameLowerCase)) / WorstCase);
					if (Score > HighestScore)
					{
						HighestScore = Score;
						SourceChainIndexToMapTo = ChainIndex;
					}
				}
				break;
			}
		case EAutoMapChainType::Exact:
			{
				// if no exact match is found, then set to None
				ChainPair.SourceChainName = NAME_None;
				
				// auto-map chains with EXACT same name
				for (int32 ChainIndex=0; ChainIndex<SourceChainNames.Num(); ++ChainIndex)
				{
					if (SourceChainNames[ChainIndex] == ChainPair.TargetChainName)
					{
						SourceChainIndexToMapTo = ChainIndex;
						break;
					}
				}
				break;
			}
		case EAutoMapChainType::Clear:
			{
				ChainPair.SourceChainName = NAME_None;
				break;
			}
		default:
			checkNoEntry();
		}

		// apply source if any decent matches were found
		if (SourceChainNames.IsValidIndex(SourceChainIndexToMapTo))
		{
			ChainPair.SourceChainName = SourceChainNames[SourceChainIndexToMapTo];
		}
	}

	// sort mapping again
	SortMapping();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void FRetargetChainMapping::LoadFromDeprecatedChainSettings(const TArray<URetargetChainSettings*>& InChainSettings)
{
	ChainMap.Empty();
	
	for (const URetargetChainSettings* ChainSetting : InChainSettings)
	{
		ChainMap.Add(FRetargetChainPair(ChainSetting->TargetChain, ChainSetting->SourceChain));
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
