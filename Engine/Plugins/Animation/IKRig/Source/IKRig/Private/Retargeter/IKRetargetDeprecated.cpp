// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargetDeprecated.h"

#include "IKRigObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetDeprecated)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

void URetargetChainSettings::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		// load the old chain settings into the new struct format
		if (Ar.CustomVer(FIKRigObjectVersion::GUID) < FIKRigObjectVersion::ChainSettingsConvertedToStruct)
		{
			Settings.FK.EnableFK =  CopyPoseUsingFK_DEPRECATED;
			Settings.FK.RotationMode =  RotationMode_DEPRECATED;
			Settings.FK.RotationAlpha =  RotationAlpha_DEPRECATED;
			Settings.FK.TranslationMode =  TranslationMode_DEPRECATED;
			Settings.FK.TranslationAlpha =  TranslationAlpha_DEPRECATED;
			Settings.IK.EnableIK =  DriveIKGoal_DEPRECATED;
			Settings.IK.BlendToSource =  BlendToSource_DEPRECATED;
			Settings.IK.BlendToSourceWeights =  BlendToSourceWeights_DEPRECATED;
			Settings.IK.StaticOffset =  StaticOffset_DEPRECATED;
			Settings.IK.StaticLocalOffset =  StaticLocalOffset_DEPRECATED;
			Settings.IK.StaticRotationOffset =  StaticRotationOffset_DEPRECATED;
			Settings.IK.Extension =  Extension_DEPRECATED;
			Settings.SpeedPlanting.EnableSpeedPlanting =  UseSpeedCurveToPlantIK_DEPRECATED;
			Settings.SpeedPlanting.SpeedCurveName =  SpeedCurveName_DEPRECATED;
			Settings.SpeedPlanting.SpeedThreshold =  VelocityThreshold_DEPRECATED;
			Settings.SpeedPlanting.UnplantStiffness =  UnplantStiffness_DEPRECATED;
			Settings.SpeedPlanting.UnplantCriticalDamping =  UnplantCriticalDamping_DEPRECATED;
		}
	}
#endif
}

void URetargetRootSettings::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		// load the old root settings into the new struct format
		if (Ar.CustomVer(FIKRigObjectVersion::GUID) < FIKRigObjectVersion::ChainSettingsConvertedToStruct)
		{
			Settings.ScaleHorizontal = GlobalScaleHorizontal_DEPRECATED;
			Settings.ScaleVertical = GlobalScaleVertical_DEPRECATED;
			Settings.BlendToSource = static_cast<float>(BlendToSource_DEPRECATED.Size());
			Settings.TranslationOffset = StaticOffset_DEPRECATED;
			Settings.RotationOffset = StaticRotationOffset_DEPRECATED;
		}
	}
#endif
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
