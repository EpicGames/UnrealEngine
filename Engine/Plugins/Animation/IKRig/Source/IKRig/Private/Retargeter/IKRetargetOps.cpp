// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargetOps.h"

#include "IKRig.h"
#include "Retargeter/IKRetargetOverrides.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetOps)

#define LOCTEXT_NAMESPACE "RetargetOpBase"

const UClass* FIKRetargetOpSettingsBase::GetControllerType() const
{
	return UIKRetargetOpControllerBase::StaticClass();
}

UIKRetargetOpControllerBase* FIKRetargetOpSettingsBase::GetController(UObject* Outer)
{
	return CreateControllerIfNeeded(Outer);
}

UIKRetargetOpControllerBase* FIKRetargetOpSettingsBase::CreateControllerIfNeeded(UObject* Outer)
{
	if (!Controller.IsValid())
	{
		const UClass* ClassType = GetControllerType();
		if (ensure(ClassType && ClassType->IsChildOf(UIKRetargetOpControllerBase::StaticClass())))
		{
			Controller = TStrongObjectPtr(NewObject<UIKRetargetOpControllerBase>(Outer, ClassType));
			Controller->OpSettingsToControl = this;
		}
	}
	return Controller.Get();
}

void FIKRetargetOpBase::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	GetSettings()->PostLoad(InVersion);
}

void FIKRetargetOpBase::SetSettings(const FIKRetargetOpSettingsBase* InSettings)
{
	GetSettings()->CopySettingsAtRuntime(InSettings);
}

#if WITH_EDITOR

void FIKRetargetOpSettingsBase::AssignSkeletonAssets(const USkeleton* InSourceSkeleton, const USkeleton* InTargetSkeleton)
{
	SourceSkeletonAsset = InSourceSkeleton;
	TargetSkeletonAsset = InTargetSkeleton;
}

FName FIKRetargetOpBase::GetDefaultName() const
{
	// calls virtual GetType() to get the derived type
	const UScriptStruct* ScriptStruct = GetType();

	// try to get the "nice name" from metadata
	FString TypeName;
    if (ScriptStruct->HasMetaData(TEXT("DisplayName")))
    {
    	TypeName = ScriptStruct->GetMetaData(TEXT("DisplayName"));
    }

	// if no "DisplayName" metadata is found, fall back to the struct name
	if (TypeName.IsEmpty())
	{
		TypeName = ScriptStruct->GetName();
	}
	
	return FName(TypeName);
}

FText FIKRetargetOpBase::GetWarningMessage() const
{
	if (bIsInitialized)
	{
		return IsEnabled() ? LOCTEXT("OpReadyAndOn", "Running.") : LOCTEXT("OpReadyAndOff", "Ready, but disabled.");
	}
	
	return LOCTEXT("OpNotReady", "Not initialized. See output log.");
}

void FOpExecutionTimer::Reset()
{
	TimeThisFrame = 0.0f;
	RunningAverageTime = 0.0f;
}

void FOpExecutionTimer::AddTimeThisFrame(const double InSeconds)
{
	TimeThisFrame += InSeconds;
}

void FOpExecutionTimer::StartNewFrame()
{
	constexpr double FrameWindow = 30.0;
	constexpr double Alpha = 2.0 / (FrameWindow + 1.0);
	RunningAverageTime += Alpha * (TimeThisFrame - RunningAverageTime);

	TimeThisFrame = 0.0f;
}

double FOpExecutionTimer::GetRunningAverage() const
{
	return RunningAverageTime;
}

void FIKRetargetOpBase::GetOverrideableProperties(TSharedPtr<FRetargetPropertyOverrideNode>& OutRootNode) const
{
	const UScriptStruct* SettingsStruct = GetSettingsType();
	const uint8* SettingsData = reinterpret_cast<const uint8*>(GetSettingsConst());
	if (!ensure(SettingsStruct && SettingsData))
	{
		return;
	}

	OutRootNode = MakeShared<FRetargetPropertyOverrideNode>();
	OutRootNode->NodeName = TEXT("Root");
	OutRootNode->AddPropertiesFromStruct(SettingsStruct, SettingsData, TEXT(""));
}

#endif

FName FIKRetargetOpBase::GetName() const
{
	return Name;
}

void FIKRetargetOpBase::SetName(const FName InName)
{
	ensure(InName != NAME_None);
	Name = InName;
#if WITH_EDITOR
	GetSettings()->OwningOpName = GetName();
#endif
}

void FIKRetargetOpBase::SetParentOpName(const FName InName)
{
	ParentOpName = InName;
}

FName FIKRetargetOpBase::GetParentOpName() const
{
	return ParentOpName;
}

void FIKRetargetOpBase::CopySettingsRaw(const FIKRetargetOpSettingsBase* InSettings, const TArray<FName>& InPropertiesToIgnore)
{
	// copy everything except any property that is filtered
	FIKRetargetOpBase::CopyStructProperties(
		GetSettingsType(),
		InSettings,
		GetSettings(),
		InPropertiesToIgnore);
}

void FIKRetargetOpBase::CopyStructProperties(
	const UStruct* InStructType,
	const void* InSrcStruct,
	void* InOutDestStruct,
	const TArray<FName>& InPropertiesToIgnore)
{
	if (!InStructType || !InSrcStruct || !InOutDestStruct)
	{
		return;
	}

	for (TFieldIterator<FProperty> PropIt(InStructType); PropIt; ++PropIt)
	{
		FProperty* Property = *PropIt;
		if (!Property)
		{
			continue;
		}

		// check if this property should be ignored
		if (InPropertiesToIgnore.Contains(Property->GetFName()))
		{
			continue;
		}

		// copy the property value from Src to Dest
		Property->CopyCompleteValue(
			Property->ContainerPtrToValuePtr<void>(InOutDestStruct),
			Property->ContainerPtrToValuePtr<void>(InSrcStruct));
	}
}

#undef LOCTEXT_NAMESPACE
