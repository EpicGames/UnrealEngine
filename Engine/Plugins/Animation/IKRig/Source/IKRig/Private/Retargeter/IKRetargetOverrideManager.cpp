// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargetOverrideManager.h"

#include "StructUtils/PropertyBag.h"
#include "Retargeter/IKRetargetProcessor.h"

void FIKRetargetRuntimeOverrideManager::ApplyPropertyOverridesToOps(
	FIKRetargetProcessor& InProcessor,
	const TArrayView<const FName> InOverrideSetsToApply,
	FInstancedPropertyBag* InVariables,
	TMap<FName, float>* InBoundCurveValues)
{
	const UIKRetargeter* Asset = InProcessor.GetRetargetAsset();
	if (!ensure(Asset))
	{
		return;
	}

	UpdateFullListOfOverrideSets(InProcessor, InOverrideSetsToApply);
	
	const int32 NewOverrideSetHash = GetOverrideSetArrayHash(OverrideSetsToApply); 
	if (bIsCacheDirty || (NewOverrideSetHash != OverrideSetHash) || (CachedAssetVersion != Asset->GetOverrideVersion()))
	{
		RebuildCache(InProcessor, InVariables);
		OverrideSetHash = NewOverrideSetHash;
		CachedAssetVersion = Asset->GetOverrideVersion();
		bIsCacheDirty = false;
	}

#if WITH_EDITOR
	UpdateCachedValues(InProcessor);
#endif

	const uint8* BagMemory = InVariables ? InVariables->GetValue().GetMemory() : nullptr;

	// the "Hot Path", a linear iteration and direct memory write of all individual property overrides
	for (const FCompiledPropertyOverride& Entry : CompiledOverrides)
	{
		// 1. Dynamic Bound Curve
		if (Entry.BoundCurveName != NAME_None && InBoundCurveValues)
		{
			if (const float* CurveValuePtr = InBoundCurveValues->Find(Entry.BoundCurveName))
			{
				const float CurveVal = *CurveValuePtr;
				switch (Entry.CurveCoercionType)
				{
					case ECurveCoercionType::Float:
						*(float*)Entry.DestPtr = CurveVal;
						break;
					case ECurveCoercionType::Double:
						*(double*)Entry.DestPtr = (double)CurveVal;
						break;
					case ECurveCoercionType::Int32:
						*(int32*)Entry.DestPtr = FMath::RoundToInt(CurveVal);
						break;
					case ECurveCoercionType::Bool:
						// NOTE: Unreal can compact multiple UProperty bools into a single byte!
						// So bools must use the property setter to safely respect bitmask boundaries in structs
						((FBoolProperty*)Entry.Property)->SetPropertyValue(Entry.DestPtr, CurveVal > 0.5f);
						break;
					case ECurveCoercionType::Byte:
						*(uint8*)Entry.DestPtr = (uint8)FMath::RoundToInt(FMath::Clamp(CurveVal, 0, 255));
						break;
					case ECurveCoercionType::Int64:
						*(int64*)Entry.DestPtr = (int64)FMath::RoundToInt(CurveVal);
						break;
					case ECurveCoercionType::None:
					default:
						break;
				}
			}
		}
		// 2. Dynamic Bound Variable
		else if (Entry.BagMemoryOffset != INDEX_NONE && BagMemory)
		{
			switch (Entry.VarCoercionType)
			{
			case EVariableCoercionType::Direct:
				FMemory::Memcpy(Entry.DestPtr, BagMemory + Entry.BagMemoryOffset, Entry.Size);
				break;
			case EVariableCoercionType::FloatToDouble:
				*reinterpret_cast<double*>(Entry.DestPtr) = *reinterpret_cast<const float*>(BagMemory + Entry.BagMemoryOffset);
				break;
			case EVariableCoercionType::DoubleToFloat:
				*reinterpret_cast<float*>(Entry.DestPtr) = static_cast<float>(*reinterpret_cast<const double*>(BagMemory + Entry.BagMemoryOffset));
				break;
			default:
				break;
			}
		}
		// 3. Static Hardcoded String
		else if (Entry.SourcePoolOffset != INDEX_NONE)
		{
			FMemory::Memcpy(Entry.DestPtr, CachedValuePool.GetData() + Entry.SourcePoolOffset, Entry.Size);
		}
	}
}

void FIKRetargetRuntimeOverrideManager::UpdateFullListOfOverrideSets(FIKRetargetProcessor& Processor, const TArrayView<const FName> InOverrideSetsToApply)
{
	OverrideSetsToApply.Reset();
	
	const UIKRetargeter* Asset = Processor.GetRetargetAsset();
	if (!ensure(Asset))
	{
		return;
	}
	
	for (const FName& OverrideSetName : InOverrideSetsToApply)
	{
		TArray<FName> CurrentChain;
		FName CurrentName = OverrideSetName;
		while (CurrentName != NAME_None)
		{
			const FRetargetOverrideSet* OverrideSet = Asset->GetOverrideSetByName(CurrentName);
			if (!OverrideSet) break;

			CurrentChain.Add(CurrentName);
			CurrentName = OverrideSet->ParentName;
		}

		for (int32 i = CurrentChain.Num() - 1; i >= 0; --i)
		{
			OverrideSetsToApply.AddUnique(CurrentChain[i]);
		}
	}
}

uint32 FIKRetargetRuntimeOverrideManager::GetOverrideSetArrayHash(const TArray<FName>& InProfileNames)
{
	uint32 CombinedHash = 0;
	for (const FName& ProfileName : InProfileNames)
	{
		CombinedHash = HashCombine(CombinedHash, GetTypeHash(ProfileName));
	}
	return CombinedHash;
}

void FIKRetargetRuntimeOverrideManager::RebuildCache(FIKRetargetProcessor& Processor, FInstancedPropertyBag* InVariables)
{
	CompiledOverrides.Reset();
	CachedValuePool.Reset();
	CachedVersionPerOp.Reset();

	const UIKRetargeter* Asset = Processor.GetRetargetAsset();
	if (!ensure(Asset))
	{
		return;
	}
	
	const UPropertyBag* BagStruct = InVariables ? InVariables->GetPropertyBagStruct() : nullptr;

	// first we calculate total memory needed for the pool
	int32 TotalPoolSize = 0;
	for (const FName& ProfileName : OverrideSetsToApply)
	{
		const FRetargetOverrideSet* OverrideSet = Asset->GetOverrideSetByName(ProfileName);
		if (!OverrideSet)
		{
			continue;
		}
		
		for (const FRetargetOpOverrides& OpOverrides : OverrideSet->OpOverrides)
		{
			const UScriptStruct* OpScriptStruct = OpOverrides.ScriptStruct;
			if (!ensure(OpScriptStruct))
			{
				continue;
			}

			for (const FRetargetOpPropertyOverride& PropOverride : OpOverrides.PropertyOverrides)
			{
				TArray<FRetargetOpPropertyOverride::FPropertySegment> Segments;
				if (FRetargetOpPropertyOverride::GetSegmentsFromProperyPath(PropOverride.GetPropertyPath(), OpScriptStruct, Segments))
				{
					if (FProperty* LeafProp = FRetargetOpPropertyOverride::GetLeafProperty(Segments))
					{
						TotalPoolSize += LeafProp->GetSize();
					}
				}
			}
		}
	}
	CachedValuePool.Reserve(TotalPoolSize);

	// now bake values into the pool and cache the pointers
	for (const FName& ProfileName : OverrideSetsToApply)
	{
		const FRetargetOverrideSet* OverrideSet = Asset->GetOverrideSetByName(ProfileName);
		if (!OverrideSet)
		{
			continue;
		}

		for (const FRetargetOpOverrides& OpOverrides : OverrideSet->OpOverrides)
		{
			FIKRetargetOpBase* OpInProcessor = Processor.GetRetargetOpByName(OpOverrides.OpName);
			if (!OpInProcessor)
			{
				continue; // can happen on first frame after op deleted
			}
			
			FIKRetargetOpSettingsBase* OpSettings = OpInProcessor->GetSettings();
			if (!ensure(OpSettings))
			{
				continue;
			}
			
			uint8* OpSettingsData = reinterpret_cast<uint8*>(OpSettings);
			const UScriptStruct* OpScriptStruct = OpOverrides.ScriptStruct;

			if (!ensure(OpScriptStruct))
			{
				continue;
			}

			for (const FRetargetOpPropertyOverride& PropOverride : OpOverrides.PropertyOverrides)
			{
				TArray<FRetargetOpPropertyOverride::FPropertySegment> Segments;
				if (!ensure(FRetargetOpPropertyOverride::GetSegmentsFromProperyPath(PropOverride.GetPropertyPath(), OpScriptStruct, Segments))) continue;
				
				FProperty* LeafProp = FRetargetOpPropertyOverride::GetLeafProperty(Segments);
				uint8* DestAddress = FRetargetOpPropertyOverride::GetDataPointerFromPathSegments(OpSettingsData, Segments);

				if (!ensure(LeafProp && DestAddress))
				{
					continue;
				}
				
				const int32 ValueSize = LeafProp->GetSize();

				// record the move parameters
				FCompiledPropertyOverride& NewMove = CompiledOverrides.AddDefaulted_GetRef();
				NewMove.DestPtr = DestAddress;
				NewMove.Size = ValueSize;
				FOpOverridesKey OpOverrideKey(ProfileName, OpOverrides.OpName);
				NewMove.OpOverrideKey = OpOverrideKey;
				NewMove.PropertyPathHash = PropOverride.GetPropertyPathHash();
				NewMove.Property = LeafProp;

				FName BoundCurveName = PropOverride.GetBoundCurveName();
				FName BoundVarName = PropOverride.GetBoundVariableName();
				FProperty* BagProp = (BagStruct && BoundVarName != NAME_None) ? BagStruct->FindPropertyByName(BoundVarName) : nullptr;

				// 1. bound to an anim curve
				if (BoundCurveName != NAME_None)
				{
					NewMove.BoundCurveName = BoundCurveName;
					
					// cache the target data type to avoid reflection costs in the hot path
					if (LeafProp->IsA<FFloatProperty>()) NewMove.CurveCoercionType = ECurveCoercionType::Float;
					else if (LeafProp->IsA<FDoubleProperty>()) NewMove.CurveCoercionType = ECurveCoercionType::Double;
					else if (LeafProp->IsA<FIntProperty>()) NewMove.CurveCoercionType = ECurveCoercionType::Int32;
					else if (LeafProp->IsA<FBoolProperty>()) NewMove.CurveCoercionType = ECurveCoercionType::Bool;
					else if (LeafProp->IsA<FByteProperty>()) NewMove.CurveCoercionType = ECurveCoercionType::Byte;
					else if (LeafProp->IsA<FInt64Property>()) NewMove.CurveCoercionType = ECurveCoercionType::Int64;
				}
				// 2. bound to a variable
				else if (BagProp)
				{
					NewMove.BagMemoryOffset = BagProp->GetOffset_ForInternal();

					// determine coercion type for numeric mismatches (e.g. PropertyBag float vs op double)
					if (BagProp->GetSize() == ValueSize)
					{
						NewMove.VarCoercionType = EVariableCoercionType::Direct;
					}
					else if (BagProp->IsA<FFloatProperty>() && LeafProp->IsA<FDoubleProperty>())
					{
						NewMove.VarCoercionType = EVariableCoercionType::FloatToDouble;
					}
					else if (BagProp->IsA<FDoubleProperty>() && LeafProp->IsA<FFloatProperty>())
					{
						NewMove.VarCoercionType = EVariableCoercionType::DoubleToFloat;
					}
					else
					{
						// incompatible types, fall back to hardcoded
						NewMove.BagMemoryOffset = INDEX_NONE;
					}
				}
				// 3. unbound / hardcoded
				else
				{
					const int32 CurrentOffset = CachedValuePool.Num();
					CachedValuePool.AddUninitialized(ValueSize);
							
					uint8* PoolEntryAddress = CachedValuePool.GetData() + CurrentOffset;
							
					const TCHAR* ImportResult = LeafProp->ImportText_Direct(*PropOverride.GetValueString(), PoolEntryAddress, nullptr, PPF_None);
					if (!ensure(ImportResult))
					{
						CompiledOverrides.Pop();
						continue; 
					}

					NewMove.SourcePoolOffset = CurrentOffset;
				}

				CachedVersionPerOp.FindOrAdd(OpOverrideKey, OpOverrides.GetValueVersion());
			}
		}
	}
}

void FIKRetargetRuntimeOverrideManager::DebugPrintVariables(FInstancedPropertyBag* InVariables)
{
	if (!InVariables || !InVariables->GetPropertyBagStruct())
	{
		return;
	}
	
	const UPropertyBag* BagStruct = InVariables->GetPropertyBagStruct();
	const uint8* ContainerMemory = InVariables->GetValue().GetMemory();

	if (!BagStruct || !ContainerMemory)
	{
		return;
	}

	UE_LOGF(LogTemp, Warning, "=== Dumping FInstancedPropertyBag ===");
	for (const FPropertyBagPropertyDesc& Desc : BagStruct->GetPropertyDescs())
	{
		if (FProperty* Prop = BagStruct->FindPropertyByName(Desc.Name))
		{
			const void* ValuePtr = Prop->ContainerPtrToValuePtr<void>(ContainerMemory);
			FString ExportedValueStr;
			Prop->ExportTextItem_Direct(ExportedValueStr, ValuePtr, nullptr, nullptr, PPF_None);
			UE_LOGF(LogTemp, Warning, "  [%ls] %ls : %ls", *Prop->GetCPPType(), *Desc.Name.ToString(), *ExportedValueStr);
		}
	}
	UE_LOGF(LogTemp, Warning, "=====================================");
}

void FIKRetargetRuntimeOverrideManager::UpdateCachedValues(FIKRetargetProcessor& Processor)
{
	const UIKRetargeter* Asset = Processor.GetRetargetAsset();
	if (!ensure(Asset) || CachedVersionPerOp.IsEmpty())
	{
		return;
	}

	uint8* PoolBase = CachedValuePool.GetData();
	
	for (const FName& OverrideSetName : OverrideSetsToApply)
	{
		const FRetargetOverrideSet* OverrideSet = Asset->GetOverrideSetByName(OverrideSetName);
		if (!ensure(OverrideSet))
		{
			continue;
		}

		for (const FRetargetOpOverrides& OpOverrides : OverrideSet->OpOverrides)
		{
			FOpOverridesKey OpOverrideKey(OverrideSetName, OpOverrides.OpName);
			const int32* CachedVersion = CachedVersionPerOp.Find(OpOverrideKey);
			if (!CachedVersion || *CachedVersion == OpOverrides.GetValueVersion())
			{
				continue;
			}

			for (FCompiledPropertyOverride& Entry : CompiledOverrides)
			{
				if (Entry.OpOverrideKey != OpOverrideKey)
				{
					continue;
				}
				
				// ignore bound variables and bound curves, they update automatically every frame
				if (Entry.SourcePoolOffset == INDEX_NONE)
				{
					continue;
				}

				for (const FRetargetOpPropertyOverride& PropertyOverride : OpOverrides.PropertyOverrides)
				{
					if (Entry.PropertyPathHash != PropertyOverride.GetPropertyPathHash())
					{
						continue;
					}
					
					uint8* PoolEntryAddress = PoolBase + Entry.SourcePoolOffset;
					Entry.Property->ImportText_Direct(*PropertyOverride.GetValueString(), PoolEntryAddress, nullptr, PPF_None);
				}
			}
		}
	}
}