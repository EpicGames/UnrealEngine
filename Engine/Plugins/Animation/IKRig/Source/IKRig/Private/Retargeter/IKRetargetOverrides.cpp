// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargetOverrides.h"

#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "IKRigLogger.h"

#include "IKRig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetOverrides)

#define LOCTEXT_NAMESPACE "IKRetargetOverrides"

TArray<FPropertyPathParser::FPropertyPathStep> FPropertyPathParser::ParsePath(const FString& InPath)
{
	TArray<FPropertyPathStep> OutSteps;
	if (InPath.IsEmpty())
	{
		return OutSteps;
	}

	TArray<FString> RawParts;
	InPath.ParseIntoArray(RawParts, TEXT("->"), true);

	for (const FString& Part : RawParts)
	{
		FPropertyPathStep Step;

		// check for array syntax: "MemberName[0]"
		if (Part.EndsWith(TEXT("]")))
		{
			FString IndexStr;
			if (Part.Split(TEXT("["), &Step.PropertyName, &IndexStr))
			{
				IndexStr.RemoveFromEnd(TEXT("]"));
				if (IndexStr.IsNumeric())
				{
					Step.ArrayIndex = FCString::Atoi(*IndexStr);
				}
			}
		}
		else
		{
			Step.PropertyName = Part;
		}

		OutSteps.Add(Step);
	}

	return OutSteps;
}

#if WITH_EDITOR
void FRetargetPropertyOverrideNode::AddPropertiesFromStruct(
	const UScriptStruct* InStruct, 
	const uint8* InStructData, 
	FString PathPrefix)
{
	if (!ensure(InStruct && InStructData))
	{
		return;
	}
	
	static const TSet<UScriptStruct*> AtomicStructTypes = {
		TBaseStructure<FVector>::Get(), TBaseStructure<FVector2D>::Get(), TBaseStructure<FVector4>::Get(),
		TBaseStructure<FLinearColor>::Get(), TBaseStructure<FColor>::Get(), 
		TBaseStructure<FRotator>::Get(), TBaseStructure<FQuat>::Get(), TBaseStructure<FTransform>::Get(),
	};

	struct FPendingStruct
	{
		const UScriptStruct* Struct;
		const uint8* Data;
		FString Path;
		TSharedPtr<FRetargetPropertyOverrideNode> ParentNode;
		// True when this frame (or any ancestor) is an element of a titled array
		// (e.g. a chain). Suppresses Category sub-nodes so element properties show
		// flat under the element, matching the "Chains → LeftArm → leaves" shape.
		bool bInsideTitledArrayElement = false;
	};

	// recurse through the property tree adding all leaf nodes
	TArray<FPendingStruct> StructStack;
	StructStack.Push({ InStruct, InStructData, PathPrefix, SharedThis(this), false });
	while (StructStack.Num() > 0)
	{
		FPendingStruct Current = StructStack.Pop();

		for (TFieldIterator<FProperty> PropIt(Current.Struct); PropIt; ++PropIt)
		{
			FProperty* Prop = *PropIt;
			
			if (!Prop->HasAnyPropertyFlags(CPF_BlueprintVisible) || Prop->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
			{
				continue;
			}
			
			if (Prop->HasMetaData(IKRigNonOverrideableMetaLabel))
			{
				continue;
			}
			
			// determine the correct parent node based on Category metadata
			TSharedPtr<FRetargetPropertyOverrideNode> TargetNode = Current.ParentNode;
			if (Prop->HasMetaData(TEXT("Category")))
			{
				FName CategoryName = FName(*Prop->GetMetaData(TEXT("Category")));
				if (CategoryName == FName(TEXT("Debug")))
				{
					continue;
				}
				// Inside a titled array element (e.g. a chain), skip the Category
				// sub-node so leaf properties attach directly to the element node.
				if (!Current.bInsideTitledArrayElement)
				{
					TargetNode = TargetNode->SubNodes.FindOrAdd(CategoryName, MakeShared<FRetargetPropertyOverrideNode>());
					TargetNode->NodeName = CategoryName;
				}
			}

			FString FullPath = Current.Path.IsEmpty() ? Prop->GetName() : Current.Path + TEXT("->") + Prop->GetName();

			// CASE 1: ARRAYS
			if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(Prop))
			{
				FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(Current.Data));
				if (ArrayHelper.Num() > 0)
				{
					FProperty* InnerProp = ArrayProp->Inner;
					FStructProperty* InnerStructProp = CastField<FStructProperty>(InnerProp);

					// Resolve meta=(TitleProperty="Name") on the array UPROPERTY so each element
					// is labeled with its identifier (e.g. TargetChainName) instead of "[i]".
					// Falls back to "[i]" on any lookup failure, empty value, "None", or duplicate.
					const FString TitleMeta = ArrayProp->GetMetaData(TEXT("TitleProperty"));

					// When TitleProperty is set, the elements are title-bearing and the array's
					// own name is redundant structural plumbing — attach element nodes directly
					// to the parent (category) node. Otherwise insert an intermediate node named
					// by the array for tree clarity.
					TSharedPtr<FRetargetPropertyOverrideNode> ArraySubMenu;
					if (TitleMeta.IsEmpty())
					{
						FName ArrayName = FName(*Prop->GetName());
						ArraySubMenu = TargetNode->SubNodes.FindOrAdd(ArrayName, MakeShared<FRetargetPropertyOverrideNode>());
						ArraySubMenu->NodeName = ArrayName;
					}
					else
					{
						ArraySubMenu = TargetNode;
					}

					for (int32 i = 0; i < ArrayHelper.Num(); ++i)
					{
						FString IndexPath = FString::Printf(TEXT("%s[%d]"), *FullPath, i);
						FString IndexName;

						if (!TitleMeta.IsEmpty() && InnerStructProp)
						{
							if (FProperty* TitleProp = FindFProperty<FProperty>(InnerStructProp->Struct, FName(*TitleMeta)))
							{
								FString Exported;
								TitleProp->ExportTextItem_Direct(
									Exported,
									TitleProp->ContainerPtrToValuePtr<void>(ArrayHelper.GetRawPtr(i)),
									nullptr, nullptr, PPF_None);
								if (!Exported.IsEmpty() && Exported != TEXT("None"))
								{
									const FName Candidate(*Exported);
									if (!ArraySubMenu->SubNodes.Contains(Candidate))
									{
										IndexName = Exported;
									}
								}
							}
						}

						if (IndexName.IsEmpty())
						{
							IndexName = FString::Printf(TEXT("[%d]"), i);
						}

						TSharedPtr<FRetargetPropertyOverrideNode> IndexNode = ArraySubMenu->SubNodes.FindOrAdd(FName(*IndexName), MakeShared<FRetargetPropertyOverrideNode>());
						IndexNode->NodeName = FName(*IndexName);

						if (InnerStructProp && !AtomicStructTypes.Contains(InnerStructProp->Struct))
						{
							// Mark the element frame as being inside a titled array so that
							// Category sub-nodes are suppressed for the element's children.
							// When pushing from a non-titled array, propagate the parent frame's flag.
							const bool bElementInsideTitledArray = !TitleMeta.IsEmpty() || Current.bInsideTitledArrayElement;
							StructStack.Push({ InnerStructProp->Struct, ArrayHelper.GetRawPtr(i), IndexPath, IndexNode, bElementInsideTitledArray });
						}
						else
						{
							FRetargetPropertyOverrideEntry Entry;
							Entry.DisplayName = FText::Format(INVTEXT("{0} {1}"), Prop->GetDisplayNameText(), FText::FromString(IndexName));
							Entry.PropertyPath = IndexPath;
							IndexNode->LeafProperties.Add(Entry);
						}
					}
				}
				continue;
			}

			// CASE 2: STRUCTS
			FStructProperty* StructProp = CastField<FStructProperty>(Prop);
			bool bIsLeafStruct = StructProp && AtomicStructTypes.Contains(StructProp->Struct);
			if (StructProp && !bIsLeafStruct)
			{
				FName StructName = FName(*Prop->GetName());
				TSharedPtr<FRetargetPropertyOverrideNode> NewSubNode = TargetNode->SubNodes.FindOrAdd(StructName, MakeShared<FRetargetPropertyOverrideNode>());
				NewSubNode->NodeName = StructName;

				// Propagate the titled-array-element flag into nested struct walks.
				StructStack.Push({ StructProp->Struct, StructProp->ContainerPtrToValuePtr<uint8>(Current.Data), FullPath, NewSubNode, Current.bInsideTitledArrayElement });
			}
			// CASE 3: LEAF PROPERTIES
			else
			{
				FRetargetPropertyOverrideEntry Entry;
				Entry.DisplayName = Prop->GetDisplayNameText();
				Entry.PropertyPath = FullPath;
				TargetNode->LeafProperties.Add(Entry);
			}
		}
	}
}
#endif

bool FRetargetOpPropertyOverride::IsValid(const UScriptStruct* InSettingsStruct) const
{
	if (!InSettingsStruct || PropertyPath.IsEmpty())
	{
		return false;
	}

	// try to parse the path into segments
	TArray<FPropertySegment> Segments;
	if (!GetSegmentsFromProperyPath(PropertyPath, InSettingsStruct, Segments))
	{
		// the path is broken (ie, a property was renamed or deleted)
		return false;
	}

	// double check the leaf property exists
	FProperty* LeafProp = GetLeafProperty(Segments);
	if (!LeafProp)
	{
		return false;
	}

	// verify it has a serialized value
	if (ValueAsString.IsEmpty())
	{
		return false;
	}
	
	// verify the serialized value is compatible
	//
	// allocate a temporary buffer for the property type
	TArray<uint8> TempStorage;
	TempStorage.AddZeroed(LeafProp->GetSize());
	uint8* TempData = TempStorage.GetData();
	// initialize the property (important for structs/strings to avoid crashes)
	LeafProp->InitializeValue(TempData);
	// attempt to import. ImportText returns nullptr on failure.
	const TCHAR* ImportResult = LeafProp->ImportText_Direct(*ValueAsString, TempData, nullptr, PPF_None, nullptr);
	// clean up the temporary value before the buffer goes out of scope
	LeafProp->DestroyValue(TempData);
	if (ImportResult == nullptr)
	{
		return false; // string value is corrupted or incompatible with the current property type
	}

	return true;
}

bool FRetargetOpPropertyOverride::IsStructPropertyAtDefault(const UScriptStruct* InStruct, uint8* InStructData, const FString& InPropertyPath)
{
	if (!InStruct || !InStructData || InPropertyPath.IsEmpty())
	{
		return false;
	}
	
	TArray<FPropertySegment> Segments;
	if (!GetSegmentsFromProperyPath(InPropertyPath, InStruct, Segments))
	{
		return false;
	}
	
	FProperty* LeafProperty = GetLeafProperty(Segments);
	if (!LeafProperty)
	{
		return false;
	}
	
	uint8* InstanceValuePtr = GetDataPointerFromPathSegments(InStructData, Segments, false);
	if (!InstanceValuePtr)
	{
		return false;
	}

	// instantiate a default struct to compare against
	TArray<uint8> DefaultBuffer;
	DefaultBuffer.AddUninitialized(InStruct->GetStructureSize());
	InStruct->InitializeStruct(DefaultBuffer.GetData());
	
	bool bIsAtDefault = false;
	constexpr bool bResizeArrays = true;
	uint8* DefaultValuePtr = GetDataPointerFromPathSegments(DefaultBuffer.GetData(), Segments, bResizeArrays);
	if (DefaultValuePtr)
	{
		// use FProperty::Identical() to perform a binary comparison of the two memory locations
		bIsAtDefault = LeafProperty->Identical(InstanceValuePtr, DefaultValuePtr);
	}

	// clean up the temporary struct memory
	InStruct->DestroyStruct(DefaultBuffer.GetData());

	return bIsAtDefault;
}

bool FRetargetOpPropertyOverride::GetSegmentsFromProperyPath(
	const FString& InPath,
	const UScriptStruct* InOwnerStruct,
	TArray<FPropertySegment>& OutSegments)
{
	OutSegments.Reset();

	if (InPath.IsEmpty() || InOwnerStruct == nullptr)
	{
		return false;
	}

	const UStruct* CurrentOwnerStruct = InOwnerStruct;
	
	// parse path into steps
	TArray<FPropertyPathParser::FPropertyPathStep> Steps = FPropertyPathParser::ParsePath(InPath);
	for (const FPropertyPathParser::FPropertyPathStep& Step : Steps)
	{
		// 1. find the property by name in the current struct scope
		FProperty* Property = FindProperty(CurrentOwnerStruct, Step.PropertyName);
		if (!Property)
		{
			UE_LOGF(LogIKRig, Warning, "Failed to find property '%ls' in struct '%ls'",
				*Step.PropertyName, *CurrentOwnerStruct->GetName());
			OutSegments.Reset();
			return false;
		}

		// 2. add the segment
		FPropertySegment& Segment = OutSegments.AddDefaulted_GetRef();
		Segment.Property = Property;
		Segment.ArrayIndex = Step.ArrayIndex;

		// 3. Update the OwnerStruct for the NEXT step in the path
		// (if this is a struct, we need its type)
		if (const FStructProperty* StructProp = CastField<FStructProperty>(Property))
		{
			CurrentOwnerStruct = StructProp->Struct;
		}
		
		// if this is an array, the next property will be inside the Inner struct
		else if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property))
		{
			if (const FStructProperty* InnerStructProp = CastField<FStructProperty>(ArrayProp->Inner))
			{
				CurrentOwnerStruct = InnerStructProp->Struct;
			}
		}
	}

	// validate we actually parsed something
	if (OutSegments.IsEmpty())
	{
		UE_LOGF(LogIKRig, Warning, "Failed to parse property path '%ls'", *InPath);
		return false;
	}

	return true;
}

uint8* FRetargetOpPropertyOverride::GetDataPointerFromPathSegments(
    uint8* InStructPtr,
    const TArray<FPropertySegment>& InSegments,
    bool bResizeArrays)
	{
	if (!InStructPtr || InSegments.IsEmpty())
	{
		return nullptr;
	}

	uint8* CurrentPtr = InStructPtr;

	for (int32 SegmeshIndex = 0; SegmeshIndex < InSegments.Num(); ++SegmeshIndex)
	{
		const FPropertySegment& Segment = InSegments[SegmeshIndex];
		FProperty* Property = Segment.Property.Get();
		if (!Property)
		{
			return nullptr;
		}

		// move to the property offset within the current container
		CurrentPtr += Property->GetOffset_ForInternal();

		// handle array indexing
		if (Segment.ArrayIndex != INDEX_NONE)
		{
			FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property);
			if (!ensure(ArrayProp))
			{
				return nullptr;
			}

			FScriptArrayHelper ArrayHelper(ArrayProp, CurrentPtr);

			if (!ArrayHelper.IsValidIndex(Segment.ArrayIndex))
			{
				if (bResizeArrays)
				{
					ArrayHelper.ExpandForIndex(Segment.ArrayIndex);
				}
				else
				{
					return nullptr;
				}
			}

			// move pointer to the specific element in the array
			CurrentPtr = ArrayHelper.GetRawPtr(Segment.ArrayIndex);

			// NOTE: if we have more segments to go, the property we are 
			// interested in for the next iteration's container is the array's inner property
			Property = ArrayProp->Inner; 
		}

		// check if we can continue diving
		if (SegmeshIndex < InSegments.Num() - 1)
		{
			// we check 'Property' which might now be the array's inner property
			FStructProperty* StructProp = CastField<FStructProperty>(Property);
			if (!StructProp)
			{
				return nullptr; // not a struct, can't dive further
			}
			// currentPtr is now the base address of this struct for the next loop iteration
		}
	}

	return CurrentPtr;
}

FProperty* FRetargetOpPropertyOverride::GetLeafProperty(const TArray<FPropertySegment>& InSegments)
{
	if (InSegments.Num() == 0)
	{
		return nullptr;
	}

	// the leaf is always the property at the end of the segment chain
	return InSegments.Last().Property.Get();
}

FProperty* FRetargetOpPropertyOverride::FindProperty(const UStruct* InStruct, const FString& InNameOrDisplayName)
{
	if (!InStruct)
	{
		return nullptr;
	}

	// first try to find by exact internal C++ name first
	if (FProperty* Prop = InStruct->FindPropertyByName(*InNameOrDisplayName))
	{
		return Prop;
	}

	return nullptr;
}

bool FRetargetOpOverrides::AddPropertyOverride(
	const FString& InPropertyPath,
	const uint8* InSettingsPtr,
	const UScriptStruct* InSettingsStruct,
	bool bRequiresReinit)
{
	if (!ensure(InSettingsPtr))
	{
		return false;
	}

	FRetargetOpPropertyOverride* PropertyOverride = FindPropertyOverride(InPropertyPath);
	if (!PropertyOverride)
	{
		// add a new one
		const int32 NewIndex = PropertyOverrides.Emplace(InPropertyPath);
		PropertyOverride = &PropertyOverrides[NewIndex];
	}

	UpdateOverrideValue(*PropertyOverride, InSettingsPtr, InSettingsStruct);
	
	return true;
}

bool FRetargetOpOverrides::RemovePropertyOverride(const FString& InPropertyPath)
{
	const int32 Index = PropertyOverrides.IndexOfByPredicate([InPropertyPath](const FRetargetOpPropertyOverride& InPropertyOverride)
		{
			return InPropertyOverride.GetPropertyPath() == InPropertyPath;
		});

	if (Index == INDEX_NONE)
	{
		return false;
	}
	
	PropertyOverrides.RemoveAt(Index);
	
	return true;
}

bool FRetargetOpOverrides::UpdateOverrideValue(
	FRetargetOpPropertyOverride& InPropertyOverride,
	const uint8* InStructData,
	const UScriptStruct* InScriptStruct)
{
	if (!ensure(InStructData && InScriptStruct))
	{
		return false;
	}
	
	// parse the string path into property segments
	TArray<FRetargetOpPropertyOverride::FPropertySegment> Segments;
	if (!InPropertyOverride.GetSegmentsFromProperyPath(InPropertyOverride.GetPropertyPath(), InScriptStruct, Segments))
	{
		return false;
	}

	// navigate to the specific memory address for this property instance
	const uint8* PropertyDataPtr = InPropertyOverride.GetDataPointerFromPathSegments(const_cast<uint8*>(InStructData), Segments);
	if (!PropertyDataPtr)
	{
		return false;
	}

	// get the leaf FProperty metadata
	FProperty* LeafProp = InPropertyOverride.GetLeafProperty(Segments);
	if (!LeafProp)
	{
		return false;
	}

	// serialize the actual value at that memory address into our string
	FString& ValueString = InPropertyOverride.GetValueStringEditable();
	ValueString.Reset();
	LeafProp->ExportTextItem_Direct(ValueString, PropertyDataPtr, nullptr, nullptr, PPF_None);

	// trigger runtime to refresh this override value in the memory pool
	Version++;

	return true;
}

bool FRetargetOpOverrides::HasPropertyOverride(const FString& InPropertyPath) const
{
	for (const FRetargetOpPropertyOverride& PropertyOverride : PropertyOverrides)
	{
		if (PropertyOverride.GetPropertyPath() == InPropertyPath)
		{
			return true;
		}
	}
	
	return false;
}

FRetargetOpPropertyOverride* FRetargetOpOverrides::FindPropertyOverride(const FString& InPropertyPath)
{
	for (FRetargetOpPropertyOverride& PropertyOverride : PropertyOverrides)
	{
		if (PropertyOverride.GetPropertyPath() == InPropertyPath)
		{
			return &PropertyOverride;
		}
	}
	
	return nullptr;
}

int32 FRetargetOpOverrides::GetNumPropertyOverrides() const
{
	return PropertyOverrides.Num();
}

FRetargetOpOverrides* FRetargetOverrideSet::FindOpOverrideByOpName(const FName InOpName)
{
	for (FRetargetOpOverrides& OpOverride : OpOverrides)
	{
		if (OpOverride.OpName == InOpName)
		{
			return &OpOverride;
		}
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE