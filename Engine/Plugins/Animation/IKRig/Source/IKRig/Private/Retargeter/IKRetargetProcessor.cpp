// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargetProcessor.h"

#include "IKRigLogger.h"
#include "Rig/IKRigDefinition.h"

#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetProfile.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"

#include "Engine/SkeletalMesh.h"
#include "Retargeter/RetargetOps/CopyBasePoseOp.h"
#include "Retargeter/RetargetOps/ScaleSourceOp.h"
#include "Retargeter/RetargetOps/RetargetCurvesOp.h"
#include "Retargeter/RetargetOps/RetargetPoseOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetProcessor)

DEFINE_STAT(STAT_Anim_RetargetProcessorRun);

#define LOCTEXT_NAMESPACE "IKRetargetProcessor"

// This is the default end of branch index value, meaning we haven't cached it yet
#define RETARGETSKELETON_INVALID_BRANCH_INDEX -2

FResolvedRetargetPose& FResolvedRetargetPoseSet::AddOrUpdateRetargetPose(
	const FRetargetSkeleton& InSkeleton,
	const FName InRetargetPoseName,
	const FIKRetargetPose* InRetargetPose,
	const FName PelvisBoneName,
	const FRetargetPoseScaleWithPivot& InSourceScale)
{
	// add retarget pose if it doesn't already exist
	FResolvedRetargetPose& RetargetPose = FindOrAddRetargetPose(InRetargetPoseName);
	
	// record the version of the retarget pose (prevents re-initialization if profile swaps it)
	RetargetPose.Version = InRetargetPose->GetVersion();
	RetargetPose.PoseScale = InSourceScale;
	
	// initialize retarget pose to the skeletal mesh reference pose
	RetargetPose.LocalPose = InSkeleton.SkeletalMesh->GetRefSkeleton().GetRefBonePose();
	// copy local pose to global
	RetargetPose.GlobalPose = RetargetPose.LocalPose;
	// convert to global space
	InSkeleton.UpdateGlobalTransformsBelowBone(INDEX_NONE, RetargetPose.LocalPose, RetargetPose.GlobalPose);

	// strip scale (done AFTER generating global pose so that scales are baked into translation)
	for (int32 BoneIndex=0; BoneIndex<InSkeleton.BoneNames.Num(); ++BoneIndex)
	{
		RetargetPose.LocalPose[BoneIndex].SetScale3D(FVector::OneVector);
		RetargetPose.GlobalPose[BoneIndex].SetScale3D(FVector::OneVector);
	}

	// apply pelvis translation offset
	const int32 PelvisBoneIndex = InSkeleton.FindBoneIndexByName(PelvisBoneName);
	if (PelvisBoneIndex != INDEX_NONE)
	{
		FTransform& PelvisTransform = RetargetPose.GlobalPose[PelvisBoneIndex];
		PelvisTransform.AddToTranslation(InRetargetPose->GetRootTranslationDelta());
		InSkeleton.UpdateLocalTransformOfSingleBone(PelvisBoneIndex, RetargetPose.LocalPose, RetargetPose.GlobalPose);
	}

	// apply retarget pose offsets (retarget pose is stored as offset relative to reference pose)
	const TArray<FTransform>& RefPoseLocal = InSkeleton.SkeletalMesh->GetRefSkeleton().GetRefBonePose();
	for (const TTuple<FName, FQuat>& BoneDelta : InRetargetPose->GetAllDeltaRotations())
	{
		const int32 BoneIndex = InSkeleton.FindBoneIndexByName(BoneDelta.Key);
		if (BoneIndex == INDEX_NONE)
		{
			// this can happen if a retarget pose recorded a bone offset for a bone that is not present in the
			// target skeleton; ie, the retarget pose was generated from a different Skeletal Mesh with extra bones
			continue;
		}

		const FQuat LocalBoneRotation = RefPoseLocal[BoneIndex].GetRotation() * BoneDelta.Value;
		RetargetPose.LocalPose[BoneIndex].SetRotation(LocalBoneRotation);
	}

	// update global transforms based on local pose modified by the retarget pose offsets
	InSkeleton.UpdateGlobalTransformsBelowBone(INDEX_NONE, RetargetPose.LocalPose, RetargetPose.GlobalPose);

	// scale the global retarget pose
	if (RetargetPose.PoseScale.ScalePose(RetargetPose.GlobalPose))
	{
		// update the local transforms
		InSkeleton.UpdateLocalTransformsBelowBone(INDEX_NONE, RetargetPose.LocalPose, RetargetPose.GlobalPose);
	}

	return RetargetPose;
}

FResolvedRetargetPose& FResolvedRetargetPoseSet::FindOrAddRetargetPose(const FName InRetargetPoseName)
{
	for (FResolvedRetargetPose& Pose : RetargetPoses)
	{
		if (Pose.Name == InRetargetPoseName)
		{
			return Pose;
		}
	}
	
	int32 NewPoseIndex = RetargetPoses.Emplace(InRetargetPoseName);
	return RetargetPoses[NewPoseIndex];
}

const FResolvedRetargetPose* FResolvedRetargetPoseSet::FindRetargetPoseByName(const FName InRetargetPoseName) const
{
	for (const FResolvedRetargetPose& Pose : RetargetPoses)
	{
		if (Pose.Name == InRetargetPoseName)
		{
			return &Pose;
		}
	}
	
	return nullptr;
}

const TArray<FTransform>& FResolvedRetargetPoseSet::GetLocalRetargetPose() const
{
	if (const FResolvedRetargetPose* ResolvedRetargetPose = FindRetargetPoseByName(CurrentRetargetPoseName))
	{
		return ResolvedRetargetPose->LocalPose;
	}
	
	return RetargetPoses[0].LocalPose;
}

const TArray<FTransform>& FResolvedRetargetPoseSet::GetGlobalRetargetPose() const
{
	if (const FResolvedRetargetPose* ResolvedRetargetPose = FindRetargetPoseByName(CurrentRetargetPoseName))
	{
		return ResolvedRetargetPose->GlobalPose;
	}
	
	return RetargetPoses[0].GlobalPose;
}

FTransform FResolvedRetargetPoseSet::GetGlobalRetargetPoseOfSingleBone(
	const FRetargetSkeleton& InSkeleton,
	const int32 BoneIndex,
	const TArray<FTransform>& InGlobalPose) const
{
	const TArray<FTransform>& LocalRetargetPose = GetLocalRetargetPose();
	const int32 ParentIndex = InSkeleton.ParentIndices[BoneIndex];
	if (ParentIndex == INDEX_NONE)
	{
		return LocalRetargetPose[BoneIndex]; // root always in global space
	}
	const FTransform& ChildLocalTransform = LocalRetargetPose[BoneIndex];
	const FTransform& ParentGlobalTransform = InGlobalPose[ParentIndex];
	return ChildLocalTransform * ParentGlobalTransform;
}

void FResolvedRetargetPoseSet::Reset()
{
	RetargetPoses.Reset();
}

void FRetargetSkeleton::Initialize(
	const USkeletalMesh* InSkeletalMesh,
	const ERetargetSourceOrTarget InSourceOrTarget,
	const UIKRetargeter* InRetargetAsset,
	const FName PelvisBoneName,
	const FRetargetPoseScaleWithPivot& InPoseScale)
{
	// reset all skeleton data
	Reset();
	
	// record which skeletal mesh this is running on
	SkeletalMesh = InSkeletalMesh;
	
	// copy names and parent indices into local storage
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	for (int32 BoneIndex=0; BoneIndex<RefSkeleton.GetNum(); ++BoneIndex)
	{
		BoneNames.Add(RefSkeleton.GetBoneName(BoneIndex));
		ParentIndices.Add(RefSkeleton.GetParentIndex(BoneIndex));	
	}

	// initialize branch caching
	CachedEndOfBranchIndices.Init(RETARGETSKELETON_INVALID_BRANCH_INDEX, ParentIndices.Num());

	// add default retarget pose for this skeleton
	static FName DefaultRetargetPoseName = UIKRetargeter::GetDefaultPoseName();
	const FIKRetargetPose* DefaultRetargetPose = InRetargetAsset->GetRetargetPoseByName(InSourceOrTarget, UIKRetargeter::GetDefaultPoseName());
	RetargetPoses.AddOrUpdateRetargetPose(*this,DefaultRetargetPoseName, DefaultRetargetPose, PelvisBoneName, InPoseScale);
	RetargetPoses.CurrentRetargetPoseName = DefaultRetargetPoseName;

	// add current retarget pose
	const FName CurrentRetargetPoseName = InRetargetAsset->GetCurrentRetargetPoseName(InSourceOrTarget);
	const FIKRetargetPose* CurrentRetargetPose = InRetargetAsset->GetRetargetPoseByName(InSourceOrTarget, CurrentRetargetPoseName);
	if (CurrentRetargetPose)
	{
		RetargetPoses.AddOrUpdateRetargetPose(*this,CurrentRetargetPoseName, CurrentRetargetPose, PelvisBoneName, InPoseScale);
		// set this as the current pose to use (may be overridden by ops)
		RetargetPoses.CurrentRetargetPoseName = CurrentRetargetPoseName;
	}
}

void FRetargetSkeleton::Reset()
{
	BoneNames.Reset();
	ParentIndices.Reset();
	RetargetPoses.Reset();
	SkeletalMesh = nullptr;
}

int32 FRetargetSkeleton::FindBoneIndexByName(const FName InName) const
{
	return BoneNames.IndexOfByPredicate([&InName](const FName BoneName)
	{
		return BoneName == InName;
	});
}

void FRetargetSkeleton::UpdateGlobalTransformsBelowBone(
	const int32 StartBoneIndex,
	const TArray<FTransform>& InLocalPose,
	TArray<FTransform>& OutGlobalPose) const
{
	check(BoneNames.IsValidIndex(StartBoneIndex+1));
	check(BoneNames.Num() == InLocalPose.Num());
	check(BoneNames.Num() == OutGlobalPose.Num());
	
	for (int32 BoneIndex=StartBoneIndex+1; BoneIndex<OutGlobalPose.Num(); ++BoneIndex)
	{
		UpdateGlobalTransformOfSingleBone(BoneIndex, InLocalPose[BoneIndex],OutGlobalPose);
	}
}

void FRetargetSkeleton::SetGlobalTransformAndUpdateChildren(
	const int32 InBoneToSetIndex,
	const FTransform& InNewTransform,
	TArray<FTransform>& InOutGlobalPose) const
{
	check(BoneNames.Num() == InOutGlobalPose.Num());
	check(InOutGlobalPose.IsValidIndex(InBoneToSetIndex));

	const FTransform PrevTransform = InOutGlobalPose[InBoneToSetIndex];
	InOutGlobalPose[InBoneToSetIndex] = InNewTransform;

	TArray<int32> ChildBoneIndices;
	GetChildrenIndicesRecursive(InBoneToSetIndex, ChildBoneIndices);

	for (const int32 ChildBoneIndex : ChildBoneIndices)
	{
		const FTransform RelativeToPrev = InOutGlobalPose[ChildBoneIndex].GetRelativeTransform(PrevTransform);
		InOutGlobalPose[ChildBoneIndex] = RelativeToPrev * InNewTransform;
		InOutGlobalPose[ChildBoneIndex].NormalizeRotation();
	}
}

void FRetargetSkeleton::UpdateLocalTransformsBelowBone(
	const int32 StartBoneIndex,
	TArray<FTransform>& OutLocalPose,
	const TArray<FTransform>& InGlobalPose) const
{
	check(BoneNames.IsValidIndex(StartBoneIndex+1));
	check(BoneNames.Num() == OutLocalPose.Num());
	check(BoneNames.Num() == InGlobalPose.Num());
	
	for (int32 BoneIndex=StartBoneIndex+1; BoneIndex<InGlobalPose.Num(); ++BoneIndex)
	{
		UpdateLocalTransformOfSingleBone(BoneIndex, OutLocalPose, InGlobalPose);
	}
}

void FRetargetSkeleton::UpdateGlobalTransformOfSingleBone(
	const int32 BoneIndex,
	const FTransform& InLocalTransform,
	TArray<FTransform>& OutGlobalPose) const
{
	const int32 ParentIndex = ParentIndices[BoneIndex];
	if (ParentIndex == INDEX_NONE)
	{
		// root always in global space already, no conversion required
		OutGlobalPose[BoneIndex] = InLocalTransform;
		return; 
	}
	const FTransform& ChildLocalTransform = InLocalTransform;
	const FTransform& ParentGlobalTransform = OutGlobalPose[ParentIndex];
	OutGlobalPose[BoneIndex] = ChildLocalTransform * ParentGlobalTransform;
}

void FRetargetSkeleton::UpdateLocalTransformOfSingleBone(
	const int32 BoneIndex,
	TArray<FTransform>& OutLocalPose,
	const TArray<FTransform>& InGlobalPose) const
{
	const int32 ParentIndex = ParentIndices[BoneIndex];
	if (ParentIndex == INDEX_NONE)
	{
		// root bone, so just set the local pose to the global pose
		OutLocalPose[BoneIndex] = InGlobalPose[BoneIndex];
		return;
	}
	const FTransform& ChildGlobalTransform = InGlobalPose[BoneIndex];
	const FTransform& ParentGlobalTransform = InGlobalPose[ParentIndex];
	OutLocalPose[BoneIndex] = ChildGlobalTransform.GetRelativeTransform(ParentGlobalTransform);
}

FTransform FRetargetSkeleton::GetLocalTransformOfSingleBone(
	const int32 BoneIndex,
	const TArray<FTransform>& InGlobalPose) const
{
	const FTransform& GlobalTransform = InGlobalPose[BoneIndex];
	const int32 ParentIndex = ParentIndices[BoneIndex];
	if (ParentIndex == INDEX_NONE)
	{
		// root bone, so local transform is same as global
		return GlobalTransform;
	}
	
	const FTransform& ParentGlobalTransform = InGlobalPose[ParentIndex];
	return GlobalTransform.GetRelativeTransform(ParentGlobalTransform);
}

TArray<FTransform> FRetargetSkeleton::GetLocalTransformsOfMultipleBones(
	const TArray<int32>& InBoneIndices,
	const TArray<FTransform>& InGlobalPose) const
{
	TArray<FTransform> OutputLocalTransforms;

	for (const int32 ChildIndex : InBoneIndices)
	{
		const int32 ParentIndex = ParentIndices[ChildIndex];
		if (ParentIndex == INDEX_NONE)
		{
			OutputLocalTransforms.Add(InGlobalPose[ChildIndex]);
			continue;
		}
		const FTransform& ChildGlobalTransform = InGlobalPose[ChildIndex];
		const FTransform& ParentGlobalTransform = InGlobalPose[ParentIndex];
		const FTransform LocalTransform = ChildGlobalTransform.GetRelativeTransform(ParentGlobalTransform);
		OutputLocalTransforms.Add(LocalTransform);
	}
	
	return MoveTemp(OutputLocalTransforms);
}

void FRetargetSkeleton::UpdateGlobalTransformsOfMultipleBones(
	const TArray<int32>& InBoneIndices,
	const TArray<FTransform>& InLocalTransforms,
	TArray<FTransform>& OutGlobalPose) const
{
	if (!ensure(InLocalTransforms.Num() == InBoneIndices.Num()))
	{
		return;
	}
	
	for (int32 BoneArrayIndex=0; BoneArrayIndex < InBoneIndices.Num(); ++BoneArrayIndex)
	{
		const int32 ChildIndex = InBoneIndices[BoneArrayIndex];
		const int32 ParentIndex = ParentIndices[ChildIndex];
		if (ParentIndex == INDEX_NONE)
		{
			// root always in global space already, no conversion required
			OutGlobalPose[ChildIndex] = InLocalTransforms[BoneArrayIndex];
			continue; 
		}
		const FTransform& ChildLocalTransform = InLocalTransforms[BoneArrayIndex];
		const FTransform& ParentGlobalTransform = OutGlobalPose[ParentIndex];
		OutGlobalPose[ChildIndex] = ChildLocalTransform * ParentGlobalTransform;
	}
}

int32 FRetargetSkeleton::GetCachedEndOfBranchIndex(const int32 InBoneIndex) const
{
	if (!CachedEndOfBranchIndices.IsValidIndex(InBoneIndex))
	{
		return INDEX_NONE;
	}

	// already cached
	if (CachedEndOfBranchIndices[InBoneIndex] != RETARGETSKELETON_INVALID_BRANCH_INDEX)
	{
		return CachedEndOfBranchIndices[InBoneIndex];
	}

	const int32 NumBones = BoneNames.Num();
	
	// if we're asking for the first or last bone, return the last bone  
	if (InBoneIndex == 0 || InBoneIndex + 1 >= NumBones)
	{
		CachedEndOfBranchIndices[InBoneIndex] = NumBones-1;
		return CachedEndOfBranchIndices[InBoneIndex];
	}

	CachedEndOfBranchIndices[InBoneIndex] = INDEX_NONE;
	const int32 StartParentIndex = GetParentIndex(InBoneIndex);
	int32 BoneIndex = InBoneIndex + 1;
	int32 ParentIndex = GetParentIndex(BoneIndex);

	// if next child bone's parent is less than or equal to StartParentIndex,
	// we are leaving the branch so no need to go further
	int32 BoneIndexAtEndOfBranch = RETARGETSKELETON_INVALID_BRANCH_INDEX;
	while (ParentIndex > StartParentIndex)
	{
		BoneIndexAtEndOfBranch = BoneIndex;
		BoneIndex++;
		if (BoneIndex >= NumBones)
		{
			break;
		}
		ParentIndex = GetParentIndex(BoneIndex);
	}

	// set once (outside of while loop above) to avoid potential race condition
	CachedEndOfBranchIndices[InBoneIndex] = BoneIndexAtEndOfBranch;

	return CachedEndOfBranchIndices[InBoneIndex];
}

void FRetargetSkeleton::GetChildrenIndices(const int32 BoneIndex, TArray<int32>& OutChildren) const
{
	const int32 LastBranchIndex = GetCachedEndOfBranchIndex(BoneIndex);
	if (LastBranchIndex == INDEX_NONE)
	{
		// no children (leaf bone)
		return;
	}
	
	for (int32 ChildBoneIndex = BoneIndex + 1; ChildBoneIndex <= LastBranchIndex; ChildBoneIndex++)
	{
		if (GetParentIndex(ChildBoneIndex) == BoneIndex)
		{
			OutChildren.Add(ChildBoneIndex);
		}
	}
}

void FRetargetSkeleton::GetChildrenIndicesRecursive(const int32 BoneIndex, TArray<int32>& OutChildren) const
{
	const int32 LastBranchIndex = GetCachedEndOfBranchIndex(BoneIndex);
	if (LastBranchIndex == INDEX_NONE)
	{
		// no children (leaf bone)
		return;
	}
	
	for (int32 ChildBoneIndex = BoneIndex + 1; ChildBoneIndex <= LastBranchIndex; ChildBoneIndex++)
	{
		OutChildren.Add(ChildBoneIndex);
	}
}

bool FRetargetSkeleton::IsParentOf(const int32 PotentialParentIndex, const int32 ChildBoneIndex) const
{
	int32 ParentIndex = GetParentIndex(ChildBoneIndex);
	while (ParentIndex != INDEX_NONE)
	{
		if (ParentIndex == PotentialParentIndex)
		{
			return true;
		}
		
		ParentIndex = GetParentIndex(ParentIndex);
	}
	
	return false;
}

int32 FRetargetSkeleton::GetParentIndex(const int32 BoneIndex) const
{
	if (BoneIndex < 0 || BoneIndex>=ParentIndices.Num() || BoneIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	return ParentIndices[BoneIndex];
}

const FTransform& FRetargetSkeleton::GetParentTransform(const int32 BoneIndex, const TArray<FTransform>& InPose) const
{
	int32 ParentIndex = GetParentIndex(BoneIndex);
	if (ParentIndex == INDEX_NONE)
	{
		return FTransform::Identity;
	}

	return InPose[ParentIndex];
}

void FTargetSkeleton::Initialize(
const USkeletalMesh* InSkeletalMesh,
	const ERetargetSourceOrTarget InSourceOrTarget,
	const UIKRetargeter* InRetargetAsset,
	const FName RetargetRootBone,
	const FRetargetPoseScaleWithPivot& InPoseScale)
{
	Reset();
	
	FRetargetSkeleton::Initialize(InSkeletalMesh, InSourceOrTarget, InRetargetAsset, RetargetRootBone, InPoseScale);

	// make storage for per-bone "Is Retargeted" flag (used for hierarchy updates)
	// these are bones that are in a target chain that is mapped to a source chain (ie, will actually be retargeted)
	// these flags are actually set later between Op::Initialize() and Op::PostInitialize()
	bIsMaskInitialized = false;
	IsBoneRetargeted.Init(false, BoneNames.Num());

	// initialize base local pose (may be overridden by pose copier, but uses retarget pose by default)
	InputLocalPose = RetargetPoses.GetLocalRetargetPose();
	
	// initialize storage for output pose (the result of the retargeting)
	OutputGlobalPose = RetargetPoses.GetGlobalRetargetPose();
}

void FTargetSkeleton::Reset()
{
	FRetargetSkeleton::Reset();
	OutputGlobalPose.Reset();
}

void FTargetSkeleton::SetRetargetedBones(const TSet<int32>& InRetargetedBones)
{
	IsBoneRetargeted.Init(false, BoneNames.Num());
	for (int32 BoneIndex : InRetargetedBones)
	{
		IsBoneRetargeted[BoneIndex] = true;
	}
	bIsMaskInitialized = true;
}

bool FTargetSkeleton::GetIsBoneRetargeted(int32 InBoneIndex) const
{
	if (!ensure(bIsMaskInitialized))
	{
		return false;
	}
	
	return IsBoneRetargeted[InBoneIndex];
}

const TArray<bool>& FTargetSkeleton::GetRetargetedBonesMask() const
{
	return IsBoneRetargeted;
}

FResolvedBoneChain::FResolvedBoneChain(
	const FBoneChain& InBoneChain,
	const FRetargetSkeleton& InSkeleton,
	FIKRigLogger& InLog)
{
	// store the bone chain data
	ChainName = InBoneChain.ChainName;
	StartBone = InBoneChain.StartBone.BoneName;
	EndBone = InBoneChain.EndBone.BoneName;
	IKGoalName = InBoneChain.IKGoalName;
	
	// validate start and end bones exist and are not the root
	const int32 StartIndex = InSkeleton.FindBoneIndexByName(StartBone);
	const int32 EndIndex = InSkeleton.FindBoneIndexByName(EndBone);
	bFoundStartBone = StartIndex > INDEX_NONE;
	bFoundEndBone = EndIndex > INDEX_NONE;

	// no need to build the chain if start/end indices are wrong 
	const bool bIsWellFormed = bFoundStartBone && bFoundEndBone && EndIndex >= StartIndex;
	if (!bIsWellFormed)
	{
		return;
	}
	
	// init array with end bone 
	BoneIndices = {EndIndex};

	// record all bones in chain while walking up the hierarchy (tip to root of chain)
	int32 ParentIndex = InSkeleton.GetParentIndex(EndIndex);
	while (ParentIndex > INDEX_NONE && ParentIndex >= StartIndex)
	{
		BoneIndices.Add(ParentIndex);
		ParentIndex = InSkeleton.GetParentIndex(ParentIndex);
	}

	// did we walk all the way up without finding the start bone?
	if (BoneIndices.Last() != StartIndex)
	{
		BoneIndices.Reset();
		bEndIsStartOrChildOfStart = false;
		return;
	}
	
	// confirm that end bone is child of start bone
	bEndIsStartOrChildOfStart = true;
	
	// reverse the indices (we want root to tip order)
	Algo::Reverse(BoneIndices);

	// update the ref pose stored in the chain
	UpdatePoseFromSkeleton(InSkeleton);
	
	// calculate parameter of each bone, normalized by the length of the bone chain
	CalculateBoneParameters(InLog);
}

bool FResolvedBoneChain::IsValid() const
{
	return bFoundStartBone &&
		bFoundEndBone &&
		bEndIsStartOrChildOfStart &&
		BoneIndices.Num() == RefPoseGlobalTransforms.Num() &&
		BoneIndices.Num() == RefPoseLocalTransforms.Num();
}

void FResolvedBoneChain::UpdatePoseFromSkeleton(const FRetargetSkeleton& InSkeleton)
{
	const TArray<FTransform>& GlobalRetargetPose = InSkeleton.RetargetPoses.GetGlobalRetargetPose();
	
	// store all the initial bone transforms in the bone chain
	RefPoseGlobalTransforms.Reset();
	for (int32 Index=0; Index < BoneIndices.Num(); ++Index)
	{
		const int32 BoneIndex = BoneIndices[Index];
		if (ensure(GlobalRetargetPose.IsValidIndex(BoneIndex)))
		{
			RefPoseGlobalTransforms.Emplace(GlobalRetargetPose[BoneIndex]);
		}
	}

	// get the local space of the chain in retarget pose
	RefPoseLocalTransforms.SetNum(RefPoseGlobalTransforms.Num());
	FillTransformsWithLocalSpaceOfChain(InSkeleton, GlobalRetargetPose, BoneIndices, RefPoseLocalTransforms);

	// initialize storage for current local transforms
	CurrentLocalTransforms = RefPoseLocalTransforms;
	
	// store chain parent data
	ChainParentBoneIndex = InSkeleton.GetParentIndex(BoneIndices[0]);
	ChainParentInitialGlobalTransform = FTransform::Identity;
	if (ChainParentBoneIndex != INDEX_NONE)
	{
		ChainParentInitialGlobalTransform = GlobalRetargetPose[ChainParentBoneIndex];
	}
}

void FResolvedBoneChain::GetWarnings(const FRetargetSkeleton& Skeleton, FIKRigLogger& Log) const
{
	// warn if START bone not found
	if (!bFoundStartBone)
	{
		Log.LogWarning( FText::Format(
			LOCTEXT("MissingStartBone", "IK Retargeter bone chain, {0}, could not find start bone, {1} in mesh {2}"),
			FText::FromName(ChainName),
			FText::FromName(StartBone),
			FText::FromString(Skeleton.SkeletalMesh->GetName())));
	}
	
	// warn if END bone not found
	if (!bFoundEndBone)
	{
		Log.LogWarning( FText::Format(
			LOCTEXT("MissingEndBone", "IK Retargeter bone chain, {0}, could not find end bone, {1} in mesh {2}"),
			FText::FromName(ChainName), FText::FromName(EndBone), FText::FromString(Skeleton.SkeletalMesh->GetName())));
	}

	// warn if END bone was not a child of START bone
	if (bFoundEndBone && !bEndIsStartOrChildOfStart)
	{
		Log.LogWarning( FText::Format(
			LOCTEXT("EndNotChildtOfStart", "IK Retargeter bone chain, {0}, end bone, '{1}' was not a child of the start bone '{2}'."),
			FText::FromName(ChainName), FText::FromName(EndBone), FText::FromName(StartBone)));
	}

	// cannot retarget chain if all the bones are sitting directly on each other
	if (InitialChainLength <= UE_KINDA_SMALL_NUMBER)
	{
		Log.LogWarning( FText::Format(
			LOCTEXT("FailedParamOfChain", "IK Retargeter bone chain, {0}, was unable to be normalized. Chain too short."),
			FText::FromName(ChainName)));
	}
}

FTransform FResolvedBoneChain::GetTransformAtChainParam(const TArray<FTransform>& Transforms, const double& Param) const
{
	check(Transforms.Num() == Params.Num());
	
	if (Params.Num() == 1)
	{
		return Transforms[0];
	}
	
	if (Param < UE_KINDA_SMALL_NUMBER)
	{
		return Transforms[0];
	}

	if (Param > 1.0f - UE_KINDA_SMALL_NUMBER)
	{
		return Transforms.Last();
	}

	for (int32 ChainIndex=1; ChainIndex<Params.Num(); ++ChainIndex)
	{
		const double CurrentParam = Params[ChainIndex];
		if (CurrentParam <= Param)
		{
			continue;
		}
		
		const double PrevParam = Params[ChainIndex-1];
		const double PercentBetweenParams = (Param - PrevParam) / (CurrentParam - PrevParam);
		const FTransform& Prev = Transforms[ChainIndex-1];
		const FTransform& Next = Transforms[ChainIndex];
		const FVector Position = FMath::Lerp(Prev.GetTranslation(), Next.GetTranslation(), PercentBetweenParams);
		const FQuat Rotation = FQuat::FastLerp(Prev.GetRotation(), Next.GetRotation(), PercentBetweenParams).GetNormalized();
		const FVector Scale = FMath::Lerp(Prev.GetScale3D(), Next.GetScale3D(), PercentBetweenParams);
		
		return FTransform(Rotation,Position, Scale);
	}

	// fallback — can be reached with degenerate chains (zero-length bones, duplicate params)
	return Transforms.Last();
}

double FResolvedBoneChain::GetStretchAtParam(
	const TArray<FTransform>& InitialTransforms,
	const TArray<FTransform>& CurrentTransforms,
	const double& Param) const
{
	check(InitialTransforms.Num() == CurrentTransforms.Num() && InitialTransforms.Num() == Params.Num());

	// chain only has 1 bone, so it can't stretch
	if (Params.Num() <= 1)
	{
		return 1.0;
	}

	// start of chain cannot stretch
	if (Param < UE_KINDA_SMALL_NUMBER)
	{
		return 1.0;
	}

	double InitialLength;
	double CurrentLength;
	
	// end of chain always uses last bone
	if (Param > 1.0 - UE_KINDA_SMALL_NUMBER)
	{
		const int32 Last = InitialTransforms.Num() - 1;
		const int32 Prev = Last - 1;
		InitialLength = (InitialTransforms[Last].GetTranslation() - InitialTransforms[Prev].GetTranslation()).Length();
		CurrentLength = (CurrentTransforms[Last].GetTranslation() - CurrentTransforms[Prev].GetTranslation()).Length();
	}
	else
	{
		// find the bone associated with this parameter
		int32 ClosestBoneChainIndex = GetBoneClosestToParam(Param);
		int32 BoneStartIndex = ClosestBoneChainIndex - 1;
		int32 BoneEndIndex = ClosestBoneChainIndex;

		if (ClosestBoneChainIndex == 0)
		{
			BoneStartIndex = 0;
			BoneEndIndex = 1;
		}
		
		InitialLength = (InitialTransforms[BoneStartIndex].GetTranslation() - InitialTransforms[BoneEndIndex].GetTranslation()).Length();
		CurrentLength = (CurrentTransforms[BoneStartIndex].GetTranslation() - CurrentTransforms[BoneEndIndex].GetTranslation()).Length();
	}

	// calculate the ratio of the current to the initial length of the closest bone
	return (InitialLength < UE_KINDA_SMALL_NUMBER ? 1.0 : CurrentLength / InitialLength);
}

int32 FResolvedBoneChain::GetBoneClosestToParam(const double& Param) const
{
	// find the chain index of the bone closest to Param
	double ClosestParamDistance = TNumericLimits<double>::Max();
	int32 ClosestBoneChainIndex = INDEX_NONE;
	for (int32 ChainIndex=0; ChainIndex<Params.Num(); ++ChainIndex)
	{
		double ParamDistance = FMath::Abs(Params[ChainIndex] - Param);
		if (ParamDistance < ClosestParamDistance)
		{
			ClosestParamDistance = ParamDistance;
			ClosestBoneChainIndex = ChainIndex;
		}
	}
	
	return ClosestBoneChainIndex;
}

int32 FResolvedBoneChain::GetBoneAtParam(const double& Param) const
{
	if (Param > 1.0f || FMath::IsNearlyEqual(Param, 1.0))
	{
		return Params.Num() - 1;
	}
	
	for (int32 ChainIndex=0; ChainIndex<Params.Num(); ++ChainIndex)
	{
		constexpr double Tolerance = 0.05;
		if (FMath::IsNearlyEqual(Params[ChainIndex], Param, Tolerance))
		{
			return ChainIndex;
		}
		
		if (Param < Params[ChainIndex])
		{
			return ChainIndex;
		}
	}

	return Params.Num() - 1;
}

int32 FResolvedBoneChain::GetEquivalentBoneInOtherChain(const int32 InChainIndex, const FResolvedBoneChain& InOtherBoneChain) const
{
	// use 1:1 mapping if chains have same number of bones
	if (BoneIndices.Num() == InOtherBoneChain.BoneIndices.Num())
	{
		return InChainIndex;
	}

	// otherwise, use the closest bone
	return InOtherBoneChain.GetBoneClosestToParam(Params[InChainIndex]);
}

double FResolvedBoneChain::GetChainLength(const TArray<FTransform>& Transforms)
{
	double TotalLength = 0.0f;
	for (int32 ChainIndex = 1; ChainIndex < Transforms.Num(); ++ChainIndex)
	{
		const FVector Start = Transforms[ChainIndex - 1].GetTranslation();
		const FVector End = Transforms[ChainIndex].GetTranslation();
		TotalLength += static_cast<float>(FVector::Dist(Start, End));
	}
	return TotalLength;
}

void FResolvedBoneChain::FillTransformsWithLocalSpaceOfChain(
	const FRetargetSkeleton& Skeleton,
	const TArray<FTransform>& InGlobalPose,
	const TArray<int32>& InBoneIndices,
	TArray<FTransform>& OutLocalTransforms)
{
	check(InBoneIndices.Num() == OutLocalTransforms.Num())
	
	for (int32 ChainIndex=0; ChainIndex<InBoneIndices.Num(); ++ChainIndex)
	{
		const int32 BoneIndex = InBoneIndices[ChainIndex];
		const int32 ParentIndex = Skeleton.GetParentIndex(BoneIndex);
		if (ParentIndex == INDEX_NONE)
		{
			// root is always in "global" space
			OutLocalTransforms[ChainIndex] = InGlobalPose[BoneIndex];
			continue;
		}

		const FTransform& ChildGlobalTransform = InGlobalPose[BoneIndex];
		const FTransform& ParentGlobalTransform = InGlobalPose[ParentIndex];
		OutLocalTransforms[ChainIndex] = ChildGlobalTransform.GetRelativeTransform(ParentGlobalTransform);
	}
}

void FResolvedBoneChain::FillTransformsWithGlobalRetargetPoseOfChain(
	const FRetargetSkeleton& InSkeleton,
	const TArray<FTransform>& InGlobalPose,
	const TArray<int32>& InBoneIndices,
	TArray<FTransform>& OutGlobalTransforms)
{
	OutGlobalTransforms.Reset(InBoneIndices.Num());
	
	// update chain current transforms to the retarget pose in global space
	TArray<FTransform> LocalRetargetPose = InSkeleton.RetargetPoses.GetLocalRetargetPose();
	for (int32 ChainIndex=0; ChainIndex<InBoneIndices.Num(); ++ChainIndex)
	{
		// update first bone in chain based on the incoming parent
		if (ChainIndex == 0)
		{
			const int32 BoneIndex = InBoneIndices[ChainIndex];
			OutGlobalTransforms.Add(InSkeleton.RetargetPoses.GetGlobalRetargetPoseOfSingleBone(InSkeleton, BoneIndex, InGlobalPose));
		}
		else
		{
			// all subsequent bones in chain are based on previous parent
			const int32 BoneIndex = InBoneIndices[ChainIndex];
			const FTransform& ParentGlobalTransform = OutGlobalTransforms[ChainIndex-1];
			const FTransform& ChildLocalTransform = LocalRetargetPose[BoneIndex];
			OutGlobalTransforms.Add(ChildLocalTransform * ParentGlobalTransform);
		}
	}
}

TArray<FTransform> FResolvedBoneChain::GetChainTransformsFromPose(const TArray<FTransform>& InPose) const
{
	TArray<FTransform> Transforms;
	Transforms.Reset(BoneIndices.Num());
	for (const int32 BoneIndex : BoneIndices)
	{
		Transforms.Add(InPose[BoneIndex]);
	}
	return MoveTemp(Transforms);
}

void FResolvedBoneChain::CalculateBoneParameters(FIKRigLogger& Log)
{
	Params.Reset();
	
	// special case, a single-bone chain
	if (RefPoseGlobalTransforms.Num() == 1)
	{
		Params.Add(1.0f);
		return;
	}

	// calculate bone lengths in chain and accumulate total length
	TArray<float> BoneDistances;
	InitialChainLength = 0.0f;
	BoneDistances.Add(0.0f);
	for (int32 i=1; i<RefPoseGlobalTransforms.Num(); ++i)
	{
		InitialChainLength += static_cast<float>((RefPoseGlobalTransforms[i].GetTranslation() - RefPoseGlobalTransforms[i-1].GetTranslation()).Size());
		BoneDistances.Add(InitialChainLength);
	}

	// very small chains will not retarget correctly, this will generate a warning
	const float Divisor = InitialChainLength > UE_KINDA_SMALL_NUMBER ? InitialChainLength : UE_KINDA_SMALL_NUMBER;

	// calc each bone's param along length
	for (int32 i=0; i<RefPoseGlobalTransforms.Num(); ++i)
	{
		Params.Add(BoneDistances[i] / Divisor); 
	}
}

bool FRetargeterBoneChains::Initialize(
	const UIKRetargeter* InRetargetAsset,
	const TArray<const UIKRigDefinition*>& InTargetIKRigs,
	const FRetargetSkeleton& InSourceSkeleton,
	const FRetargetSkeleton& InTargetSkeleton,
	FIKRigLogger& InLog)
{
	// load ALL bone chains on source and target (ops may use them even if they aren't mapped)
	auto LoadChains = [](
		const FRetargetSkeleton& InRetargetSkeleton,
		const TArray<FBoneChain>& InBoneChains,
		FIKRigLogger& InLog,
		TArray<FResolvedBoneChain>& OutResolvedChains)
	{
		for (const FBoneChain& BoneChain : InBoneChains)
		{
			FResolvedBoneChain NewChain(BoneChain, InRetargetSkeleton, InLog);
			NewChain.GetWarnings(InRetargetSkeleton, InLog);
			if (NewChain.IsValid())
			{
				OutResolvedChains.Add(NewChain);
			}
		}
	};

	// reset all the chains
	Reset();

	// store the default IK Rig (may be null)
	DefaultTargetIKRig = InTargetIKRigs.IsEmpty() ? nullptr : InTargetIKRigs[0];

	// load chains for source
	const UIKRigDefinition* SourceIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source);
	if (SourceIKRig)
	{
		LoadChains(InSourceSkeleton, SourceIKRig->GetRetargetChains(), InLog, SourceBoneChains);
	}

	// load chains for all target IK rigs
	for (const UIKRigDefinition* TargetIKRig : InTargetIKRigs)
	{
		TArray<FResolvedBoneChain>& BoneChains = TargetBoneChains.Add(TargetIKRig);
		LoadChains(InTargetSkeleton, TargetIKRig->GetRetargetChains(), InLog, BoneChains);
	}

	// sort the chains based on their StartBone's index
	auto ChainsSorter = [this](const FResolvedBoneChain& A, const FResolvedBoneChain& B)
	{
		const int32 IndexA = A.BoneIndices.Num() > 0 ? A.BoneIndices[0] : INDEX_NONE;
		const int32 IndexB = B.BoneIndices.Num() > 0 ? B.BoneIndices[0] : INDEX_NONE;
		if (IndexA == IndexB)
		{
			// fallback to sorting alphabetically
			return A.ChainName.LexicalLess(B.ChainName);
		}
		return IndexA < IndexB;
	};
	SourceBoneChains.Sort(ChainsSorter);
	for (TTuple<const UIKRigDefinition*, TArray<FResolvedBoneChain>>& TargetChains : TargetBoneChains)
	{
		TargetChains.Value.Sort(ChainsSorter);
	}

	return true;
}

const TArray<FResolvedBoneChain>* FRetargeterBoneChains::GetResolvedBoneChains(
	ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InTargetIKRig) const
{
	if (SourceOrTarget == ERetargetSourceOrTarget::Source)
	{
		return &SourceBoneChains;
	}

	if (InTargetIKRig)
	{
		return TargetBoneChains.Find(InTargetIKRig);
	}

	return TargetBoneChains.Find(DefaultTargetIKRig);
}

const TMap<const UIKRigDefinition*, TArray<FResolvedBoneChain>>& FRetargeterBoneChains::GetAllResolvedTargetBoneChains() const
{
	return TargetBoneChains;
}

const FResolvedBoneChain* FRetargeterBoneChains::GetResolvedBoneChainByName(
	const FName InChainName,
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* TargetIKRig) const
{
	const TArray<FResolvedBoneChain>* ResolvedBoneChains = GetResolvedBoneChains(SourceOrTarget, TargetIKRig);
	if (!ResolvedBoneChains)
	{
		return nullptr;
	}
	
	return ResolvedBoneChains->FindByPredicate([InChainName](const FResolvedBoneChain& Value)
	{
		return Value.ChainName == InChainName;
	});
}

void FRetargeterBoneChains::UpdatePoseFromSkeleton(
	const FRetargetSkeleton& InSkeleton,
	const ERetargetSourceOrTarget SourceOrTarget)
{
	// update source poses
	if (SourceOrTarget == ERetargetSourceOrTarget::Source)
	{
		for (FResolvedBoneChain& BoneChain : SourceBoneChains)
		{
			BoneChain.UpdatePoseFromSkeleton(InSkeleton);
		}
		return;
	}

	// update target poses
	for (TTuple<const UIKRigDefinition*, TArray<FResolvedBoneChain>>& Pair : TargetBoneChains)
	{
		for (FResolvedBoneChain& BoneChain : Pair.Value)
		{
			BoneChain.UpdatePoseFromSkeleton(InSkeleton);
		}
	}
}

TSet<FName> FRetargeterBoneChains::GetChainsThatContainBone(int32 InBoneIndex, ERetargetSourceOrTarget SourceOrTarget) const
{
	TSet<FName> ChainsThatContainBone;
	
	if (SourceOrTarget == ERetargetSourceOrTarget::Source)
	{
		// source
		for (const FResolvedBoneChain& SourceChain : SourceBoneChains)
		{
			if (SourceChain.BoneIndices.Contains(InBoneIndex))
			{
				ChainsThatContainBone.Add(SourceChain.ChainName);
			}
		}
		return MoveTemp(ChainsThatContainBone);
	}
	else
	{
		// target
		for (const TTuple<const UIKRigDefinition*, TArray<FResolvedBoneChain>>& TargetPair : TargetBoneChains)
		{
			for (const FResolvedBoneChain& TargetChain : TargetPair.Value)
			{
				if (TargetChain.BoneIndices.Contains(InBoneIndex))
				{
					ChainsThatContainBone.Add(TargetChain.ChainName);
				}
			}
		}
		return MoveTemp(ChainsThatContainBone);
	}
}

void FRetargeterBoneChains::Reset()
{
	SourceBoneChains.Reset();
	TargetBoneChains.Reset();
}

void FIKRetargetProcessor::InitialOpStackSetup(
	const TArray<FInstancedStruct>& OpsFromAsset,
	const FRetargetProfile* InRetargetProfile)
{
	// create copies of all the ops in the asset
	OpStack.Reset(OpsFromAsset.Num());
	for (const FInstancedStruct& AssetOpStruct : OpsFromAsset)
	{
		if (!ensure(AssetOpStruct.IsValid()))
		{
			// this can happen if asset references deleted op type which should only happen during development (if at all)
			Log.LogWarning(LOCTEXT("UnknownOP", "IK Retargeter, '{0}' has null/unknown op in it. Please reload the asset to remove it."));
			continue;
		}

		// create a copy of the op
		int32 NewOpIndex = OpStack.Emplace(AssetOpStruct);
		// apply the profile settings to the op (may be different than what was loaded from the asset)
		if (InRetargetProfile)
		{
			InRetargetProfile->ApplyOpProfilesToOpStruct(OpStack[NewOpIndex], ECopyOpSettingsContext::PreInitialize);
		}
		
#if WITH_EDITOR
		FIKRetargetOpBase& NewOp = OpStack[NewOpIndex].GetMutable<FIKRetargetOpBase>();
		FIKRetargetOpBase* AssetOp = const_cast<FIKRetargetOpBase*>(AssetOpStruct.GetPtr<FIKRetargetOpBase>());
		FIKRetargetOpSettingsBase* AssetOpSettings = AssetOp->GetSettings();
		// asset instance gets reference to running editor instance
		AssetOpSettings->EditorInstance = NewOp.GetSettings();
		AssetOp->EditorInstance = &NewOp;
		NewOp.Timer.Reset();
		// references the skeletons are used for bone selector widgets in the details panel
		AssetOpSettings->AssignSkeletonAssets(
			SourceSkeleton.SkeletalMesh->GetSkeleton(),
			TargetSkeleton.SkeletalMesh->GetSkeleton());
#endif

	}

	// collect all the IK Rigs referenced by the ops
	TArray<const UIKRigDefinition*> AllIKRigs;

	// add the default target IK Rig
	const UIKRigDefinition* DefaultTargetIKRig = RetargeterAsset->GetIKRig(ERetargetSourceOrTarget::Target);
	if (DefaultTargetIKRig)
	{
		AllIKRigs.Add(DefaultTargetIKRig);
	}
	// add any IK Rigs referenced by the ops
	for (FInstancedStruct& OpStruct : OpStack)
	{
		FIKRetargetOpBase& NewOp = OpStruct.GetMutable<FIKRetargetOpBase>();
		if (const UIKRigDefinition* TargetIKRigFromOp = NewOp.GetCustomTargetIKRig())
		{
			AllIKRigs.AddUnique(TargetIKRigFromOp);
		}
	}
	
	// resolve all bone chains onto the skeletons
	AllBoneChains.Initialize(RetargeterAsset, AllIKRigs, SourceSkeleton, TargetSkeleton, Log);

	// record meta-data to build execution order
	InitializeOpEntries();
}

void FIKRetargetProcessor::InitializeRetargetOps()
{
	auto PostInitializeProcessor = [this]()
		{
			// gather retargeted bones from each op
			TSet<int32> AllRetargetedBones;
			for (FInstancedStruct& OpStruct : OpStack)
			{
				FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
				if (Op.IsInitialized())
				{
					Op.CollectRetargetedBones(AllRetargetedBones);
				}
			}
	
			// store retargeted bone mask on target skeleton
			TargetSkeleton.SetRetargetedBones(AllRetargetedBones);
		};
	
	// build a stack of op calls in the correct order for initialization
	TArray<FRetargetOpCall> InitializationStack;
	BuildInitializationStack(InitializationStack);

	// initialize the ops
	for (FRetargetOpCall& Call : InitializationStack)
	{
		FIKRetargetOpBase* Op = Call.GetOp();
		
		switch (Call.Phase)
		{
		case ERetargetOpPhase::InitializeBeforeChildren:
			Op->InitializeBeforeChildren(*this, SourceSkeleton, TargetSkeleton, Log);
			break;

		case ERetargetOpPhase::Initialize:
			Op->Initialize(*this, SourceSkeleton, TargetSkeleton, Call.GetParentOp(), Log);
			break;

		case ERetargetOpPhase::PostInitializeProcessor:
			PostInitializeProcessor();
			break;

		case ERetargetOpPhase::PostInitializeOp:
			Op->PostInitialize(*this, SourceSkeleton, TargetSkeleton, Log);
			break;

		default:
			checkNoEntry();
		}
	}
}

void FIKRetargetProcessor::Initialize(const FRetargetInitParameters& Params)
{
	// check prerequisite assets
	if (!Params.RetargeterAsset)
	{
		Log.LogError(LOCTEXT("MissingRetargetAsset", "IK Retargeter unable to initialize because no IK Retargeter asset was supplied."));
		bIsInitialized = false;
		return;
	}
	if (!Params.SourceSkeletalMesh)
	{
		Log.LogError(LOCTEXT("MissingSourceMesh", "IK Retargeter unable to initialize. Missing source Skeletal Mesh asset."));
		bIsInitialized = false;
		return;
	}
	if (!Params.TargetSkeletalMesh)
	{
		Log.LogError(LOCTEXT("MissingTargetMesh", "IK Retargeter unable to initialize. Missing target Skeletal Mesh asset."));
		bIsInitialized = false;
		return;
	}
	
	// don't attempt reinitialization unless inputs have changed
	if (WasInitializedWithTheseAssets(Params.SourceSkeletalMesh, Params.TargetSkeletalMesh, Params.RetargeterAsset))
	{
		return;
	}

	// assumed initialization fails unless we get to the bottom of this function
	bIsInitialized = false;

	// record source asset
	RetargeterAsset = Params.RetargeterAsset;
	
	// initialize source skeleton
	SourceSkeleton.Initialize(
		Params.SourceSkeletalMesh,
		ERetargetSourceOrTarget::Source,
		RetargeterAsset,
		GetPelvisBone(ERetargetSourceOrTarget::Source, ERetargetOpsToSearch::AssetOps),
		GetPoseScale(ERetargetSourceOrTarget::Source));
	
	// initialize target skeleton
	TargetSkeleton.Initialize(
		Params.TargetSkeletalMesh,
		ERetargetSourceOrTarget::Target,
		RetargeterAsset,
		GetPelvisBone(ERetargetSourceOrTarget::Target, ERetargetOpsToSearch::AssetOps),
		GetPoseScale(ERetargetSourceOrTarget::Target));

	// copy ops from asset 
	InitialOpStackSetup(RetargeterAsset->GetRetargetOps(), Params.CustomProfile);
	
	// initialize all the retarget ops
	InitializeRetargetOps();

	// rebuild the override cache with new memory locations
	OverrideManager.DirtyProfileCache();

	// confirm for the user that the retargeter is initialized
	Log.LogInfo(FText::Format(
			LOCTEXT("SuccessfulInit", "Success! The IK Retargeter is ready to transfer animation from the source, {0} to the target, {1}"),
			FText::FromString(Params.SourceSkeletalMesh->GetName()), FText::FromString(Params.TargetSkeletalMesh->GetName())));
	
	bIsInitialized = true;
	AssetVersionInitializedWith = RetargeterAsset->GetVersion();
#if WITH_EDITOR
	RetargeterInitialized.Broadcast();
#endif
}

TArray<FTransform>& FIKRetargetProcessor::RunRetargeter(const FRetargetRunParameters& Params)
{
	SCOPE_CYCLE_COUNTER(STAT_Anim_RetargetProcessorRun);

	if (!ensure(Params.SourceGlobalPose))
	{
		Log.LogError(LOCTEXT("MissingInputPose", "IK Retargeter unable to run. Missing input source pose."));
		return TargetSkeleton.OutputGlobalPose;
	}
	
	if (!ensure(bIsInitialized))
	{
		Log.LogError(LOCTEXT("NotInitialized", "IK Retargeter unable to run. Was not successfully initialized first."));
		return TargetSkeleton.OutputGlobalPose;
	}
	
#if WITH_EDITOR
	// validate system running the retargeter has stripped all the scale out of the incoming pose
	const TArray<FTransform>& SourcePose = *Params.SourceGlobalPose;
	for (const FTransform& Transform : SourcePose)
	{
		const bool bHasNoScale = Transform.GetScale3D().Equals(FVector::OneVector);
		ensureMsgf(bHasNoScale, TEXT("Found scale values on incoming pose in retarget processor. Scale should be baked into translation and set to 1,1,1. "));
	}
#endif

	// apply the retargeting settings
	UpdateSettingsAtRuntime(Params.Profile, Params.OverrideSetsToApply, Params.Variables, Params.BoundCurveValues);

	// applying settings can cause the retargeter to require reinitialization (when enabling/disabling IK on a chain)
	// we should be able to safely reinitialize here because we are already initialized or it wouldn't pass the check() at the top
	if (!IsInitialized())
	{
		FRetargetInitParameters InitParams;
		InitParams.SourceSkeletalMesh = SourceSkeleton.SkeletalMesh;
		InitParams.TargetSkeletalMesh = TargetSkeleton.SkeletalMesh;
		InitParams.RetargeterAsset = RetargeterAsset;
		InitParams.CustomProfile = Params.Profile;
		InitParams.bSuppressWarnings = false;
		Initialize(InitParams);

		if (!IsInitialized())
		{
			// something went wrong during reinit, bail out and return the input pose
			return TargetSkeleton.OutputGlobalPose;	
		}
	}
	
	// generate the pose to start retargeting from
	GenerateBasePoses(*Params.SourceGlobalPose);
	
	// run the stack of retargeting operations
	RunRetargetOps(*Params.SourceGlobalPose, TargetSkeleton.OutputGlobalPose, Params.DeltaTime, Params.LOD);
	
	return TargetSkeleton.OutputGlobalPose;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FIKRetargetProcessor::Initialize(
    const USkeletalMesh* SourceSkeletalMesh, 
    const USkeletalMesh* TargetSkeletalMesh,
    const UIKRetargeter* InRetargeterAsset, 
    const FRetargetProfile& InCustomProfile, 
    const bool bSuppressWarnings)
{
	// forward to the new struct-based API
    FRetargetInitParameters Params;
    Params.SourceSkeletalMesh = SourceSkeletalMesh;
    Params.TargetSkeletalMesh = TargetSkeletalMesh;
    Params.RetargeterAsset = InRetargeterAsset;
    Params.CustomProfile = &InCustomProfile;
    Params.bSuppressWarnings = bSuppressWarnings;
    Initialize(Params);
}

TArray<FTransform>& FIKRetargetProcessor::RunRetargeter(
	TArray<FTransform>& InSourceGlobalPose,
	const FRetargetProfile& InProfile,
	const float InDeltaTime, 
	const int32 InLOD)
{
	// forward to the new struct-based API
	FRetargetRunParameters Params;
	Params.SourceGlobalPose = &InSourceGlobalPose;
	Params.Profile = &InProfile;
	Params.DeltaTime = InDeltaTime;
	Params.LOD = InLOD;
	return RunRetargeter(Params);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FIKRetargetProcessor::GenerateBasePoses(TArray<FTransform>& InSourceGlobalPose)
{
	// start from the retarget pose
	TargetSkeleton.OutputGlobalPose = TargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	TargetSkeleton.InputLocalPose = TargetSkeleton.RetargetPoses.GetLocalRetargetPose();
}

void FIKRetargetProcessor::RunRetargetOps(
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose,
	const double InDeltaTime,
	const int32 InLOD)
{
#if WITH_EDITOR
	// reset op timers for this frame (editor only)
	for (FInstancedStruct& OpStruct : OpStack)
	{
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		Op.Timer.StartNewFrame();
	}
#endif

	// build a stack of op calls in the correct order
	TArray<FRetargetOpCall> ExecutionStack;
	BuildExecutionStack(InLOD, ExecutionStack);

	// run the ops
	for (FRetargetOpCall& Call : ExecutionStack)
	{
		
#if WITH_EDITOR
		const double StartTime = FPlatformTime::Seconds();
#endif

		FIKRetargetOpBase* Op = Call.GetOp();

		switch (Call.Phase)
		{
		case ERetargetOpPhase::RunBeforeChildren:
			Op->RunBeforeChildren(*this, InDeltaTime, InSourceGlobalPose, OutTargetGlobalPose);
			break;

		case ERetargetOpPhase::Run:
			Op->Run(*this, InDeltaTime, InSourceGlobalPose, OutTargetGlobalPose);
			break;

		case ERetargetOpPhase::RunAfterParent:
			Op->RunAfterParent(*this, InDeltaTime, InSourceGlobalPose, OutTargetGlobalPose);
			break;

		default:
			checkNoEntry();
		}

#if WITH_EDITOR
		Op->Timer.AddTimeThisFrame(FPlatformTime::Seconds() - StartTime);
#endif
	}
}

bool FIKRetargetProcessor::ShouldExecuteOp(const FIKRetargetOpBase& InOp, int32 InLOD)
{
	if (!InOp.IsEnabled())
	{
		return false;
	}
	if (!InOp.IsInitialized())
	{
		return false;
	}

	const int32 LODThreshold = InOp.GetSettingsConst()->LODThreshold;
	if (LODThreshold != INDEX_NONE && InLOD > LODThreshold)
	{
		return false;
	}

	return true;
}

FIKRetargetOpBase* FIKRetargetProcessor::FRetargetOpCall::GetOp()
{
	if (!OpEntry)
	{
		return nullptr;
	}
	
	return OpEntry->Op;
}

FIKRetargetOpBase* FIKRetargetProcessor::FRetargetOpCall::GetParentOp()
{
	if (!OpEntry)
	{
		return nullptr;
	}
	
	if (!OpEntry->Parent)
	{
		return nullptr;
	}

	return OpEntry->Parent->Op;
}

void FIKRetargetProcessor::BuildExecutionStack(int32 InLOD, TArray<FRetargetOpCall>& OutStack)
{
	OutStack.Reset();

	// 1st pass update bExecute
	for (FOpEntry& Entry : OpEntries)
	{
		Entry.bExecute = ShouldExecuteOp(*Entry.Op, InLOD);
	}

	// 2nd pass, generate execution order and store in stack
	for (FOpEntry& OpEntry : OpEntries)
	{
		if (!OpEntry.bExecute)
		{
			// op not running this frame
			// NOTE: this implies it's children will not run either (by design)
			continue;
		}

		const bool bIsChild = OpEntry.Parent != nullptr;
		const bool bIsParent = OpEntry.Op->CanHaveChildOps();

		if (bIsChild)
		{
			continue; // children ops managed by parent
		}

		// neither a child or a parent, ie a simple op, just run Main once.
		if (!bIsParent && !bIsChild)
		{
			OutStack.Add({ ERetargetOpPhase::Run, &OpEntry });
			continue;
		}

		// Parent with children execution order:
		// NOTE: parents run all steps even if they don't have children
		// but children don't run at all if the parent is disabled.
		// 1. Parent->RunBeforeChildren
		// 2. Children->Run
		// 3. Parent->Run
		// 4. Children->RunAfterParent

		// 1. Parent->RunBeforeChildren()
		OutStack.Emplace(ERetargetOpPhase::RunBeforeChildren, &OpEntry);

		// 2. Children->Run()
		for (FOpEntry* ChildEntry : OpEntry.Children)
		{
			if (ChildEntry->bExecute)
			{
				OutStack.Emplace(ERetargetOpPhase::Run, ChildEntry);
			}
		}

		// 3. Parent->Run()
		OutStack.Emplace(ERetargetOpPhase::Run, &OpEntry);

		// 4. Children->RunAfterParent()
		for (FOpEntry* ChildEntry : OpEntry.Children)
		{
			if (ChildEntry->bExecute)
			{
				OutStack.Emplace(ERetargetOpPhase::RunAfterParent, ChildEntry);
			}
		}
	}
}

void FIKRetargetProcessor::BuildInitializationStack(TArray<FRetargetOpCall>& OutStack)
{
	OutStack.Reset();

	// NOTE: we initialize all ops regardless of if they are disabled
	// this is because ops can be toggled on/off at runtime

	// 1st pass add Initialize and InitializeBeforeChildren calls
	for (FOpEntry& OpEntry : OpEntries)
	{
		const bool bIsChild = OpEntry.Parent != nullptr;
		const bool bIsParent = OpEntry.Op->CanHaveChildOps();

		if (bIsChild)
		{
			continue; // children ops managed by parent
		}

		// neither a child or a parent, ie a simple op, just run Initialize.
		if (!bIsParent && !bIsChild)
		{
			OutStack.Add({ ERetargetOpPhase::Initialize, &OpEntry });
			continue;
		}

		if (bIsParent)
		{
			// Parent->InitializeBeforeChildren()
			// Child0...N->Initialize()
			// Parent->Initialize()
			OutStack.Emplace(ERetargetOpPhase::InitializeBeforeChildren, &OpEntry);
			for (FOpEntry* Child : OpEntry.Children)
			{
				OutStack.Add({ ERetargetOpPhase::Initialize, Child });
			}
			OutStack.Add({ ERetargetOpPhase::Initialize, &OpEntry });
		}
	}

	// 2nd pass, give processor a chance to gather data from initialized ops
	OutStack.Add({ERetargetOpPhase::PostInitializeProcessor, nullptr});

	// 3nd pass, add PostInitialize() for all ops
	for (FOpEntry& OpEntry : OpEntries)
	{
		OutStack.Add({ ERetargetOpPhase::PostInitializeOp, &OpEntry });
	}
}

void FIKRetargetProcessor::InitializeOpEntries()
{
	auto FindOpEntry = [this](const FIKRetargetOpBase* InOpToFind) -> FOpEntry*
	{
		for (FOpEntry& Entry : OpEntries)
		{
			if (Entry.Op == InOpToFind)
			{
				return &Entry;
			}
		}
		return nullptr;
	};
	
	OpEntries.Reset(OpStack.Num());

	// create entries for each op in the stack
	for (int32 Index = 0; Index < OpStack.Num(); ++Index)
	{
		FIKRetargetOpBase& Op = OpStack[Index].GetMutable<FIKRetargetOpBase>();
		
		FOpEntry& Entry = OpEntries.AddDefaulted_GetRef();
		Entry.Op = &Op;
		Entry.StackIndex = Index;
		Entry.bExecute = true; // default to true, set during runtime
	}

	// wire up parent/children on the entries
	for (FOpEntry& Entry : OpEntries)
	{
		const FName ParentName = Entry.Op->GetParentOpName();
		if (ParentName.IsNone())
		{
			continue; // no parent
		}

		FIKRetargetOpBase* ParentOp = GetRetargetOpByName(ParentName);
		if (!ensure(ParentOp))
		{
			continue; // should not happen, ill formed op stack referencing missing parent
		}
		
		FOpEntry* ParentEntry = FindOpEntry(ParentOp);
		if (!ensure(ParentEntry))
		{
			continue; // should not happen, entries out of sync with op stack
		}

		Entry.Parent = ParentEntry;
		ParentEntry->Children.Add(&Entry);
	}

	// keep children ordered by stack index
	for (FOpEntry& Entry : OpEntries)
	{
		if (Entry.Children.Num() > 1)
		{
			Entry.Children.Sort([](const FOpEntry& A, const FOpEntry& B)
			{
				return A.StackIndex < B.StackIndex;
			});
		}
	}
}

TArray<const FIKRetargetOpBase*> FIKRetargetProcessor::GetRetargetOpsByType(const UScriptStruct* OpType) const
{
	TArray<const FIKRetargetOpBase*> Ops;

	for (const FInstancedStruct& OpStruct : OpStack)
	{
		if (OpStruct.GetScriptStruct()->IsChildOf(OpType))
		{
			const FIKRetargetOpBase* Op = OpStruct.GetPtr<FIKRetargetOpBase>();
			Ops.Add(Op);
		}
	}

	return MoveTemp(Ops);
}

FIKRetargetOpBase* FIKRetargetProcessor::GetRetargetOpByName(const FName InOpName)
{
	for (FInstancedStruct& OpStruct : OpStack)
	{
		FIKRetargetOpBase* Op = OpStruct.GetMutablePtr<FIKRetargetOpBase>();
		if (Op->GetName() == InOpName)
		{
			return Op;
		}
	}
	return nullptr;
}

void FIKRetargetProcessor::OnPlaybackReset()
{
	for (FInstancedStruct& OpStruct : OpStack)
	{
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		Op.OnPlaybackReset();
	}
}

void FIKRetargetProcessor::OnAnimGraphPreUpdateMainThread(
	USkeletalMeshComponent& SourceMeshComponent,
	USkeletalMeshComponent& TargetMeshComponent)
{
	for (FInstancedStruct& OpStruct : OpStack)
	{
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		Op.AnimGraphPreUpdateMainThread(SourceMeshComponent, TargetMeshComponent);
	}
}

void FIKRetargetProcessor::OnAnimGraphEvaluateAnyThread(FPoseContext& Output)
{
	// work out which (if any) op is the first enabled curve processing op, so that in this case, we know to copy the inputs from the source anim instance
	// after that, we take input from the curves output by the previous node
	bool bFirstCurveOp = true;
	for (FInstancedStruct& OpStruct : OpStack)
	{
		const UStruct* StructType = OpStruct.GetScriptStruct();
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		bool bHasCurveProcessing = false;
		if (StructType && StructType->IsChildOf(FIKRetargetCurvesOpBase::StaticStruct()))
		{
			FIKRetargetCurvesOpBase& CurveOp = OpStruct.GetMutable<FIKRetargetCurvesOpBase>();
			CurveOp.SetTakeInputCurvesFromSourceAnimInstance(bFirstCurveOp);
			bHasCurveProcessing = CurveOp.IsEnabled() && CurveOp.HasCurveProcessing();
		}
		Op.AnimGraphEvaluateAnyThread(Output);
		if (bHasCurveProcessing)
		{
			bFirstCurveOp = false;
		}
	}
}

bool FIKRetargetProcessor::WasInitializedWithTheseAssets(
	const USkeletalMesh* InSourceMesh,
	const USkeletalMesh* InTargetMesh,
	const UIKRetargeter* InRetargetAsset) const
{
	// not initialized at all
	if (!IsInitialized())
	{
		return false;
	}

	// check that the retarget asset is the same as what we initialized with
	const bool bSameAsset = InRetargetAsset == RetargeterAsset;
	const bool bSameVersion = AssetVersionInitializedWith == InRetargetAsset->GetVersion();
	if (!(bSameAsset && bSameVersion))
	{
		// asset has been modified in a way that requires reinitialization
		return false;
	}

	// check that both the source and target skeletal meshes are the same as what we initialized with
	const FRetargetSkeleton& SourceRetargetSkeleton = GetSkeleton(ERetargetSourceOrTarget::Source);
	const FRetargetSkeleton& TargetRetargetSkeleton = GetSkeleton(ERetargetSourceOrTarget::Target);
	const bool bSourceMatches = InSourceMesh == SourceRetargetSkeleton.SkeletalMesh;
	const bool bTargetMatches = InTargetMesh == TargetRetargetSkeleton.SkeletalMesh;
	if (!(bSourceMatches && bTargetMatches))
	{
		// skeletal mesh swapped out
		return false;
	}

	// check that the number of bones are the same as what we initialized with
	const bool bSourceHasSameNumberOfBones = InSourceMesh->GetRefSkeleton().GetNum() == SourceRetargetSkeleton.BoneNames.Num();
	const bool bTargetHasSameNumberOfBones = InTargetMesh->GetRefSkeleton().GetNum() == TargetRetargetSkeleton.BoneNames.Num();
	if (!(bSourceHasSameNumberOfBones && bTargetHasSameNumberOfBones))
	{
		// skeletal mesh modified
		return false;
	}
	
	return true;
}

#if WITH_EDITOR

bool FIKRetargetProcessor::IsBoneMapped(
	const FName BoneName,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	ensure(bIsInitialized);
	
	// NOTE: cannot use Skeleton.IsBoneRetargeted because that only exists
	// on the target skeleton and this function needs to work for either skeleton

	if (BoneName == NAME_None)
	{
		return false;
	}

	// special case: if the bone is a pelvis, check if it's mapped to a pelvis on the other side
	const FName PelvisBoneName = GetPelvisBone(SourceOrTarget, ERetargetOpsToSearch::AssetOps);
	if (BoneName == PelvisBoneName)
	{
		const ERetargetSourceOrTarget OtherSide = SourceOrTarget == ERetargetSourceOrTarget::Source ? ERetargetSourceOrTarget::Target : ERetargetSourceOrTarget::Source;
		const FName OtherPelvisBoneName = GetPelvisBone(OtherSide, ERetargetOpsToSearch::AssetOps);
		if (OtherPelvisBoneName != NAME_None)
		{
			return true;
		}
	}
	
	// find chains on both sides
	FName ChainWithBone;
	FName MappedChain;
	const bool bFoundMapping = GetFirstChainMappingForBone(BoneName, SourceOrTarget, ChainWithBone, MappedChain);

	return bFoundMapping;
}

int32 FIKRetargetProcessor::GetBoneIndexFromName(
	const FName BoneName,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	return GetSkeleton(SourceOrTarget).FindBoneIndexByName(BoneName);
}

const FRetargeterBoneChains& FIKRetargetProcessor::GetBoneChains()
{
	ensure(bIsInitialized);
	return AllBoneChains;
}

FName FIKRetargetProcessor::GetFirstChainForBone(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget) const
{
	const int32 BoneIndex = GetBoneIndexFromName(BoneName, SourceOrTarget);
	const TSet<FName> ChainsThatContainBone = AllBoneChains.GetChainsThatContainBone(BoneIndex, SourceOrTarget);
	return !ChainsThatContainBone.IsEmpty() ? *ChainsThatContainBone.begin() : NAME_None;
}

TSet<FName> FIKRetargetProcessor::GetAllChainNamesForBone(
	const FName BoneName,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	const int32 BoneIndex = GetBoneIndexFromName(BoneName, SourceOrTarget);
	return AllBoneChains.GetChainsThatContainBone(BoneIndex, SourceOrTarget);
}

bool FIKRetargetProcessor::GetFirstChainMappingForBone(
	const FName BoneName,
	const ERetargetSourceOrTarget SourceOrTarget,
	FName& OutChainContainingBone,
	FName& OutMappedChain) const
{
	if (!ensure(bIsInitialized))
	{
		return false;
	}
	
	OutChainContainingBone = NAME_None;
	OutMappedChain = NAME_None;

	// bone must exist
	const int32 BoneIndex = GetBoneIndexFromName(BoneName, SourceOrTarget);
	if (BoneIndex == INDEX_NONE)
	{
		return false;
	}
	
	// bone must be in a chain that is mapped to alternate chain (source-to-target or vice versa)
	const TSet<FName> ChainsThatContainBone = AllBoneChains.GetChainsThatContainBone(BoneIndex, SourceOrTarget);
	for (const FName ChainContainingBone : ChainsThatContainBone)
	{
		const FName MappedChain = GetFirstChainMappedToChain(ChainContainingBone, SourceOrTarget);
		if (MappedChain != NAME_None)
		{
			OutChainContainingBone = ChainContainingBone;
			OutMappedChain = MappedChain;
			return true;
		}
	}
	
	return false;
}

FTransform FIKRetargetProcessor::GetGlobalRetargetPoseAtParam(
	const FName InChainName,
	const float Param,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	if (const FResolvedBoneChain* Chain = GetBoneChains().GetResolvedBoneChainByName(InChainName, SourceOrTarget))
	{
		return Chain->GetTransformAtChainParam(Chain->RefPoseGlobalTransforms, Param);
	}

	checkNoEntry();
	return FTransform::Identity;
}

FTransform FIKRetargetProcessor::GetRetargetPoseBoneTransform(
	const FName InBoneName,
	const ERetargetSourceOrTarget SourceOrTarget,
	ERetargetBoneSpace BoneSpace) const
{
	const int32 BoneIndex = GetBoneIndexFromName(InBoneName, SourceOrTarget);
	if (!ensure(BoneIndex != INDEX_NONE))
	{
		return FTransform::Identity;
	}

	const FRetargetSkeleton& Skeleton = GetSkeleton(SourceOrTarget);
	const bool bGlobal = BoneSpace == ERetargetBoneSpace::Global;
	const TArray<FTransform>& Pose = bGlobal ? Skeleton.RetargetPoses.GetGlobalRetargetPose() : Skeleton.RetargetPoses.GetLocalRetargetPose();
	return Pose[BoneIndex];
}

float FIKRetargetProcessor::GetParamOfBoneInChain(
	const FName InBoneName,
	const ERetargetSourceOrTarget SourceOrTarget,
	const FName InChainName) const
{
	FName ChainName = InChainName;
	if (ChainName == NAME_None)
	{
		ChainName = GetFirstChainForBone(InBoneName, SourceOrTarget);
	}
	
	const FResolvedBoneChain* Chain = GetBoneChains().GetResolvedBoneChainByName(ChainName, SourceOrTarget);
	if (!Chain)
	{
		return INDEX_NONE;
	}

	const int32 BoneIndexInSkeleton = GetBoneIndexFromName(InBoneName, SourceOrTarget);
	const int32 BoneIndexInChain =  Chain->BoneIndices.Find(BoneIndexInSkeleton);
	if (Chain->Params.IsValidIndex(BoneIndexInChain))
	{
		return Chain->Params[BoneIndexInChain];	
	}

	ensureMsgf(false, TEXT("Should never fail to find a chain index if bone exists in chain."));
	return INDEX_NONE;
}

FName FIKRetargetProcessor::GetClosestBoneToParam(
	const FName InChainName, 
	const float InParam,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	const FResolvedBoneChain* Chain = GetBoneChains().GetResolvedBoneChainByName(InChainName, SourceOrTarget);
	if (!ensure(Chain != nullptr))
	{
		return NAME_None;
	}
	
	const FRetargetSkeleton& Skeleton = GetSkeleton(SourceOrTarget);
	if (Chain->BoneIndices.Num() == 1 || InParam < 0.0f)
	{
		return Skeleton.BoneNames[Chain->BoneIndices[0]];
	}

	float ClosestDistance = TNumericLimits<float>::Max();
	int32 ChainIndexOfClosestBone = 0;
	for (int32 ChainIndex = 0; ChainIndex < Chain->Params.Num(); ++ChainIndex)
	{
		const float DistanceToParam = FMath::Abs(Chain->Params[ChainIndex] - InParam);
		if (DistanceToParam <= ClosestDistance)
		{
			ChainIndexOfClosestBone = ChainIndex;
			ClosestDistance = DistanceToParam;
		}
	}

	return Skeleton.BoneNames[Chain->BoneIndices[ChainIndexOfClosestBone]];
}

FName FIKRetargetProcessor::GetFirstChainMappedToChain(
	const FName InChainName,
	const ERetargetSourceOrTarget InSourceOrTarget) const
{
	if (!ensure(bIsInitialized))
	{
		return NAME_None;
	}

	for (const FInstancedStruct& OpStruct : OpStack)
	{
		FIKRetargetOpBase* Op = const_cast<FIKRetargetOpBase*>(OpStruct.GetPtr<FIKRetargetOpBase>());
		const FRetargetChainMapping* ChainMapping = Op->GetChainMapping();
		if (!ChainMapping)
		{
			continue;
		}
		const FName MappedChain = ChainMapping->GetChainMappedTo(InChainName, InSourceOrTarget);
		if (MappedChain != NAME_None)
		{
			return MappedChain;
		}
	}

	return NAME_None;
}

void FIKRetargetProcessor::DebugDrawAllOps(
	FPrimitiveDrawInterface* InPDI,
	const FIKRetargetDebugDrawState& EditorState) const
{
	const UIKRetargeter* RetargetAsset = GetRetargetAsset();
	if (!ensure(RetargetAsset))
	{
		return;
	}

	// construct the target preview transform
	const double TargetScale = RetargetAsset->TargetMeshScale;
	const FTransform TargetTransform = FTransform(FQuat::Identity, RetargetAsset->TargetMeshOffset, FVector(TargetScale));
	
	// NOTE: we cannot directly use the source mesh component transform because it has translation and scaling applied to preview the
	// custom pivot source scaling results. The source pose bone transforms that the ops themselves have access to use scaled translations,
	// but have scale at 1,1,1. We avoid actual bone scaling in the retargeter entirely, but if we visualized the source mesh in this manner,
	// it would appear like the skin is stretched out and thin (when scaled up) and squished up and fat (when scaled down); hence the use of scaling.
	//
	// So to avoid double transforms in debug drawing, the ops should use the bone translations passed into them, and this source transform with NO scaling.
	const FTransform SourceTransform = FTransform(FQuat::Identity, RetargetAsset->SourceMeshOffset, FVector::OneVector);

	// debug draw each op in the stack
	for (const FOpEntry& OpEntry : OpEntries)
	{
		const FIKRetargetOpBase* Op = OpEntry.Op;

		// filter disabled or uninitialized ops
		if (!Op->IsEnabled() || !Op->IsInitialized() || !Op->GetSettingsConst()->bDebugDraw)
		{
			continue;
		}

		// skip if parent is disabled
		if (OpEntry.Parent != nullptr && !OpEntry.Parent->Op->IsEnabled())
		{
			continue;
		}

		Op->DebugDraw(InPDI, SourceTransform, TargetTransform, TargetScale, EditorState);
	}
}

void FIKRetargetProcessor::UpdateOpsFromAnimSequence(UAnimSequence* Sequence, float FrameTime)
{
	for (FInstancedStruct& OpStruct : OpStack)
	{
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		Op.UpdateFromAnimSequence(Sequence, FrameTime);
	}
}

#endif

const FRetargetSkeleton& FIKRetargetProcessor::GetSkeleton(ERetargetSourceOrTarget SourceOrTarget) const
{
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceSkeleton : TargetSkeleton;
}

FTargetSkeleton& FIKRetargetProcessor::GetTargetSkeleton()
{
	return TargetSkeleton;
}

const FTargetSkeleton& FIKRetargetProcessor::GetTargetSkeleton() const
{
	return TargetSkeleton;
}

bool FIKRetargetProcessor::IsInitialized() const
{
	return bIsInitialized;
}

void FIKRetargetProcessor::SetNeedsInitialized()
{
	bIsInitialized = false;

	if (RetargeterAsset)
	{
		RetargeterAsset->IncrementVersion(); // triggers re-init
	}
}

void FIKRetargetProcessor::UpdateSettingsAtRuntime(
	const FRetargetProfile* InProfile,
	const TArrayView<const FName> InOverrideSetsToApply,
	FInstancedPropertyBag* InVariables,
	TMap<FName, float>* InBoundCurveValues)
{
	// apply op settings from the profile
	if (InProfile)
	{
		for (FInstancedStruct& OpStruct : OpStack)
		{
			InProfile->ApplyOpProfilesToOpStruct(OpStruct, ECopyOpSettingsContext::Runtime);
		}
	}

	// apply stored property overrides
	OverrideManager.ApplyPropertyOverridesToOps(*this, InOverrideSetsToApply, InVariables, InBoundCurveValues);

	// apply flag to force IK off
	if (InProfile)
	{
		bIKForcedOff = InProfile->bForceAllIKOff;
	}

	// update retarget poses
	{
		FName SourcePoseToApply = NAME_None;
		FName TargetPoseToApply = NAME_None;

		// allow profile to override retarget poses
		if (InProfile)
		{
			SourcePoseToApply = InProfile->SourceRetargetPoseName;
			TargetPoseToApply = InProfile->TargetRetargetPoseName;
		}

		// allow retarget pose op to override
		if (const FIKRetargetPoseOp* RetargetPoseOp = GetFirstRetargetOpOfType<FIKRetargetPoseOp>())
		{
			const FIKRetargetPoseOpSettings* Settings = reinterpret_cast<const FIKRetargetPoseOpSettings*>(RetargetPoseOp->GetSettingsConst());
			if (Settings->bOverrideSourcePose && !Settings->SourcePoseToUse.IsNone())
			{
				SourcePoseToApply = Settings->SourcePoseToUse;
			}
			if (Settings->bOverrideTargetPose && !Settings->TargetPoseToUse.IsNone())
			{
				TargetPoseToApply = Settings->TargetPoseToUse;
			}
		}
	
		// apply retarget poses specified in the profile
		// NOTE: must be done AFTER applying op settings because it uses the SourceScaleFactor in the ScaleSourceOp settings
		UpdateRetargetPoseAtRuntime(SourcePoseToApply, ERetargetSourceOrTarget::Source);
		UpdateRetargetPoseAtRuntime(TargetPoseToApply, ERetargetSourceOrTarget::Target);
	}	
}

void FIKRetargetProcessor::UpdateRetargetPoseAtRuntime(
	const FName RetargetPoseToUseName,
	ERetargetSourceOrTarget SourceOrTarget)
{
	auto LoadRetargetPoseFromAsset = [this, RetargetPoseToUseName, SourceOrTarget](const FIKRetargetPose* PoseToLoad)
	{
		// load the pose and resolve it onto the current skeleton
		const FName PelvisBoneName = GetPelvisBone(SourceOrTarget, ERetargetOpsToSearch::ProcessorOps);
		const bool bIsSource = SourceOrTarget == ERetargetSourceOrTarget::Source;
		const FRetargetPoseScaleWithPivot PoseScale = GetPoseScale(SourceOrTarget);
		FRetargetSkeleton& RetargetSkeleton = bIsSource ? SourceSkeleton : TargetSkeleton;
		FResolvedRetargetPose& ResolvedPose = RetargetSkeleton.RetargetPoses.AddOrUpdateRetargetPose(
			RetargetSkeleton,
			RetargetPoseToUseName,
			PoseToLoad,
			PelvisBoneName,
			PoseScale);

		// set this as the current pose to use
		RetargetSkeleton.RetargetPoses.CurrentRetargetPoseName = RetargetPoseToUseName;

		// re-load the updated pose into the bone chains
		AllBoneChains.UpdatePoseFromSkeleton(RetargetSkeleton, SourceOrTarget);
	
		// re-initialize the op stack (ops sometimes cache retarget poses)
		InitializeRetargetOps();
		
		return &ResolvedPose;
	};

	// verify that retarget pose exists in the retarget asset
	const FIKRetargetPose* UnresolvedPoseFromAsset = RetargeterAsset->GetRetargetPoseByName(SourceOrTarget, RetargetPoseToUseName);
	if (!UnresolvedPoseFromAsset)
	{
		return; // retarget pose not found
	}

	// load pose out of asset if it hasn't been loaded yet
	const bool bIsSource = SourceOrTarget == ERetargetSourceOrTarget::Source;
	FRetargetSkeleton& RetargetSkeleton = bIsSource ? SourceSkeleton : TargetSkeleton;
	const FResolvedRetargetPose* ResolvedPoseToUse = RetargetSkeleton.RetargetPoses.FindRetargetPoseByName(RetargetPoseToUseName);
	if (!ResolvedPoseToUse)
	{
		// trying to switch to a retarget pose that hasn't been loaded yet, so load it
		ResolvedPoseToUse = LoadRetargetPoseFromAsset(UnresolvedPoseFromAsset);
	}
	
	// verify if pose needs rebuilding
	const FName CurrentRetargetPoseName = RetargetSkeleton.RetargetPoses.CurrentRetargetPoseName;
	const FResolvedRetargetPose& CurrentResolvedPose = RetargetSkeleton.RetargetPoses.FindOrAddRetargetPose(CurrentRetargetPoseName);
	const bool bSameRetargetPose = CurrentResolvedPose.Name == ResolvedPoseToUse->Name;
	const bool bSameVersion = CurrentResolvedPose.Version == UnresolvedPoseFromAsset->GetVersion();
	const bool bSameGlobalScale = GetPoseScale(SourceOrTarget) == CurrentResolvedPose.PoseScale;
	if (bSameRetargetPose && bSameVersion && bSameGlobalScale)
	{
		return; // retarget pose has not changed since it was initialized
	}
	
	// reload/update the retarget pose
	LoadRetargetPoseFromAsset(UnresolvedPoseFromAsset);
}

FName FIKRetargetProcessor::GetPelvisBone(ERetargetSourceOrTarget SourceOrTarget, ERetargetOpsToSearch InOpsToSearch) const
{
	// first try get it from a pelvis of
	const FIKRetargetPelvisMotionOp* PelvisMotionOp = GetFirstRetargetOpOfType<FIKRetargetPelvisMotionOp>(InOpsToSearch);
	if (PelvisMotionOp)
	{
		FName PelvisNameFromOp = PelvisMotionOp->GetPelvisBoneName(SourceOrTarget);
		if (PelvisNameFromOp != NAME_None)
		{
			return PelvisNameFromOp;
		}
	}

	const UIKRetargeter* Asset = GetRetargetAsset();
	if (!Asset)
	{
		return NAME_None;
	}

	// fallback to going directly to the IK Rig
	const UIKRigDefinition* IKRigAsset = Asset->GetIKRig(SourceOrTarget);
	if (!IKRigAsset)
	{
		return NAME_None;	
	}

	return IKRigAsset->GetPelvis();
}

void FIKRetargetProcessor::ApplySourceScaleToPose(TArray<FTransform>& InOutSourceGlobalPose) const
{
	const FIKRetargetScaleSourceOp* ScaleSourceOp = GetFirstRetargetOpOfType<FIKRetargetScaleSourceOp>();
	if (!ScaleSourceOp || !ScaleSourceOp->IsEnabled())
	{
		CurrentSourceScale.Factor = 1.0;
		CurrentSourceScale.Pivot = FVector::ZeroVector;
		return;
	}
	
	auto GetScalePivot = [this, &InOutSourceGlobalPose, ScaleSourceOp]() -> FVector
		{
			switch (ScaleSourceOp->Settings.ScalePivot)
			{
			case EScaleSourcePivot::ComponentOrigin:
				{
					return FVector::ZeroVector;
				}
			case EScaleSourcePivot::Bone:
				{
					const int32 ScalePivotBoneIndex = ScaleSourceOp->GetScalePivotBoneIndex(SourceSkeleton);
					if (!InOutSourceGlobalPose.IsValidIndex(ScalePivotBoneIndex))
					{
						return FVector::ZeroVector;
					}

					// optionally project to floor
					const FVector Projection = ScaleSourceOp->Settings.bProjectScalePivotToFloor ? FVector(1,1,0) : FVector::OneVector;
					return InOutSourceGlobalPose[ScalePivotBoneIndex].GetTranslation() * Projection;
				}
			default:
				checkNoEntry();
				return FVector::ZeroVector;
			}
		};

	// record latest scale pivot and factor
	CurrentSourceScale.Factor = ScaleSourceOp->Settings.SourceScaleFactor;
	CurrentSourceScale.Pivot = GetScalePivot();
	
	// scale the input pose
	CurrentSourceScale.ScalePose(InOutSourceGlobalPose);
}

double FIKRetargetProcessor::GetSourceScaleFactor() const
{
	// first try get it from the processor
	const FIKRetargetScaleSourceOp* ScaleSourceOp = GetFirstRetargetOpOfType<FIKRetargetScaleSourceOp>(ERetargetOpsToSearch::ProcessorOps);
	// if the processor doesn't have the op, check the asset
	ScaleSourceOp = ScaleSourceOp ? ScaleSourceOp : GetFirstRetargetOpOfType<FIKRetargetScaleSourceOp>(ERetargetOpsToSearch::AssetOps);
	if (ScaleSourceOp && ScaleSourceOp->IsEnabled())
	{
		return ScaleSourceOp->Settings.SourceScaleFactor;
	}

	// if no source scale op is present, then we don't do any scaling
	return 1.0;
}

const FRetargetPoseScaleWithPivot& FIKRetargetProcessor::GetPoseScale(const ERetargetSourceOrTarget InSourceOrTarget) const
{
	static FRetargetPoseScaleWithPivot TargetPoseScale;
	return InSourceOrTarget == ERetargetSourceOrTarget::Source ? CurrentSourceScale : TargetPoseScale;
}

#undef LOCTEXT_NAMESPACE

