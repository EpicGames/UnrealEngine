// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargetProfile.h"

#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetOps.h"

// NOTE: all these op includes can go once we no longer support loading deprecated profile settings
#include "Retargeter/RetargetOps/AlignPoleVectorOp.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"
#include "Retargeter/RetargetOps/IKChainsOp.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"
#include "Retargeter/RetargetOps/SpeedPlantingOp.h"
#include "Retargeter/RetargetOps/StrideWarpingOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetProfile)

FRetargetOpProfile::FRetargetOpProfile(
	const FName InOpName,
	const UScriptStruct* InSettingsType,
	const FIKRetargetOpSettingsBase* InOpSettings)
{
	OpToApplySettingsTo = InOpName;
	SettingsToApply.InitializeAs(InSettingsType, reinterpret_cast<const uint8*>(InOpSettings));
}

void FRetargetOpProfile::CopyFromOtherOpProfile(const FRetargetOpProfile& OtherOpProfile)
{
	SettingsToApply = OtherOpProfile.SettingsToApply;
}

bool FRetargetOpProfile::CopySettingsToOp(FInstancedStruct& InOutOpStruct, ECopyOpSettingsContext InApplyContext) const
{
	FIKRetargetOpBase* Op = InOutOpStruct.GetMutablePtr<FIKRetargetOpBase>();
	if (!ensure(Op))
	{
		return false; // should never get here with an invalid op struct
	}
	FIKRetargetOpSettingsBase* OpSettings = Op->GetSettings();
	if (!ensure(OpSettings))
	{
		return false; // no settings to modify
	}
	const UScriptStruct* OpSettingsType = Op->GetSettingsType();
	if (!ensure(OpSettingsType))
	{
		return false; // unknown settings type
	}

	if (!SettingsToApply.IsValid() || OpSettingsType != SettingsToApply.GetScriptStruct())
	{
		return false; // invalid settings data or incorrect type
	}
	
	// apply wholesale op settings struct overrides
	const FIKRetargetOpSettingsBase& OpSettingsFromProfile = SettingsToApply.Get<FIKRetargetOpSettingsBase>();
	switch (InApplyContext)
	{
	case ECopyOpSettingsContext::Runtime:
		// apply the profile settings to an op in the stack through the CopySettingsAtRuntime interface
		// this gives each op a chance to digest the new settings in a way that won't require reinitialization
		OpSettings->CopySettingsAtRuntime(&OpSettingsFromProfile);
		break;
		
	case ECopyOpSettingsContext::PreInitialize:
		// wholesale copy all settings properties into the op struct
		// this is only safe to do before initialization
		Op->CopySettingsRaw(&OpSettingsFromProfile, {} /* properties to ignore*/);
		break;

	default:
		checkNoEntry();
	}
	
	return true;
}

UIKRetargetOpControllerBase* FRetargetOpProfile::CreateControllerIfNeeded(UObject* Outer)
{
	if (!ensure(SettingsToApply.IsValid()))
	{
		return nullptr;
	}
	
	if (!Controller.IsValid())
	{
		FIKRetargetOpSettingsBase* OpSettings = SettingsToApply.GetMutablePtr<FIKRetargetOpSettingsBase>();
		const UClass* ClassType = OpSettings->GetControllerType();
		if (ensure(ClassType && ClassType->IsChildOf(UIKRetargetOpControllerBase::StaticClass())))
		{
			Controller = TStrongObjectPtr(NewObject<UIKRetargetOpControllerBase>(Outer, ClassType));
			Controller->OpSettingsToControl = OpSettings;
		}
	}
	
	return Controller.Get();
}

void FRetargetProfile::FillProfileWithAssetSettings(const UIKRetargeter* InAsset)
{
	if (!InAsset)
	{
		return;
	}
	
	// profile can apply retarget poses
	bApplyTargetRetargetPose = true;
	TargetRetargetPoseName = InAsset->GetCurrentRetargetPoseName(ERetargetSourceOrTarget::Target);
	bApplySourceRetargetPose = true;
	SourceRetargetPoseName = InAsset->GetCurrentRetargetPoseName(ERetargetSourceOrTarget::Source);
	
	// profile can apply op settings
	RetargetOpProfiles.Reset();
	TArray<FInstancedStruct> AssetOps = InAsset->GetRetargetOps();
	for (int32 OpIndex=0; OpIndex<AssetOps.Num(); ++OpIndex)
	{
		FIKRetargetOpBase* Op = AssetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
		RetargetOpProfiles.Emplace(Op->GetName(), Op->GetSettingsType(), Op->GetSettings());
	}

	// now override any settings in the asset's current profile
	if (const FRetargetProfile* ProfileToUse = InAsset->GetCurrentProfile())
	{
		MergeWithOtherProfile(*ProfileToUse);
	}
}

void FRetargetProfile::MergeWithOtherProfile(const FRetargetProfile& OtherProfile)
{
	// merge retarget pose from other profile
	if (OtherProfile.bApplyTargetRetargetPose)
	{
		TargetRetargetPoseName = OtherProfile.TargetRetargetPoseName;
	}
	if (OtherProfile.bApplySourceRetargetPose)
	{
		SourceRetargetPoseName = OtherProfile.SourceRetargetPoseName;
	}

	// merge op settings from other profile
	for (const FRetargetOpProfile& OtherOpProfile : OtherProfile.RetargetOpProfiles)
	{
		if (FRetargetOpProfile* MatchingOpProfile = FindMatchingOpProfile(OtherOpProfile))
		{
			// found an op profile for the same op (same index and type) so overwrite it
			MatchingOpProfile->CopyFromOtherOpProfile(OtherOpProfile);
		}
		else
		{
			// no matching op profile, so just add it
			RetargetOpProfiles.Emplace(OtherOpProfile);
		}
	}

	MergeDeprecatedProperties(OtherProfile);
}

FRetargetOpProfile* FRetargetProfile::FindMatchingOpProfile(const FRetargetOpProfile& OtherOpProfile)
{
	for (FRetargetOpProfile& OpProfile : RetargetOpProfiles)
	{
		if (OpProfile.OpToApplySettingsTo == OtherOpProfile.OpToApplySettingsTo && OpProfile.StaticStruct() == OtherOpProfile.StaticStruct())
		{
			return &OpProfile;
		}
	}

	return nullptr;
}

bool FRetargetProfile::ApplyOpProfilesToOpStruct(
	FInstancedStruct& InOutOpStruct,
	const ECopyOpSettingsContext InCopyContext) const
{
	const FIKRetargetOpBase* OpPtr = InOutOpStruct.GetPtr<FIKRetargetOpBase>();
	if (!OpPtr)
	{
		return false;
	}
	const FName OpName = OpPtr->GetName();
	
	bool bOpSettingsApplied = true;
	for (const FRetargetOpProfile& OpProfile : RetargetOpProfiles)
	{
		if (OpProfile.OpToApplySettingsTo == NAME_None || OpProfile.OpToApplySettingsTo == OpName)
		{
			bOpSettingsApplied &= OpProfile.CopySettingsToOp(InOutOpStruct, InCopyContext);
		}
	}
	return bOpSettingsApplied;
}

FRetargetOpProfile* FRetargetProfile::GetOpProfileByName(const FName InOpName)
{
	for (FRetargetOpProfile& OpProfile : RetargetOpProfiles)
	{
		if (OpProfile.OpToApplySettingsTo == InOpName)
		{
			return &OpProfile;
		}
	}
	return nullptr;
}

FRetargetProfile URetargetProfileLibrary::CopyRetargetProfileFromRetargetAsset(const UIKRetargeter* InRetargetAsset)
{
	FRetargetProfile RetargetProfile;
	RetargetProfile.FillProfileWithAssetSettings(InRetargetAsset);
	return MoveTemp(RetargetProfile);
}

UIKRetargetOpControllerBase* URetargetProfileLibrary::GetOpControllerFromRetargetProfile(
	FRetargetProfile& InRetargetProfile,
	const FName InRetargetOpName)
{
	FRetargetOpProfile* OpProfile = InRetargetProfile.GetOpProfileByName(InRetargetOpName);
	if (!OpProfile)
	{
		return nullptr;
	}
	
	return OpProfile->CreateControllerIfNeeded((UObject*)GetTransientPackage());
}

void FRetargetProfile::MergeDeprecatedProperties(const FRetargetProfile& OtherProfile)
{
	//
	// BEGIN merge deprecated properties from other profile
	//
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (OtherProfile.bApplyRootSettings)
	{
		// copy deprecated pelvis settings
		{
			TArray<FIKRetargetPelvisMotionOpSettings*> AllSettingsInProfile;
			GetOpSettingsByTypeInProfile<FIKRetargetPelvisMotionOpSettings>(AllSettingsInProfile);
			for (FIKRetargetPelvisMotionOpSettings* SettingsInProfile : AllSettingsInProfile)
			{
				SettingsInProfile->RotationAlpha = OtherProfile.RootSettings.RotationAlpha;
				SettingsInProfile->TranslationAlpha = OtherProfile.RootSettings.TranslationAlpha;
				SettingsInProfile->BlendToSourceTranslation = OtherProfile.RootSettings.BlendToSource;
				SettingsInProfile->BlendToSourceTranslationWeights = OtherProfile.RootSettings.BlendToSourceWeights;
				SettingsInProfile->ScaleHorizontal = OtherProfile.RootSettings.ScaleHorizontal;
				SettingsInProfile->ScaleVertical = OtherProfile.RootSettings.ScaleVertical;
				SettingsInProfile->TranslationOffsetGlobal = OtherProfile.RootSettings.TranslationOffset;
				SettingsInProfile->RotationOffsetLocal = OtherProfile.RootSettings.RotationOffset;
				SettingsInProfile->AffectIKHorizontal = OtherProfile.RootSettings.AffectIKHorizontal;
				SettingsInProfile->AffectIKVertical = OtherProfile.RootSettings.AffectIKVertical;
			}
		}
	}

	if (OtherProfile.bApplyChainSettings)
	{
		// copy deprecated FK chain settings
		{
			TArray<FIKRetargetFKChainsOpSettings*> AllSettingsInProfile;
			GetOpSettingsByTypeInProfile<FIKRetargetFKChainsOpSettings>(AllSettingsInProfile);
			for (const TPair<FName,FTargetChainSettings>& Pair : OtherProfile.ChainSettings)
			{
				const FName TargetChainName = Pair.Key;
				const FTargetChainSettings& OtherChainSettings = Pair.Value;
				for (FIKRetargetFKChainsOpSettings* SettingsInProfile : AllSettingsInProfile)
				{
					for (FRetargetFKChainSettings& NewChainSettings : SettingsInProfile->ChainsToRetarget)
					{
						if (NewChainSettings.TargetChainName == TargetChainName)
						{
							NewChainSettings.EnableFK = OtherChainSettings.FK.EnableFK;
							NewChainSettings.RotationMode = static_cast<EFKChainRotationMode>(OtherChainSettings.FK.RotationMode);
							NewChainSettings.RotationAlpha = OtherChainSettings.FK.RotationAlpha;
							NewChainSettings.TranslationMode = static_cast<EFKChainTranslationMode>(OtherChainSettings.FK.TranslationMode);
							NewChainSettings.TranslationAlpha = OtherChainSettings.FK.TranslationAlpha;
						}
					}
				}
			}
		}
		
		// copy deprecated IK chain settings
		{
			TArray<FIKRetargetIKChainsOpSettings*> AllSettingsInProfile;
			GetOpSettingsByTypeInProfile<FIKRetargetIKChainsOpSettings>(AllSettingsInProfile);
			for (const TPair<FName,FTargetChainSettings>& Pair : OtherProfile.ChainSettings)
			{
				const FName TargetChainName = Pair.Key;
				const FTargetChainSettings& OtherChainSettings = Pair.Value;
				for (FIKRetargetIKChainsOpSettings* SettingsInProfile : AllSettingsInProfile)
				{
					for (FRetargetIKChainSettings& NewChainSettings : SettingsInProfile->ChainsToRetarget)
					{
						if (NewChainSettings.TargetChainName == TargetChainName)
						{
							NewChainSettings.EnableIK = OtherChainSettings.IK.EnableIK;
							NewChainSettings.BlendToSource = OtherChainSettings.IK.BlendToSource;
							NewChainSettings.BlendToSourceTranslation = OtherChainSettings.IK.BlendToSourceTranslation;
							NewChainSettings.BlendToSourceRotation = OtherChainSettings.IK.BlendToSourceRotation;
							NewChainSettings.BlendToSourceWeights = OtherChainSettings.IK.BlendToSourceWeights;
							NewChainSettings.StaticOffset = OtherChainSettings.IK.StaticOffset;
							NewChainSettings.StaticLocalOffset = OtherChainSettings.IK.StaticLocalOffset;
							NewChainSettings.StaticRotationOffset = OtherChainSettings.IK.StaticRotationOffset;
							NewChainSettings.ScaleVertical = OtherChainSettings.IK.ScaleVertical;
							NewChainSettings.Extension = OtherChainSettings.IK.Extension;
						}
					}
				}
			}
		}

		// copy deprecated speed planting settings (these are stored in the deprecated FK Chain settings)
		{
			TArray<FIKRetargetSpeedPlantingOpSettings*> AllSettingsInProfile;
			GetOpSettingsByTypeInProfile<FIKRetargetSpeedPlantingOpSettings>(AllSettingsInProfile);
			for (const TPair<FName,FTargetChainSettings>& Pair : OtherProfile.ChainSettings)
			{
				const FName TargetChainName = Pair.Key;
				const FTargetChainSettings& OtherChainSettings = Pair.Value;
				if (OtherChainSettings.SpeedPlanting.SpeedCurveName != NAME_None)
				{
					for (FIKRetargetSpeedPlantingOpSettings* SettingsInProfile : AllSettingsInProfile)
					{
						SettingsInProfile->SpeedThreshold = OtherChainSettings.SpeedPlanting.SpeedThreshold;
						SettingsInProfile->Stiffness = OtherChainSettings.SpeedPlanting.UnplantStiffness;
					}
				}
			}
		}

		// copy deprecated pole vector settings (these are stored in the deprecated FK Chain settings)
		{
			TArray<FIKRetargetAlignPoleVectorOpSettings*> OpSettingsInProfile;
			GetOpSettingsByTypeInProfile<FIKRetargetAlignPoleVectorOpSettings>(OpSettingsInProfile);
			for (const TPair<FName,FTargetChainSettings>& Pair : OtherProfile.ChainSettings)
			{
				const FName TargetChainName = Pair.Key;
				const FTargetChainSettings& OtherChainSettings = Pair.Value;

				FRetargetPoleVectorSettings SettingsToMerge;
				SettingsToMerge.bEnabled = true;
				SettingsToMerge.TargetChainName = TargetChainName;
				SettingsToMerge.AlignAlpha = OtherChainSettings.FK.PoleVectorMatching;
				SettingsToMerge.MaintainOffset = OtherChainSettings.FK.PoleVectorMaintainOffset;
				SettingsToMerge.StaticAngularOffset = OtherChainSettings.FK.PoleVectorOffset;
				
				for (FIKRetargetAlignPoleVectorOpSettings* SettingsInProfile : OpSettingsInProfile)
				{
					SettingsInProfile->MergePoleVectorSettings(SettingsToMerge);
				}
			}
		}
	}

	if (OtherProfile.bApplyGlobalSettings)
	{
		// copy deprecated stride warp settings
		{
			TArray<FIKRetargetStrideWarpingOpSettings*> AllSettingsInProfile;
			GetOpSettingsByTypeInProfile<FIKRetargetStrideWarpingOpSettings>(AllSettingsInProfile);
			for (FIKRetargetStrideWarpingOpSettings* SettingsInProfile : AllSettingsInProfile)
			{
				SettingsInProfile->DirectionSource = OtherProfile.GlobalSettings.DirectionSource;
				SettingsInProfile->ForwardDirection = OtherProfile.GlobalSettings.ForwardDirection;
				SettingsInProfile->DirectionChain = OtherProfile.GlobalSettings.DirectionChain;
				SettingsInProfile->WarpForwards = OtherProfile.GlobalSettings.WarpForwards;
				SettingsInProfile->SidewaysOffset = OtherProfile.GlobalSettings.SidewaysOffset;
				SettingsInProfile->WarpSplay = OtherProfile.GlobalSettings.WarpSplay;
			}
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//
	// END merge deprecated properties
	//
}
