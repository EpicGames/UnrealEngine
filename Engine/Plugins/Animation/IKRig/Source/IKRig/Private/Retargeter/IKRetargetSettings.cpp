// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargetSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetSettings)

#if WITH_EDITOR
FLinearColor FIKRetargetDebugDrawState::Muted = FLinearColor::Gray;
FLinearColor FIKRetargetDebugDrawState::SourceColor = (FLinearColor::Gray * FLinearColor::Blue) * Muted;
FLinearColor FIKRetargetDebugDrawState::GoalColor = FLinearColor::Yellow;
FLinearColor FIKRetargetDebugDrawState::MainColor = FLinearColor::Green;
FLinearColor FIKRetargetDebugDrawState::NonSelected = FLinearColor::Gray * 0.3f;
#endif
