// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/IKRetargeter.h"

#include "IKRigObjectVersion.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetProfile.h"
#include "Engine/SkeletalMesh.h"
#include "Retargeter/RetargetOps/AlignPoleVectorOp.h"
#include "Retargeter/RetargetOps/BlendToSourceOp.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"
#include "Retargeter/RetargetOps/IKChainsOp.h"
#include "Retargeter/RetargetOps/OffsetGoalsOp.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "Retargeter/RetargetOps/ScaleGoalsOp.h"
#include "Retargeter/RetargetOps/ScaleSourceOp.h"
#include "Retargeter/RetargetOps/SpeedPlantingOp.h"
#include "Retargeter/RetargetOps/StrideWarpingOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargeter)

#if WITH_EDITOR
const FName UIKRetargeter::GetSourceIKRigPropertyName() { return GET_MEMBER_NAME_STRING_CHECKED(UIKRetargeter, SourceIKRigAsset); };
const FName UIKRetargeter::GetTargetIKRigPropertyName() { return GET_MEMBER_NAME_STRING_CHECKED(UIKRetargeter, TargetIKRigAsset); };
const FName UIKRetargeter::GetSourcePreviewMeshPropertyName() { return GET_MEMBER_NAME_STRING_CHECKED(UIKRetargeter, SourcePreviewMesh); };
const FName UIKRetargeter::GetTargetPreviewMeshPropertyName() { return GET_MEMBER_NAME_STRING_CHECKED(UIKRetargeter, TargetPreviewMesh); }
#endif

UIKRetargeter::UIKRetargeter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	RootSettings_DEPRECATED = CreateDefaultSubobject<URetargetRootSettings>(TEXT("RootSettings"));
	RootSettings_DEPRECATED->SetFlags(RF_Transactional);

	GlobalSettings_DEPRECATED = CreateDefaultSubobject<UIKRetargetGlobalSettings>(TEXT("GlobalSettings"));
	GlobalSettings_DEPRECATED->SetFlags(RF_Transactional);
	
	OpStack_DEPRECATED = CreateDefaultSubobject<URetargetOpStack>(TEXT("PostSettings"));
	OpStack_DEPRECATED->SetFlags(RF_Transactional);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// we need this to ensure that new retargeters always have a default retarget pose
	CleanRetargetPoses();
}

const UIKRigDefinition* UIKRetargeter::GetIKRig(ERetargetSourceOrTarget SourceOrTarget) const
{
	const TSoftObjectPtr<UIKRigDefinition> SoftIKRig = SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceIKRigAsset : TargetIKRigAsset;
	if (SoftIKRig.IsValid())
	{
		return SoftIKRig.Get();
	}
	
	return IsInGameThread() ? SoftIKRig.LoadSynchronous() : nullptr;
}

UIKRigDefinition* UIKRetargeter::GetIKRigWriteable(ERetargetSourceOrTarget SourceOrTarget) const
{
	const TSoftObjectPtr<UIKRigDefinition> SoftIKRig = SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceIKRigAsset : TargetIKRigAsset;
	if (SoftIKRig.IsValid())
	{
		return SoftIKRig.Get();
	}
	
	return IsInGameThread() ? SoftIKRig.LoadSynchronous() : nullptr;
}

#if WITH_EDITORONLY_DATA
USkeletalMesh* UIKRetargeter::GetPreviewMesh(ERetargetSourceOrTarget SourceOrTarget) const
{
	if (!IsInGameThread())
	{
		return nullptr;
	}

	// the preview mesh override on the retarget takes precedence
	if (SourceOrTarget == ERetargetSourceOrTarget::Source)
	{
		if (SourcePreviewMesh.IsValid())
		{
			return SourcePreviewMesh.LoadSynchronous();
		}
	}
	else
	{
		if (TargetPreviewMesh.IsValid())
		{
			return TargetPreviewMesh.LoadSynchronous();
		}
	}

	// fallback to preview mesh from the IK Rig itself
	if (const UIKRigDefinition* IKRig = GetIKRig(SourceOrTarget))
	{
		return IKRig->GetPreviewMesh();
	}

	return nullptr;
}
#endif

void UIKRetargeter::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
#if WITH_EDITORONLY_DATA
	Controller = nullptr;
#endif
}

void UIKRetargeter::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);
};

void UIKRetargeter::PostLoad()
{
	Super::PostLoad();

	// very early versions of the asset may not have been set as standalone
	SetFlags(RF_Standalone);

	if (IsValid(SourceIKRigAsset))
	{
		SourceIKRigAsset->ConditionalPostLoad();
	}

	if (IsValid(TargetIKRigAsset))
	{
		TargetIKRigAsset->ConditionalPostLoad();
	}

	// remove null ops (could happen if op is in plugin that is not loaded)
	RetargetOps.RemoveAll([](const FInstancedStruct& InOp)
	{
		return !InOp.IsValid();
	});

#if WITH_EDITOR
	// post-load patches in order of oldest to newest asset upgrades
	PostLoadOldSettingsToNew();
	PostLoadOldOpsToNewStructOps();
	PostLoadConvertEverythingToOps();
	PostLoadPutChainMappingInOps();
#endif

	// give ops a chance to post load patch themselves
	// plugin architecture here allows ops in other plugins to load-patch
	PostLoadOpStack();

#if WITH_EDITOR
	// NOTE: must run after PostLoadOpStack() to properly patch blend to source "Affect IK"
	PostLoadSplitIKChainsOp();	
	PostLoadSplitOps(); // Note: Generic ops splitting, i.e. when turning singleton op into mutliple ones
#endif
	
	CleanOpStack();
	CleanRetargetPoses();
}

#if WITH_EDITOR
void UIKRetargeter::PostLoadOldSettingsToNew()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
#if WITH_EDITORONLY_DATA
	// load deprecated target actor offset
	if (!FMath::IsNearlyZero(TargetActorOffset_DEPRECATED))
	{
		TargetMeshOffset.X = TargetActorOffset_DEPRECATED;
	}

	// load deprecated target actor scale
	if (!FMath::IsNearlyZero(TargetActorScale_DEPRECATED))
	{
		TargetMeshScale = TargetActorScale_DEPRECATED;
	}

	// load deprecated global settings
	if (!bRetargetRoot_DEPRECATED)
	{
		GlobalSettings_DEPRECATED->Settings.bEnableRoot = false;
	}
	if (!bRetargetFK_DEPRECATED)
	{
		GlobalSettings_DEPRECATED->Settings.bEnableFK = false;
	}
	if (!bRetargetIK_DEPRECATED)
	{
		GlobalSettings_DEPRECATED->Settings.bEnableIK = false;
	}
#endif

	// load deprecated retarget poses (pre adding retarget poses for source)
	if (!RetargetPoses.IsEmpty())
	{
		TargetRetargetPoses = RetargetPoses;
	}

	// load deprecated current retarget pose (pre adding retarget poses for source)
	if (CurrentRetargetPose != NAME_None)
	{
		CurrentTargetRetargetPose = CurrentRetargetPose;
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UIKRetargeter::PostLoadOldOpsToNewStructOps()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
	// validate the input FInstancedStruct is a valid subclass of FRetargetOpBase
	auto DerivesFromBaseOpType = [](const FInstancedStruct& InConvertedSolver)
	{
		if (!InConvertedSolver.IsValid())
		{
			return false;
		}
		
		bool bIsDerivedFromBase = false;
		UStruct* CurrentSuperStruct = InConvertedSolver.GetScriptStruct()->GetSuperStruct();
		while (CurrentSuperStruct)
		{
			if (CurrentSuperStruct == FIKRetargetOpBase::StaticStruct())
			{
				bIsDerivedFromBase = true;
				break;
			}
			CurrentSuperStruct = CurrentSuperStruct->GetSuperStruct();
		}
		
		return bIsDerivedFromBase;
	};

	// load old UObject-based op stack and convert to new struct-based types
	for (TObjectPtr<URetargetOpBase> DeprecatedOp : OpStack_DEPRECATED->RetargetOps_DEPRECATED)
	{
		if (DeprecatedOp == nullptr)
		{
			continue;
		}
		
		FInstancedStruct ConvertedOp;
		DeprecatedOp->ConvertToInstancedStruct(ConvertedOp);
		if (DerivesFromBaseOpType(ConvertedOp))
		{
			RetargetOps.Add(ConvertedOp);
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "IK Retargeter: unable to load old UObject-based op type. Conversion failed for type:  %ls", *DeprecatedOp->GetName());
		}
	}
	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UIKRetargeter::PostLoadConvertEverythingToOps()
{
	// only perform this if loading older package version
	if (GetLinkerCustomVersion(FIKRigObjectVersion::GUID) >= FIKRigObjectVersion::ModularRetargeterOps)
	{
		return;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// load old chain settings into new chain map
	ChainMap_DEPRECATED.LoadFromDeprecatedChainSettings(ChainSettings_DEPRECATED);
	
	// copy old post-ops to add to the end of the stack
	TArray<FInstancedStruct> OldPostOps = RetargetOps;
	RetargetOps.Empty();

	// record all chains that have an IK goal assigned
	TArray<FName> ChainsWithIK;
	if (IsValid(TargetIKRigAsset))
	{
		const TArray<FBoneChain>& RetargetBoneChains =TargetIKRigAsset->GetRetargetChains();
		for (const FBoneChain& Chain : RetargetBoneChains)
		{
			if (Chain.IKGoalName != NAME_None)
			{
				ChainsWithIK.Add(Chain.ChainName);
			}
		}
	}
	
	int32 OpIndex = 0;

	FName RunIKOpName = FIKRetargetRunIKRigOp().GetDefaultName();

	// Source Scale Op from old "global" settings
	{
		if (!FMath::IsNearlyEqual(GlobalSettings_DEPRECATED->Settings.SourceScaleFactor, 1.0f))
		{
			RetargetOps.EmplaceAt(OpIndex,FIKRetargetScaleSourceOp::StaticStruct());
			FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
			FIKRetargetScaleSourceOpSettings* Settings = reinterpret_cast<FIKRetargetScaleSourceOpSettings*>(Op->GetSettings());
			Settings->SourceScaleFactor = GlobalSettings_DEPRECATED->Settings.SourceScaleFactor;
		}
	}

	// Pelvis Motion Op from old "root" settings
	{
		RetargetOps.EmplaceAt(OpIndex,FIKRetargetPelvisMotionOp::StaticStruct());
		FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
		Op->SetEnabled(GlobalSettings_DEPRECATED->Settings.bEnableRoot);
		FIKRetargetPelvisMotionOpSettings* Settings = reinterpret_cast<FIKRetargetPelvisMotionOpSettings*>(Op->GetSettings());
		Settings->SourcePelvisBone.BoneName = SourceIKRigAsset ? SourceIKRigAsset->GetPelvis() : NAME_None;
		Settings->TargetPelvisBone.BoneName = TargetIKRigAsset ? TargetIKRigAsset->GetPelvis() : NAME_None;
		Settings->RotationAlpha = RootSettings_DEPRECATED->Settings.RotationAlpha;
		Settings->TranslationAlpha = RootSettings_DEPRECATED->Settings.TranslationAlpha;
		Settings->BlendToSourceTranslation = RootSettings_DEPRECATED->Settings.BlendToSource;
		Settings->BlendToSourceTranslationWeights = RootSettings_DEPRECATED->Settings.BlendToSourceWeights;
		Settings->ScaleHorizontal = RootSettings_DEPRECATED->Settings.ScaleHorizontal;
		Settings->ScaleVertical = RootSettings_DEPRECATED->Settings.ScaleVertical;

		// NOTE: load these into deprecated properties, which are later converted to final properties by FIKRetargetPelvisMotionOp::PostLoad()
		Settings->TranslationOffset_DEPRECATED = RootSettings_DEPRECATED->Settings.TranslationOffset;
		Settings->RotationOffset_DEPRECATED = RootSettings_DEPRECATED->Settings.RotationOffset;
		
		Settings->AffectIKHorizontal = RootSettings_DEPRECATED->Settings.AffectIKHorizontal;
		Settings->AffectIKVertical = RootSettings_DEPRECATED->Settings.AffectIKVertical;
	}

	// FK Chains Op from old FK chain settings
	{
		++OpIndex;
		RetargetOps.EmplaceAt(OpIndex,FIKRetargetFKChainsOp::StaticStruct());
		FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
		Op->SetEnabled(GlobalSettings_DEPRECATED->Settings.bEnableFK);
		FIKRetargetFKChainsOpSettings* Settings = reinterpret_cast<FIKRetargetFKChainsOpSettings*>(Op->GetSettings());
		for (URetargetChainSettings* Chain : ChainSettings_DEPRECATED)
		{
			FRetargetFKChainSettings ChainToRetarget;
			ChainToRetarget.TargetChainName = Chain->TargetChain;
			ChainToRetarget.EnableFK = Chain->Settings.FK.EnableFK;
			ChainToRetarget.RotationMode = static_cast<EFKChainRotationMode>(Chain->Settings.FK.RotationMode);
			ChainToRetarget.RotationAlpha = Chain->Settings.FK.RotationAlpha;
			ChainToRetarget.TranslationMode = static_cast<EFKChainTranslationMode>(Chain->Settings.FK.TranslationMode);
			ChainToRetarget.TranslationAlpha = Chain->Settings.FK.TranslationAlpha;
			
			Settings->ChainsToRetarget.Add(ChainToRetarget);
		}
	}

	// IK Chains Op from old IK chain settings
	{
		++OpIndex;
		RetargetOps.EmplaceAt(OpIndex,FIKRetargetIKChainsOp::StaticStruct());
		FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
		Op->SetParentOpName(RunIKOpName);
		Op->SetEnabled(GlobalSettings_DEPRECATED->Settings.bEnableIK);
		FIKRetargetIKChainsOpSettings* Settings = reinterpret_cast<FIKRetargetIKChainsOpSettings*>(Op->GetSettings());
		for (URetargetChainSettings* Chain : ChainSettings_DEPRECATED)
		{
			if (!ChainsWithIK.Contains(Chain->TargetChain))
			{
				continue; // skip chains with no IK
			}
			FRetargetIKChainSettings ChainToRetarget;
			ChainToRetarget.TargetChainName = Chain->TargetChain;
			ChainToRetarget.EnableIK = Chain->Settings.IK.EnableIK;
			ChainToRetarget.BlendToSource = Chain->Settings.IK.BlendToSource;
			ChainToRetarget.BlendToSourceTranslation = Chain->Settings.IK.BlendToSourceTranslation;
			ChainToRetarget.BlendToSourceRotation = Chain->Settings.IK.BlendToSourceRotation;
			ChainToRetarget.BlendToSourceWeights = Chain->Settings.IK.BlendToSourceWeights;
			ChainToRetarget.StaticOffset = Chain->Settings.IK.StaticOffset;
			ChainToRetarget.StaticLocalOffset = Chain->Settings.IK.StaticLocalOffset;
			ChainToRetarget.StaticRotationOffset = Chain->Settings.IK.StaticRotationOffset;
			ChainToRetarget.ScaleVertical = Chain->Settings.IK.ScaleVertical;
			ChainToRetarget.Extension = Chain->Settings.IK.Extension;
			
			Settings->ChainsToRetarget.Add(ChainToRetarget);
		}
	}

	// Stride Warping Op from old "IK" chain settings and global settings
	{
		++OpIndex;
		RetargetOps.EmplaceAt(OpIndex,FIKRetargetStrideWarpingOp::StaticStruct());
		FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
		Op->SetParentOpName(RunIKOpName);
		Op->SetEnabled(GlobalSettings_DEPRECATED->Settings.bWarping);
		FIKRetargetStrideWarpingOpSettings* Settings = reinterpret_cast<FIKRetargetStrideWarpingOpSettings*>(Op->GetSettings());
		Settings->DirectionSource = GlobalSettings_DEPRECATED->Settings.DirectionSource;
		Settings->ForwardDirection = GlobalSettings_DEPRECATED->Settings.ForwardDirection;
		Settings->DirectionChain = GlobalSettings_DEPRECATED->Settings.DirectionChain;
		Settings->WarpForwards = GlobalSettings_DEPRECATED->Settings.WarpForwards;
		Settings->SidewaysOffset = GlobalSettings_DEPRECATED->Settings.SidewaysOffset;
		Settings->WarpSplay = GlobalSettings_DEPRECATED->Settings.WarpSplay;
		
		for (URetargetChainSettings* Chain : ChainSettings_DEPRECATED)
		{
			if (!ChainsWithIK.Contains(Chain->TargetChain))
			{
				continue; // skip chains with no IK
			}
			
			if (Chain->Settings.IK.bAffectedByIKWarping)
			{
				Settings->ChainSettings.Add(Chain->TargetChain);
			}
		}
	}
	
	// Speed Planting Op from old "IK" chain settings and global settings
	{
		++OpIndex;
		RetargetOps.EmplaceAt(OpIndex,FIKRetargetSpeedPlantingOp::StaticStruct());
		FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
		Op->SetParentOpName(RunIKOpName);
		FIKRetargetSpeedPlantingOpSettings* Settings = reinterpret_cast<FIKRetargetSpeedPlantingOpSettings*>(Op->GetSettings());
		for (URetargetChainSettings* Chain : ChainSettings_DEPRECATED)
		{
			if (!ChainsWithIK.Contains(Chain->TargetChain))
			{
				continue; // skip chains with no IK
			}
			
			if (Chain->Settings.SpeedPlanting.SpeedCurveName != NAME_None)
			{
				FRetargetSpeedPlantingSettings ChainToPlant;
				ChainToPlant.TargetChainName = Chain->TargetChain;
				ChainToPlant.SpeedCurveName = Chain->Settings.SpeedPlanting.SpeedCurveName;
				Settings->ChainsToSpeedPlant.Add(ChainToPlant);

				Settings->SpeedThreshold = Chain->Settings.SpeedPlanting.SpeedThreshold;
				Settings->Stiffness = Chain->Settings.SpeedPlanting.UnplantStiffness;
			}
		}
	}
	
	// IK Solve Op from old global settings
	{
		++OpIndex;
		RetargetOps.EmplaceAt(OpIndex,FIKRetargetRunIKRigOp::StaticStruct());
		FIKRetargetRunIKRigOp* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetRunIKRigOp>();
		Op->SetName(RunIKOpName);
		FIKRetargetRunIKRigOpSettings* Settings = reinterpret_cast<FIKRetargetRunIKRigOpSettings*>(Op->GetSettings());
		Settings->IKRigAsset = TargetIKRigAsset;
	}

	// Pole Vector Op from old chains
	{
		++OpIndex;
		RetargetOps.EmplaceAt(OpIndex,FIKRetargetAlignPoleVectorOp::StaticStruct());
		FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
		FIKRetargetAlignPoleVectorOpSettings* Settings = reinterpret_cast<FIKRetargetAlignPoleVectorOpSettings*>(Op->GetSettings());
		for (URetargetChainSettings* Chain : ChainSettings_DEPRECATED)
		{
			if (Chain->Settings.FK.PoleVectorMatching > 0.0f)
			{
				FRetargetPoleVectorSettings ChainToAlign;
				ChainToAlign.bEnabled = true;
				ChainToAlign.TargetChainName = Chain->TargetChain;
				ChainToAlign.AlignAlpha = Chain->Settings.FK.PoleVectorMatching;
				ChainToAlign.MaintainOffset = Chain->Settings.FK.PoleVectorMaintainOffset;
				ChainToAlign.StaticAngularOffset = Chain->Settings.FK.PoleVectorOffset;
				Settings->ChainsToAlign.Add(ChainToAlign);
			}
		}
	}

	// append old post-ops to the end of the stack
	RetargetOps.Append(OldPostOps);
	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UIKRetargeter::PostLoadPutChainMappingInOps()
{
	// only perform this if loading older package version
	if (GetLinkerCustomVersion(FIKRigObjectVersion::GUID) >= FIKRigObjectVersion::OpsOwnChainMapping)
	{
		return;
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// NOTE: we go through all op types that have been upgraded to support a custom IK Rig asset and
	// copy the global chain mapping into the op's local chain mapping.
	// Later in PostLoad(), the op stack will be cleaned which gives each op a callback to regenerate it's per-chain settings
	
	TArray<FIKRetargetRunIKRigOp*> RunIKOps = GetAllRetargetOpsOfType<FIKRetargetRunIKRigOp>();
	for (FIKRetargetRunIKRigOp* Op : RunIKOps)
	{
		Op->ChainMapping = ChainMap_DEPRECATED;
	}
	
	TArray<FIKRetargetFKChainsOp*> FKChainOps = GetAllRetargetOpsOfType<FIKRetargetFKChainsOp>();
	for (FIKRetargetFKChainsOp* Op : FKChainOps)
	{
		Op->Settings.IKRigAsset = TargetIKRigAsset;
		Op->Settings.ChainMapping = ChainMap_DEPRECATED;
	}

	TArray<FIKRetargetAlignPoleVectorOp*> PoleVectorOps = GetAllRetargetOpsOfType<FIKRetargetAlignPoleVectorOp>();
	for (FIKRetargetAlignPoleVectorOp* Op : PoleVectorOps)
	{
		Op->Settings.IKRigAsset = TargetIKRigAsset;
		Op->ChainMapping = ChainMap_DEPRECATED;
	}
	
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UIKRetargeter::PostLoadSplitIKChainsOp()
{
	// only perform this if loading older package version
	if (GetLinkerCustomVersion(FIKRigObjectVersion::GUID) >= FIKRigObjectVersion::IKChainsOpSplitIntoSeparateOps)
	{
		return;
	}

	auto AllChainSettingsAtDefault = [](FIKRetargetIKChainsOp* InIKOp)
		{
			const FRetargetIKChainSettings DefaultSettings;
			for (const FRetargetIKChainSettings& Chain : InIKOp->Settings.ChainsToRetarget)
			{
				if (Chain != DefaultSettings)
				{
					return false;
				}
			}
			return true;
		};

	auto CopySettingsToRunIKRigOp = [](FIKRetargetIKChainsOp& InIKOp, FIKRetargetRunIKRigOp& InRunIKRigOp)
		{
			for (const FRetargetIKChainSettings& CopyFromChain : InIKOp.Settings.ChainsToRetarget)
			{
				for (FIKRetargetRunIKRigChainSettings& CopyToChain : InRunIKRigOp.Settings.Chains)
				{
					if (CopyFromChain.TargetChainName != CopyToChain.TargetChainName)
					{
						continue;
					}

					// copy only the enable flag to this op
					CopyToChain.bEnableIK = CopyFromChain.EnableIK;
				}
			}
		};

	auto UsingBlendToSource = [](FIKRetargetIKChainsOp* InIKOp)
		{
			for (const FRetargetIKChainSettings& Chain : InIKOp->Settings.ChainsToRetarget)
			{
				if (!FMath::IsNearlyEqual(Chain.BlendToSource, 0.0))
				{
					return true;
				}
			}
			return false;
		};
	
	auto CopySettingsToBlendToSourceOp = [](FIKRetargetIKChainsOp& InIKOp, FIKRetargetBlendToSourceOp& InNewOp)
		{
			for (const FRetargetIKChainSettings& CopyFromChain : InIKOp.Settings.ChainsToRetarget)
			{
				for (FIKRetargetBlendToSourceChainSettings& CopyToChain : InNewOp.Settings.Chains)
				{
					if (CopyFromChain.TargetChainName != CopyToChain.TargetChainName)
					{
						continue;
					}

					CopyToChain.BlendToSource = CopyFromChain.BlendToSource;
					CopyToChain.TranslationAlpha = CopyFromChain.BlendToSourceTranslation;
					CopyToChain.RotationAlpha = CopyFromChain.BlendToSourceRotation;
					CopyToChain.TranslationPerAxisAlpha = CopyFromChain.BlendToSourceWeights;
					CopyToChain.ApplyPelvisOffset = CopyFromChain.ApplyPelvisOffsetToSourceGoals;
				}
			}
		};
	
	auto UsingStaticOffsets = [](FIKRetargetIKChainsOp* InIKOp)
		{
			for (const FRetargetIKChainSettings& Chain : InIKOp->Settings.ChainsToRetarget)
			{
				if (!Chain.StaticOffset.Equals(FVector::ZeroVector))
				{
					return true;
				}
				if (!Chain.StaticLocalOffset.Equals(FVector::ZeroVector))
				{
					return true;
				}
				if (!Chain.StaticRotationOffset.Equals(FRotator::ZeroRotator))
				{
					return true;
				}
			}
			return false;
		};

	auto CopySettingsToOffsetOp = [](FIKRetargetIKChainsOp& InIKOp, FIKRetargetOffsetGoalsOp& InNewOp)
		{
			for (const FRetargetIKChainSettings& CopyFromChain : InIKOp.Settings.ChainsToRetarget)
			{
				for (FIKRetargetOffsetGoalsChainSettings& CopyToChain : InNewOp.Settings.Chains)
				{
					if (CopyFromChain.TargetChainName != CopyToChain.TargetChainName)
					{
						continue;
					}

					// if any of the offsets on this chain are non-default, set alpha to 1 to enable them
					if (!CopyFromChain.StaticOffset.Equals(FVector::ZeroVector) ||
						!CopyFromChain.StaticLocalOffset.Equals(FVector::ZeroVector) ||
						!CopyFromChain.StaticRotationOffset.Equals(FRotator::ZeroRotator))
					{
						CopyToChain.Alpha = 1.0;
						CopyToChain.GlobalTranslationOffset = CopyFromChain.StaticOffset;
						CopyToChain.LocalTranslationOffset = CopyFromChain.StaticLocalOffset;
						CopyToChain.LocalRotationOffset = CopyFromChain.StaticRotationOffset;
					}
				}
			}
		};

	auto UsingScaling = [](FIKRetargetIKChainsOp* InIKOp)
		{
			for (const FRetargetIKChainSettings& Chain : InIKOp->Settings.ChainsToRetarget)
			{
				if (!FMath::IsNearlyEqual(Chain.ScaleVertical, 1.0))
				{
					return true;
				}
				if (!FMath::IsNearlyEqual(Chain.Extension, 1.0))
				{
					return true;
				}
			}
			return false;
		};

	auto CopySettingsToScaleGoalsOp = [](FIKRetargetIKChainsOp& InIKOp, FIKRetargetScaleGoalsOp& InNewOp)
		{
			for (const FRetargetIKChainSettings& CopyFromChain : InIKOp.Settings.ChainsToRetarget)
			{
				for (FIKRetargetScaleGoalsChainSettings& CopyToChain : InNewOp.Settings.Chains)
				{
					if (CopyFromChain.TargetChainName != CopyToChain.TargetChainName)
					{
						continue;
					}

					CopyToChain.ScaleVertical = CopyFromChain.ScaleVertical;
					CopyToChain.ScaleAlongChain = CopyFromChain.Extension;
				}
			}
		};

	auto RemoveOp = [this](FIKRetargetOpBase* InOp)
		{
			int32 OpIndex = GetIndexOfRetargetOp(InOp);
			if (OpIndex != INDEX_NONE)
			{
				RetargetOps.RemoveAt(OpIndex);	
			}
		};

	// 1st pass:
	// initialize chain settings on any RunIKRig ops so they have settings for all the chains in the rig
	// prior version of this op had no chain settings, so we must forcibly generate them now
	TArray<FIKRetargetRunIKRigOp*> RunIKRigOps = GetAllRetargetOpsOfType<FIKRetargetRunIKRigOp>();
	for (FIKRetargetRunIKRigOp* Op : RunIKRigOps)
	{
		Op->RegenerateChainSettings();
	}

	// 2nd pass
	// go through all the "Retarget IK Goal" ops and split them out into the new ops as needed
	TArray<FIKRetargetIKChainsOp*> IKChainOps = GetAllRetargetOpsOfType<FIKRetargetIKChainsOp>();
	for (FIKRetargetIKChainsOp* Op : IKChainOps)
	{
		FIKRetargetOpBase* ParentOp = GetRetargetOpByNameNonConst(Op->GetParentOpName());
		if (!ParentOp)
		{
			// should not happen
			// orphaned IK Chain op is invalid, skip upgrade
			continue;
		}

		if (ParentOp->GetType() != FIKRetargetRunIKRigOp::StaticStruct())
		{
			// should not happen
			// IK Chain op parented incorrectly, skip upgrade
			continue;
		}
		
		FIKRetargetRunIKRigOp* RunIKRigOp = reinterpret_cast<FIKRetargetRunIKRigOp*>(ParentOp);

		// copy chain settings to the run ik rig op
		CopySettingsToRunIKRigOp(*Op, *RunIKRigOp);
		
		// with default settings, the op behaves identically to the new default RunIKRigOp
		// so we can safely delete it and let the parent op do the work
		if (AllChainSettingsAtDefault(Op))
		{
			continue;
		}

		FIKRetargetOpBase* NextOpInStack = Op;

		// if using any blend to source features, create an identically configured Blend to Source op
		if (UsingBlendToSource(Op))
		{
			FInstancedStruct& NewOpStruct = RetargetOps.InsertDefaulted_GetRef(GetIndexOfRetargetOp(NextOpInStack) + 1);
			NewOpStruct.InitializeAs<FIKRetargetBlendToSourceOp>();
			FIKRetargetBlendToSourceOp& NewOp = NewOpStruct.GetMutable<FIKRetargetBlendToSourceOp>();
			NewOp.RegenerateChainSettings(ParentOp);
			CopySettingsToBlendToSourceOp(*Op, NewOp);
			NewOp.SetEnabled(Op->IsEnabled());
			NextOpInStack = &NewOp;
		}

		// if using any offset features, create an identically configured Offset Goals op
		if (UsingStaticOffsets(Op))
		{
			FInstancedStruct& NewOpStruct = RetargetOps.InsertDefaulted_GetRef(GetIndexOfRetargetOp(NextOpInStack) + 1);
			NewOpStruct.InitializeAs<FIKRetargetOffsetGoalsOp>();
			FIKRetargetOffsetGoalsOp& NewOp = NewOpStruct.GetMutable<FIKRetargetOffsetGoalsOp>();
			NewOp.RegenerateChainSettings(ParentOp);
			CopySettingsToOffsetOp(*Op, NewOp);
			NewOp.SetEnabled(Op->IsEnabled());
			NextOpInStack = &NewOp;
		}

		// if using any scaling features, create an identically configured Scale Goals op
		if (UsingScaling(Op))
		{
			FInstancedStruct& NewOpStruct = RetargetOps.InsertDefaulted_GetRef(GetIndexOfRetargetOp(NextOpInStack) + 1);
			NewOpStruct.InitializeAs<FIKRetargetScaleGoalsOp>();
			FIKRetargetScaleGoalsOp& NewOp = NewOpStruct.GetMutable<FIKRetargetScaleGoalsOp>();
			NewOp.RegenerateChainSettings(ParentOp);
			CopySettingsToScaleGoalsOp(*Op, NewOp);
			NewOp.SetEnabled(Op->IsEnabled());
			NextOpInStack = &NewOp;
		}
	}

	// 3rd pass
	// remove all the IKGoal ops
	for (FIKRetargetIKChainsOp* Op : IKChainOps)
	{
		RemoveOp(Op);
	}
}

void UIKRetargeter::PostLoadSplitOps()
{
	TArray<int32> OpsToRemove;
	TArray<FInstancedStruct> OpsToAdd;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	for (int32 OpIndex = 0; OpIndex < RetargetOps.Num(); OpIndex++)
	{
		FInstancedStruct& OpStruct = RetargetOps[OpIndex];
		FIKRetargetOpBase* Op = OpStruct.GetMutablePtr<FIKRetargetOpBase>();
		if (Op)
		{
			TArray<FInstancedStruct> NewOps;
			// By default there will be no splitting
			if (Op->PostLoadSplitOp(NewOps))
			{
				if (NewOps.Num() > 0)
				{
					// Mark index of the original op for removal
					OpsToRemove.Add(OpIndex);

					// Add the new ops
					OpsToAdd.Append(MoveTemp(NewOps));

					UE_LOGF(LogTemp, Log, "IK Retargeter: Op %ls split into %d new ops during PostLoad", *Op->GetName().ToString(), NewOps.Num());
				}
			}
		}
	}
	// Remove old ops in reverse order to maintain indices
	for (int32 i = OpsToRemove.Num() - 1; i >= 0; i--)
	{
		RetargetOps.RemoveAt(OpsToRemove[i]);
	}
	// Add new ops
	RetargetOps.Append(OpsToAdd);

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#endif

FName UIKRetargeter::GetCleanAndUniqueOpName(const FName& InOpName, const int32 InIndexOfOp)
{
	FName OutCleanedName = InOpName;

	// if None, revert to default name for the op
	if (InOpName == NAME_None)
	{
		#if WITH_EDITOR
		OutCleanedName = RetargetOps[InIndexOfOp].GetPtr<FIKRetargetOpBase>()->GetDefaultName();
		#else
		OutCleanedName = FName("DefaultRetargetOpName");	
		#endif
	}

	auto OpNameInUse = [this](const FName InOpNameToCheck, int32 InOpIndexToIgnore)
	{
		for (int32 OpIndex=0; OpIndex<RetargetOps.Num(); ++OpIndex)
		{
			if (InOpIndexToIgnore != INDEX_NONE && OpIndex == InOpIndexToIgnore)
			{
				continue;
			}
		
			const FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetPtr<FIKRetargetOpBase>();
			if (Op->GetName() == InOpNameToCheck)
			{
				return true;
			}
		}
		return false;
	};

	if (!OpNameInUse(OutCleanedName, InIndexOfOp))
	{
		return OutCleanedName;
	}

	// keep concatenating an incremented integer suffix until name is unique
	int32 Number = OutCleanedName.GetNumber() + 1;
	while(OpNameInUse(FName(OutCleanedName, Number), InIndexOfOp))
	{
		Number++;
	}

	return FName(OutCleanedName, Number);
}

void UIKRetargeter::CleanRetargetPoses()
{
	// use default pose as current pose unless set to something else
	if (CurrentSourceRetargetPose == NAME_None)
	{
		CurrentSourceRetargetPose = GetDefaultPoseName();
	}
	if (CurrentTargetRetargetPose == NAME_None)
	{
		CurrentTargetRetargetPose = GetDefaultPoseName();
	}

	// enforce the existence of a default pose
	if (!SourceRetargetPoses.Contains(GetDefaultPoseName()))
	{
		SourceRetargetPoses.Emplace(GetDefaultPoseName());
	}
	if (!TargetRetargetPoses.Contains(GetDefaultPoseName()))
	{
		TargetRetargetPoses.Emplace(GetDefaultPoseName());
	}

	// ensure current pose exists, otherwise set it to the default pose
	if (!SourceRetargetPoses.Contains(CurrentSourceRetargetPose))
	{
		CurrentSourceRetargetPose = GetDefaultPoseName();
	}
	if (!TargetRetargetPoses.Contains(CurrentTargetRetargetPose))
	{
		CurrentTargetRetargetPose = GetDefaultPoseName();
	}
}

void UIKRetargeter::CleanOpStack()
{
	auto GetNamesOfTopLevelOps = [this]() -> TArray<FName>
	{
		TArray<FName> NamesOfTopLevelOps;
		for (FInstancedStruct& OpStruct : RetargetOps)
		{
			const FIKRetargetOpBase& Op = OpStruct.Get<FIKRetargetOpBase>();
			if (Op.GetParentOpName() == NAME_None)
			{
				NamesOfTopLevelOps.Add(Op.GetName());
			}
		}
		return MoveTemp(NamesOfTopLevelOps);
	};

	auto GetChildOpNames = [this](const int32 InOpIndex) -> TArray<FName>
	{
		TArray<FName> ChildrenNames;
		const FName InOpName = RetargetOps[InOpIndex].Get<FIKRetargetOpBase>().GetName();
		for (FInstancedStruct& OpStruct : RetargetOps)
		{
			const FIKRetargetOpBase& Op = OpStruct.Get<FIKRetargetOpBase>();
			if (Op.GetParentOpName() == InOpName)
			{
				ChildrenNames.Add(Op.GetName());
			}
		}
		return MoveTemp(ChildrenNames);
	};

	auto GetIndexOfOpByName = [this](const FName InOpName) -> int32
	{
		for (int32 OpIndex=0; OpIndex<RetargetOps.Num(); ++OpIndex)
		{
			if (RetargetOps[OpIndex].Get<FIKRetargetOpBase>().GetName() == InOpName)
			{
				return OpIndex;
			}
		}
		return INDEX_NONE;
	};
	
	// enforce unique non-None names on all ops
	for (int32 OpIndex=0; OpIndex<RetargetOps.Num(); ++OpIndex)
	{
		FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetMutablePtr<FIKRetargetOpBase>();
		FName OldOpName = Op->GetName();
		FName CleanedOpName = GetCleanAndUniqueOpName(OldOpName, OpIndex);
		Op->SetName(CleanedOpName);

		// update any children pointing at the old name
		if (OldOpName != NAME_None)
		{
			for (FInstancedStruct& OpStruct : RetargetOps)
			{
				FIKRetargetOpBase& OtherOp = OpStruct.GetMutable<FIKRetargetOpBase>();
				if (OtherOp.GetParentOpName() == OldOpName)
				{
					OtherOp.SetParentOpName(CleanedOpName);
				}
			}
		}	
	}

	// auto parent ops with missing or unset parent
	for (FInstancedStruct& OpStruct : RetargetOps)
	{
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		const UScriptStruct* ParentType = Op.GetParentOpType();
		if (ParentType == nullptr)
		{
			// op doesn't need a parent
			continue;
		}
		
		if (GetIndexOfOpByName(Op.GetParentOpName()) != INDEX_NONE)
		{
			// op already has a parent and it exists
			continue;
		}

		// op needs a parent but doesn't have one
		// find the first op of the correct type and parent it
		for (FInstancedStruct& OtherOpStruct : RetargetOps)
		{
			if (OtherOpStruct.GetScriptStruct() == ParentType)
			{
				const FIKRetargetOpBase& OtherOp = OtherOpStruct.Get<FIKRetargetOpBase>();
				Op.SetParentOpName(OtherOp.GetName());
				break;
			}
		}
	}

	// enforce correct execution order of the ops according to the following constraints
	// 1. all children must come BEFORE parent
	// 2. compact
	// 3. retain existing order of ops to the extent possible
	// 4. no non-siblings between siblings

	// get list of op names in the correct execution order
	TArray<FName> TopLevelOpsNames = GetNamesOfTopLevelOps();
	TArray<FName> CorrectedOpOrder;
	for (const FName TopLevelOpName : TopLevelOpsNames)
	{
		const int32 OpIndex = GetIndexOfOpByName(TopLevelOpName);
		CorrectedOpOrder.Append(GetChildOpNames(OpIndex));
		CorrectedOpOrder.Add(TopLevelOpName);
	}
	
	// re-order op stack accordingly
	TArray<FInstancedStruct> TempOps = MoveTemp(RetargetOps);
	for (const FName& NameOfNextOpToAdd : CorrectedOpOrder)
	{
		for (FInstancedStruct& TempOpStruct : TempOps)
		{
			if (!TempOpStruct.IsValid())
			{
				continue; // already moved
			}
			
			FIKRetargetOpBase* TempOp = TempOpStruct.GetMutablePtr<FIKRetargetOpBase>();
			if (TempOp->GetName() == NameOfNextOpToAdd)
			{
				RetargetOps.Add(MoveTemp(TempOpStruct));
				break;
			}
		}
	}

	// clean chain mappings inside ops
	// NOTE: this updates the IK Rig references in the chain mappings and refreshes the list of source/target chains
	for (FInstancedStruct& OpStruct : RetargetOps)
	{
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		FRetargetChainMapping* ChainMapping = Op.GetChainMapping();
		if (ChainMapping == nullptr)
		{
			continue;
		}
		const UIKRigDefinition* SourceIKRig = GetIKRig(ERetargetSourceOrTarget::Source);
		const UIKRigDefinition* TargetIKRig = Op.GetCustomTargetIKRig();
		if (TargetIKRig == nullptr)
		{
			if (const FIKRetargetOpBase* ParentOp = GetRetargetOpByName(Op.GetParentOpName()))
			{
				TargetIKRig = ParentOp->GetCustomTargetIKRig();	
			}
		}
		ChainMapping->ReinitializeWithIKRigs(SourceIKRig, TargetIKRig);
	}

	// give each op a chance to clean its own data based on its parents state
	for (FInstancedStruct& OpStruct : RetargetOps)
	{
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		Op.OnReinitPropertyEdited(nullptr);
		
		if (const FIKRetargetOpBase* ParentOp = GetRetargetOpByName(Op.GetParentOpName()))
		{
			Op.OnParentReinitPropertyEdited(*ParentOp,nullptr);
		}
	}
	
	ensureAlwaysMsgf(RetargetOps.Num() == TempOps.Num(), TEXT("Retarget ops were lost during cleaning."));
}

void UIKRetargeter::PostLoadOpStack()
{
	// this is required because TStructOpsTypeTraits PostSerialize() is not called if the struct data is at default values
	const FIKRigObjectVersion::Type CustomVersion = static_cast<FIKRigObjectVersion::Type>(GetLinkerCustomVersion(FIKRigObjectVersion::GUID));
	for (FInstancedStruct& OpStruct : RetargetOps)
	{
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		Op.PostLoad(CustomVersion);

		// in 5.8 we moved the enabled state from the op itself into the op settings.
		// only apply for 5.6/5.7-era assets (after ModularRetargeterOps but before EnabledStateMovedFromOpToOpSettings).
		// pre-5.6 assets have bIsEnabled_DEPRECATED at its default value (true) since it was never serialized —
		// their enabled state is already correctly set by PostLoadConvertEverythingToOps/PostLoadOldOpsToNewStructOps.
		if (CustomVersion >= FIKRigObjectVersion::ModularRetargeterOps &&
			CustomVersion < FIKRigObjectVersion::EnabledStateMovedFromOpToOpSettings)
		{
			Op.GetSettings()->bEnabled = Op.GetDeprecatedEnabledState();
		}
	}
};

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UIKRetargeter::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(URetargetChainSettings::StaticClass()));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

FQuat FIKRetargetPose::GetDeltaRotationForBone(const FName BoneName) const
{
	const FQuat* BoneRotationOffset = BoneRotationOffsets.Find(BoneName);
	return BoneRotationOffset != nullptr ? *BoneRotationOffset : FQuat::Identity;
}

void FIKRetargetPose::SetDeltaRotationForBone(FName BoneName, const FQuat& RotationDelta)
{
	IncrementVersion();

	FQuat* RotOffset = BoneRotationOffsets.Find(BoneName);
	if (RotOffset == nullptr)
	{
		// first time this bone has been modified in this pose
		BoneRotationOffsets.Emplace(BoneName, RotationDelta);
		return;
	}

	*RotOffset = RotationDelta;
}

FVector FIKRetargetPose::GetRootTranslationDelta() const
{
	return RootTranslationOffset;
}

void FIKRetargetPose::SetRootTranslationDelta(const FVector& TranslationDelta)
{
	IncrementVersion();
	
	RootTranslationOffset = TranslationDelta;
	// only allow vertical offset of root in retarget pose
	RootTranslationOffset.X = 0.0f;
	RootTranslationOffset.Y = 0.0f;
}

void FIKRetargetPose::AddToRootTranslationDelta(const FVector& TranslateDelta)
{
	IncrementVersion();
	
	RootTranslationOffset += TranslateDelta;
	// only allow vertical offset of root in retarget pose
	RootTranslationOffset.X = 0.0f;
	RootTranslationOffset.Y = 0.0f;
}

void FIKRetargetPose::SortHierarchically(const FIKRigSkeleton& Skeleton)
{
	// sort offsets hierarchically so that they are applied in leaf to root order
	// when generating the component space retarget pose in the processor
	BoneRotationOffsets.KeySort([Skeleton](FName A, FName B)
	{
		return Skeleton.GetBoneIndexFromName(A) > Skeleton.GetBoneIndexFromName(B);
	});
}

const FIKRetargetOpBase* UIKRetargeter::GetRetargetOpByName(const FName& InOpName) const
{
	for (const FInstancedStruct& OpStruct : RetargetOps)
	{
		const FIKRetargetOpBase* Op = OpStruct.GetPtr<FIKRetargetOpBase>();
		if (Op->GetName() == InOpName)
		{
			return Op;
		}
	}
	
	return nullptr;
}

FIKRetargetOpBase* UIKRetargeter::GetRetargetOpByNameNonConst(const FName& InOpName)
{
	return const_cast<FIKRetargetOpBase*>(GetRetargetOpByName(InOpName));
}

int32 UIKRetargeter::GetIndexOfRetargetOp(const FIKRetargetOpBase* InOp) const
{
	for (int32 OpIndex=0; OpIndex < RetargetOps.Num(); OpIndex++)
	{
		const FInstancedStruct& OpStruct = RetargetOps[OpIndex];
		const FIKRetargetOpBase* Op = OpStruct.GetPtr<FIKRetargetOpBase>();
		if (Op==InOp)
		{
			return OpIndex;
		}
	}

	return INDEX_NONE;
}

const FIKRetargetPose* UIKRetargeter::GetCurrentRetargetPose(const ERetargetSourceOrTarget& SourceOrTarget) const
{
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? &SourceRetargetPoses[CurrentSourceRetargetPose] : &TargetRetargetPoses[CurrentTargetRetargetPose];
}

FName UIKRetargeter::GetCurrentRetargetPoseName(const ERetargetSourceOrTarget& SourceOrTarget) const
{
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? CurrentSourceRetargetPose : CurrentTargetRetargetPose;
}

const FIKRetargetPose* UIKRetargeter::GetRetargetPoseByName(
	const ERetargetSourceOrTarget& SourceOrTarget,
	const FName PoseName) const
{
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceRetargetPoses.Find(PoseName) : TargetRetargetPoses.Find(PoseName);
}

const FName UIKRetargeter::GetDefaultPoseName()
{
	static const FName DefaultPoseName = "Default Pose";
	return DefaultPoseName;
}

const FRetargetProfile* UIKRetargeter::GetCurrentProfile() const
{
	return GetProfileByName(CurrentProfile);
}

const FRetargetProfile* UIKRetargeter::GetProfileByName(const FName& ProfileName) const
{
	return Profiles.Find(ProfileName);
}

const TArray<FName>& UIKRetargeter::GetOverrideSetsToApply() const
{
	OverridesToApply.Reset();
	
	for (const TTuple<FName, FRetargetOverrideSet>& OverrideEntry : OverrideSets)
	{
		if (OverrideEntry.Value.bActiveByDefault)
		{
			OverridesToApply.AddUnique(OverrideEntry.Key);
		}
	}
	
	return OverridesToApply;
}

const FRetargetOverrideSet* UIKRetargeter::GetOverrideSetByName(const FName& OverrideName) const
{
	return OverrideSets.Find(OverrideName);
}

//
// BEGIN DEPRECATED API
//
PRAGMA_DISABLE_DEPRECATION_WARNINGS
const TObjectPtr<URetargetChainSettings> UIKRetargeter::GetChainMapByName(const FName& TargetChainName) const
{
	const TObjectPtr<URetargetChainSettings>* AllChainMapSettings = ChainSettings_DEPRECATED.FindByPredicate(
		[TargetChainName](const TObjectPtr<URetargetChainSettings>& AllChainMapSettings)
		{
			return AllChainMapSettings->TargetChain == TargetChainName;
		});
	
	return !AllChainMapSettings ? nullptr : AllChainMapSettings->Get();
}

const FTargetChainSettings* UIKRetargeter::GetChainSettingsByName(const FName& TargetChainName) const
{
	const TObjectPtr<URetargetChainSettings> AllChainMaps = GetChainMapByName(TargetChainName);
	if (AllChainMaps)
	{
		return &AllChainMaps->Settings;
	}

	return nullptr;
}

FTargetChainSettings UIKRetargeter::GetChainUsingGoalFromRetargetAsset(
	const UIKRetargeter* RetargetAsset,
	const FName IKGoalName)
{
	FTargetChainSettings EmptySettings;

	if (!RetargetAsset)
	{
		return EmptySettings;
	}

	const UIKRigDefinition* IKRig = RetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target);
	if (!IKRig)
	{
		return EmptySettings;
	}

	const TArray<FBoneChain>& RetargetChains = IKRig->GetRetargetChains();
	const FBoneChain* ChainWithGoal = nullptr;
	for (const FBoneChain& RetargetChain : RetargetChains)
	{
		if (RetargetChain.IKGoalName == IKGoalName)
		{
			ChainWithGoal = &RetargetChain;
			break;
		}
	}

	if (!ChainWithGoal)
	{
		return EmptySettings;
	}

	// found a chain using the specified goal, return a copy of it's settings
	const FTargetChainSettings* ChainSettings = RetargetAsset->GetChainSettingsByName(ChainWithGoal->ChainName);
	return ChainSettings ? *ChainSettings : EmptySettings;
}

FTargetChainSettings UIKRetargeter::GetChainSettingsFromRetargetAsset(
	const UIKRetargeter* RetargetAsset,
	const FName TargetChainName,
	const FName OptionalProfileName)
{
	FTargetChainSettings OutSettings;
	
	if (!RetargetAsset)
	{
		return OutSettings;
	}
	
	// optionally get the chain settings from a profile
	if (OptionalProfileName != NAME_None)
	{
		if (const FRetargetProfile* RetargetProfile = RetargetAsset->GetProfileByName(OptionalProfileName))
		{
			if (const FTargetChainSettings* ProfileChainSettings = RetargetProfile->ChainSettings.Find(TargetChainName))
			{
				return *ProfileChainSettings;
			}
		}

		// no profile with this chain found, return default settings
		return OutSettings;
	}
	
	// return the chain settings stored in the retargeter (if it has one matching specified name)
	if (const FTargetChainSettings* AssetChainSettings = RetargetAsset->GetChainSettingsByName(TargetChainName))
	{
		return *AssetChainSettings;
	}

	// no chain map with the given target chain, so return default settings
	return OutSettings;
}

FTargetChainSettings UIKRetargeter::GetChainSettingsFromRetargetProfile(
	FRetargetProfile& RetargetProfile,
	const FName TargetChainName)
{
	return RetargetProfile.ChainSettings.FindOrAdd(TargetChainName);
}

void UIKRetargeter::GetRootSettingsFromRetargetAsset(
	const UIKRetargeter* RetargetAsset,
	const FName OptionalProfileName,
	FTargetRootSettings& OutSettings)
{
	if (!RetargetAsset)
	{
		OutSettings = FTargetRootSettings();
		return;
	}
	
	// optionally get the root settings from a profile
	if (OptionalProfileName != NAME_None)
	{
		if (const FRetargetProfile* RetargetProfile = RetargetAsset->GetProfileByName(OptionalProfileName))
		{
			if (RetargetProfile->bApplyRootSettings)
			{
				OutSettings =  RetargetProfile->RootSettings;
				return;
			}
		}
		
		// could not find profile, so return default settings
		OutSettings = FTargetRootSettings();
		return;
	}

	// return the base root settings
	OutSettings =  RetargetAsset->GetRootSettingsUObject()->Settings;
}

FTargetRootSettings UIKRetargeter::GetRootSettingsFromRetargetProfile(FRetargetProfile& RetargetProfile)
{
	return RetargetProfile.RootSettings;
}

void UIKRetargeter::GetGlobalSettingsFromRetargetAsset(
	const UIKRetargeter* RetargetAsset,
	const FName OptionalProfileName,
	FRetargetGlobalSettings& OutSettings)
{
	if (!RetargetAsset)
	{
		OutSettings = FRetargetGlobalSettings();
		return;
	}
	
	// optionally get the root settings from a profile
	if (OptionalProfileName != NAME_None)
	{
		if (const FRetargetProfile* RetargetProfile = RetargetAsset->GetProfileByName(OptionalProfileName))
		{
			if (RetargetProfile->bApplyGlobalSettings)
			{
				OutSettings = RetargetProfile->GlobalSettings;
				return;
			}
		}
		
		// could not find profile, so return default settings
		OutSettings = FRetargetGlobalSettings();
		return;
	}

	// return the base root settings
	OutSettings = RetargetAsset->GetGlobalSettings();
}

FRetargetGlobalSettings UIKRetargeter::GetGlobalSettingsFromRetargetProfile(FRetargetProfile& RetargetProfile)
{
	return RetargetProfile.GlobalSettings;
}

void UIKRetargeter::SetGlobalSettingsInRetargetProfile(
	FRetargetProfile& RetargetProfile,
	const FRetargetGlobalSettings& GlobalSettings)
{
	RetargetProfile.GlobalSettings = GlobalSettings;
	RetargetProfile.bApplyGlobalSettings = true;
}

void UIKRetargeter::SetRootSettingsInRetargetProfile(
	FRetargetProfile& RetargetProfile,
	const FTargetRootSettings& RootSettings)
{
	RetargetProfile.RootSettings = RootSettings;
	RetargetProfile.bApplyRootSettings = true;
}

void UIKRetargeter::SetChainSettingsInRetargetProfile(
	FRetargetProfile& RetargetProfile,
	const FTargetChainSettings& ChainSettings,
	const FName TargetChainName)
{
	RetargetProfile.ChainSettings.Add(TargetChainName, ChainSettings);
	RetargetProfile.bApplyChainSettings = true;
}

void UIKRetargeter::SetChainFKSettingsInRetargetProfile(
	FRetargetProfile& RetargetProfile,
	const FTargetChainFKSettings& FKSettings,
	const FName TargetChainName)
{
	FTargetChainSettings& ChainSettings = RetargetProfile.ChainSettings.FindOrAdd(TargetChainName);
	ChainSettings.FK = FKSettings;
	RetargetProfile.bApplyChainSettings = true;
}

void UIKRetargeter::SetChainIKSettingsInRetargetProfile(
	FRetargetProfile& RetargetProfile,
	const FTargetChainIKSettings& IKSettings,
	const FName TargetChainName)
{
	FTargetChainSettings& ChainSettings = RetargetProfile.ChainSettings.FindOrAdd(TargetChainName);
	ChainSettings.IK = IKSettings;
	RetargetProfile.bApplyChainSettings = true;
}

void UIKRetargeter::SetChainSpeedPlantSettingsInRetargetProfile(
	FRetargetProfile& RetargetProfile,
	const FTargetChainSpeedPlantSettings& SpeedPlantSettings,
	const FName TargetChainName)
{
	FTargetChainSettings& ChainSettings = RetargetProfile.ChainSettings.FindOrAdd(TargetChainName);
	ChainSettings.SpeedPlanting = SpeedPlantSettings;
	RetargetProfile.bApplyChainSettings = true;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
//
// END DEPRECATED API
//
