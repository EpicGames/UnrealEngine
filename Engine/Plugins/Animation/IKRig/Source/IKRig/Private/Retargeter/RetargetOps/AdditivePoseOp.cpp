// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/AdditivePoseOp.h"

#include "Retargeter/IKRetargetProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AdditivePoseOp)

const UClass* FIKRetargetAdditivePoseOpSettings::GetControllerType() const
{
	return UIKRetargetAdditivePoseController::StaticClass();
}

void FIKRetargetAdditivePoseOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy all properties
	static TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetAdditivePoseOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

bool FIKRetargetAdditivePoseOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& Log)
{
	bIsInitialized = true;
	PelvisBoneName = InProcessor.GetPelvisBone(ERetargetSourceOrTarget::Target, ERetargetOpsToSearch::ProcessorOps);
	return true;
}

void FIKRetargetAdditivePoseOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	ApplyAdditivePose(InProcessor, OutTargetGlobalPose);
}

FIKRetargetOpSettingsBase* FIKRetargetAdditivePoseOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetAdditivePoseOp::GetSettingsType() const
{
	return FIKRetargetAdditivePoseOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetAdditivePoseOp::GetType() const
{
	return FIKRetargetAdditivePoseOp::StaticStruct();
}

void FIKRetargetAdditivePoseOp::ApplyAdditivePose(
	FIKRetargetProcessor& InProcessor,
	TArray<FTransform>& OutTargetGlobalPose)
{
	// verify that retarget pose exists in the retarget asset
	const UIKRetargeter* RetargetAsset = InProcessor.GetRetargetAsset();
	const FIKRetargetPose* RetargetPose = RetargetAsset->GetRetargetPoseByName(ERetargetSourceOrTarget::Target, Settings.PoseToApply);
	if (!RetargetPose)
	{
		return; // retarget pose not found
	}

	const FRetargetSkeleton& TargetSkeleton = InProcessor.GetSkeleton(ERetargetSourceOrTarget::Target);

	// apply pelvis translation offset
	const int32 PelvisBoneIndex = TargetSkeleton.FindBoneIndexByName(PelvisBoneName);
	if (PelvisBoneIndex != INDEX_NONE)
	{
		FTransform PelvisTransform = OutTargetGlobalPose[PelvisBoneIndex];
		PelvisTransform.AddToTranslation(RetargetPose->GetRootTranslationDelta());
		TargetSkeleton.SetGlobalTransformAndUpdateChildren(PelvisBoneIndex, PelvisTransform, OutTargetGlobalPose);
	}
	
	// NOTE: we could convert the entire global pose to a local pose, apply the offsets, and then convert it back to global space
	// BUT, for the majority of use cases, retarget poses only affect a small set of bones and so we are doing sparse updates.
	// If the retarget pose affects many bones, this could end up being slower due to SetGlobalTransformAndUpdateChildren()
	// We chose to optimize for the common case; if this ever shows up in a profile we could modify this to use a batch conversion if the
	// retarget pose modifies a high percentage of the bones in the skeleton.

	// apply retarget pose offsets (retarget pose is stored as offset relative to reference pose)
	for (const TTuple<FName, FQuat>& BoneDelta : RetargetPose->GetAllDeltaRotations())
	{
		const int32 BoneIndex = TargetSkeleton.FindBoneIndexByName(BoneDelta.Key);
		if (BoneIndex == INDEX_NONE)
		{
			// this can happen if a retarget pose recorded a bone offset for a bone that is not present in the
			// target skeleton; ie, the retarget pose was generated from a different Skeletal Mesh with extra bones
			continue;
		}

		// get local transform of bone
		FTransform LocalTransform = TargetSkeleton.GetLocalTransformOfSingleBone(BoneIndex, OutTargetGlobalPose);

		// apply local offset
		FQuat DeltaRotation = BoneDelta.Value;
		if (!FMath::IsNearlyEqual(Settings.Alpha, 1.0))
		{
			DeltaRotation = FQuat::FastLerp(FQuat::Identity, DeltaRotation, Settings.Alpha);
			DeltaRotation.Normalize();
		}
		LocalTransform.SetRotation(LocalTransform.GetRotation() * DeltaRotation);

		// convert back to global and update the output pose
		FTransform ParentGlobalTransform = FTransform::Identity;
		const int32 ParentIndex = TargetSkeleton.ParentIndices[BoneIndex];
		if (ParentIndex != INDEX_NONE)
		{
			ParentGlobalTransform = OutTargetGlobalPose[ParentIndex];
		}
		FTransform GlobalTransform = LocalTransform * ParentGlobalTransform;
		TargetSkeleton.SetGlobalTransformAndUpdateChildren(BoneIndex, GlobalTransform, OutTargetGlobalPose);
	}
}

FIKRetargetAdditivePoseOpSettings UIKRetargetAdditivePoseController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetAdditivePoseOpSettings*>(OpSettingsToControl);
}

void UIKRetargetAdditivePoseController::SetSettings(FIKRetargetAdditivePoseOpSettings InSettings)
{
	reinterpret_cast<FIKRetargetAdditivePoseOpSettings*>(OpSettingsToControl)->CopySettingsAtRuntime(&InSettings);
}

