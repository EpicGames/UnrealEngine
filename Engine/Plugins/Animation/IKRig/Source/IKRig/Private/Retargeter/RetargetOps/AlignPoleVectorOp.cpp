// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/AlignPoleVectorOp.h"

#include "Engine/SkeletalMesh.h"
#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/StrideWarpingOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AlignPoleVectorOp)

#define LOCTEXT_NAMESPACE "AlignPoleVectorOp"

bool FPoleVectorMatcher::Initialize(
	const FRetargetPoleVectorSettings* InSettings,
	const FResolvedBoneChain& InSourceBoneChain,
	const FResolvedBoneChain& InTargetBoneChain,
	const FRetargetSkeleton& InSourceSkeleton,
	const FRetargetSkeleton& InTargetSkeleton)
{
	Settings = InSettings;
	SourceBoneChain = &InSourceBoneChain;
	TargetBoneChain = &InTargetBoneChain;

	const TArray<FTransform>& SourceRetargetPose = InSourceSkeleton.RetargetPoses.GetGlobalRetargetPose();
	const TArray<FTransform>& TargetRetargetPose = InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	
	SourcePoleAxis = CalculateBestPoleAxisForChain(SourceBoneChain->BoneIndices, SourceRetargetPose);
	TargetPoleAxis = CalculateBestPoleAxisForChain(TargetBoneChain->BoneIndices, TargetRetargetPose);

	const FVector SourcePoleVector = CalculatePoleVector(SourcePoleAxis, SourceBoneChain->BoneIndices, SourceRetargetPose);
	const FVector TargetPoleVector = CalculatePoleVector(TargetPoleAxis, TargetBoneChain->BoneIndices, TargetRetargetPose);
	
	TargetToSourceAngularOffsetAtRefPose = static_cast<float>(FMath::Acos(FVector::DotProduct(TargetPoleVector, SourcePoleVector)));

	// cache indices of bones in chain (and their children recursively) excluding children of the end bone
	TArray<int32> AllChildrenIndices;
	InTargetSkeleton.GetChildrenIndicesRecursive(TargetBoneChain->BoneIndices[0],AllChildrenIndices);
	TArray<int32> ChildrenOfEndIndices = {TargetBoneChain->BoneIndices.Last()};
	InTargetSkeleton.GetChildrenIndicesRecursive(TargetBoneChain->BoneIndices.Last(),ChildrenOfEndIndices);
	AllChildrenWithinChain.Reset();
	for (int32 ChildIndex : AllChildrenIndices)
	{
		if (ChildrenOfEndIndices.Contains(ChildIndex))
		{
			continue;
		}
		AllChildrenWithinChain.Add(ChildIndex);
	}

	return true;
}

void FPoleVectorMatcher::MatchPoleVector(
	const TArray<FTransform> &SourceGlobalPose,
	const FRetargetSkeleton& TargetSkeleton,
	TArray<FTransform> &OutTargetGlobalPose)
{
	const bool bIsMatchingPoleVector = Settings->AlignAlpha > UE_KINDA_SMALL_NUMBER;
	const bool bIsOffsetingPoleVector = !FMath::IsNearlyZero(Settings->StaticAngularOffset);
	if (!(bIsMatchingPoleVector || bIsOffsetingPoleVector))
	{
		return;
	}
	
	// update local spaces of all bones within chain as they are passed in
	TArray<FTransform> AllChildrenInputLocalTransforms = TargetSkeleton.GetLocalTransformsOfMultipleBones(AllChildrenWithinChain, OutTargetGlobalPose);

	// normalized vector pointing from root to tip of chain
	const FVector TargetChainAxisNorm = GetChainAxisNormalized(TargetBoneChain->BoneIndices, OutTargetGlobalPose);

	// calculate rotation to match the target to the source pole vector
	FQuat MatchingRotation = FQuat::Identity;
	if (bIsMatchingPoleVector)
	{
		const FVector SourcePoleVector = CalculatePoleVector(SourcePoleAxis, SourceBoneChain->BoneIndices, SourceGlobalPose);
		const FVector TargetPoleVector = CalculatePoleVector(TargetPoleAxis, TargetBoneChain->BoneIndices, OutTargetGlobalPose);
	
		const float RotateTargetToSource = static_cast<float>(FMath::Acos(FVector::DotProduct(SourcePoleVector, TargetPoleVector)));
		const float MatchPoleAngle = RotateTargetToSource - (Settings->MaintainOffset ? TargetToSourceAngularOffsetAtRefPose : 0);
		
		MatchingRotation = FQuat(TargetChainAxisNorm, MatchPoleAngle);
		MatchingRotation = FQuat::FastLerp(FQuat::Identity, MatchingRotation, Settings->AlignAlpha).GetNormalized();
	}

	// manual offset rotation
	FQuat OffsetRotation = FQuat::Identity;
	if (bIsOffsetingPoleVector)
	{
		OffsetRotation = FQuat(TargetChainAxisNorm,FMath::DegreesToRadians(Settings->StaticAngularOffset));
	}

	// rotate the base of the chain to match the pole vectors
	FTransform& BaseOfChain = OutTargetGlobalPose[TargetBoneChain->BoneIndices[0]];
	BaseOfChain.SetRotation(MatchingRotation * OffsetRotation * BaseOfChain.GetRotation());

	// now propagate changes to children of the chain
	TargetSkeleton.UpdateGlobalTransformsOfMultipleBones(AllChildrenWithinChain, AllChildrenInputLocalTransforms,OutTargetGlobalPose);
}

EAxis::Type FPoleVectorMatcher::CalculateBestPoleAxisForChain(
	const TArray<int32>& InBoneIndices,
	const TArray<FTransform>& InGlobalPose)
{
	check(!InBoneIndices.IsEmpty());

	TArray<EAxis::Type> PreferredAxes;

	// handle 1-bone case, there is no "chain" to speak of just a single bone
	// so we arbitrarily pick an axis and move on... realistically, this feature is fairly meaningless for a single bone
	if (InBoneIndices.Num() == 1)
	{
		return EAxis::Y;
	}
	
	const FVector ChainOrigin = InGlobalPose[InBoneIndices[0]].GetLocation();
	const FVector ChainAxisNormal = (InGlobalPose[InBoneIndices.Last()].GetLocation() - ChainOrigin).GetSafeNormal();
	const EAxis::Type MostDifferentAxis = GetMostDifferentAxis(InGlobalPose[InBoneIndices[0]], ChainAxisNormal);
	return MostDifferentAxis;
}

FVector FPoleVectorMatcher::CalculatePoleVector(
	const EAxis::Type& PoleAxis,
	const TArray<int32>& BoneIndices,
	const TArray<FTransform>& GlobalPose)
{
	check(!BoneIndices.IsEmpty())

	const FVector ChainNormal = GetChainAxisNormalized(BoneIndices, GlobalPose);
	const FVector UnitPoleAxis = GlobalPose[BoneIndices[0]].GetUnitAxis(PoleAxis);
	const FVector PoleVector = FVector::VectorPlaneProject(UnitPoleAxis, ChainNormal);
	return PoleVector.GetSafeNormal();
}

EAxis::Type FPoleVectorMatcher::GetMostDifferentAxis(
	const FTransform& Transform,
	const FVector& InNormal)
{
	float MostDifferentDot = 2.0f;
	EAxis::Type MostDifferentAxis = EAxis::Y;
	const TArray<EAxis::Type> CardinalAxes = {EAxis::X, EAxis::Y, EAxis::Z};
	for (const EAxis::Type Axis : CardinalAxes)
	{
		const FVector AxisVector = Transform.GetUnitAxis(Axis);
		const float AbsAxisDotNormal = static_cast<float>(FMath::Abs(FVector::DotProduct(AxisVector, InNormal)));
		if (AbsAxisDotNormal < MostDifferentDot)
		{
			MostDifferentDot = AbsAxisDotNormal;
			MostDifferentAxis = Axis;
		}
	}

	return MostDifferentAxis;
}

FVector FPoleVectorMatcher::GetChainAxisNormalized(
	const TArray<int32>& BoneIndices,
	const TArray<FTransform>& GlobalPose)
{
	const FVector ChainOrigin = GlobalPose[BoneIndices[0]].GetLocation();
	const FVector ChainAxis = GlobalPose[BoneIndices.Last()].GetLocation() - ChainOrigin;
	return ChainAxis.GetSafeNormal();
}

bool FRetargetPoleVectorSettings::operator==(const FRetargetPoleVectorSettings& Other) const
{
	return bEnabled == Other.bEnabled
		&& FMath::IsNearlyEqualByULP(AlignAlpha,Other.AlignAlpha)
		&& FMath::IsNearlyEqualByULP(StaticAngularOffset, Other.StaticAngularOffset)
		&& MaintainOffset == Other.MaintainOffset;
}

void FIKRetargetAlignPoleVectorOpSettings::MergePoleVectorSettings(const FRetargetPoleVectorSettings& InSettingsToMerge)
{
	for (FRetargetPoleVectorSettings& ChainToAlign : ChainsToAlign)
	{
		if (ChainToAlign.TargetChainName == InSettingsToMerge.TargetChainName)
		{
			ChainToAlign = InSettingsToMerge;
			return;
		}
	}
	ChainsToAlign.Add(InSettingsToMerge);
}

const UClass* FIKRetargetAlignPoleVectorOpSettings::GetControllerType() const
{
	return UIKRetargetAlignPoleVectorController::StaticClass();
}

void FIKRetargetAlignPoleVectorOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the chains
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetAlignPoleVectorOpSettings, ChainsToAlign)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetAlignPoleVectorOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);

	// copy chains
	const FIKRetargetAlignPoleVectorOpSettings* NewSettings = reinterpret_cast<const FIKRetargetAlignPoleVectorOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->ChainsToAlign, ChainsToAlign);
}

bool FIKRetargetAlignPoleVectorOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;

	PoleVectorMatchers.Reset();

	// spin through all the mapped retarget bone chains and load them
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	for (const FRetargetPoleVectorSettings& ChainSettings : Settings.ChainsToAlign)
	{
		if (!ChainSettings.bEnabled)
		{
			continue; // if the enabled flag is toggled it will trigger reinit to make a PoleVectorMatcher
		}
		
		const FResolvedBoneChain* TargetBoneChain = BoneChains.GetResolvedBoneChainByName(
			ChainSettings.TargetChainName,
			ERetargetSourceOrTarget::Target,
			Settings.IKRigAsset);
		if (TargetBoneChain == nullptr)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("PoleVectorOpMissingChain", "Align Pole Vector Op: chain data is out of sync with IK Rig. Missing target chain, '{0}."),
			FText::FromName(ChainSettings.TargetChainName)));
			continue;
		}

		// which source chain was this target chain mapped to?
		const FName SourceChainName = ChainMapping.GetChainMappedTo(ChainSettings.TargetChainName, ERetargetSourceOrTarget::Target);
		const FResolvedBoneChain* SourceBoneChain = BoneChains.GetResolvedBoneChainByName(SourceChainName, ERetargetSourceOrTarget::Source);
		if (SourceBoneChain == nullptr)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("PoleVectorOpMissingSource", "Align Pole Vector Op: missing source chain. Cannot align a pole vector without a source chain, '{0}."),
			FText::FromName(ChainSettings.TargetChainName)));
			continue;
		}
		
		// initialize the mapped pair of source/target bone chains
		FPoleVectorMatcher PoleVectorMatcher;
		const bool bPoleVectorMatcherInitialized = PoleVectorMatcher.Initialize(
			&ChainSettings,
			*SourceBoneChain,
			*TargetBoneChain,
			InSourceSkeleton,
			InTargetSkeleton);
		if (!bPoleVectorMatcherInitialized)
		{
			InLog.LogWarning( FText::Format(
				LOCTEXT("PoleVectorOpFailedInit", "Align Pole Vector Op: failed to initialize pole matching for chain, '{0}', on Skeletal Mesh: '{1}'"),
				FText::FromName(ChainSettings.TargetChainName), FText::FromString(InTargetSkeleton.SkeletalMesh->GetName())));
			return false;
		}

		// store valid chain pair to be retargeted
		PoleVectorMatchers.Add(PoleVectorMatcher);
	}

	bIsInitialized = !PoleVectorMatchers.IsEmpty();
	return bIsInitialized;
}

void FIKRetargetAlignPoleVectorOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	const FTargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
	for (FPoleVectorMatcher& PoleVectorMatcher : PoleVectorMatchers)
	{
		PoleVectorMatcher.MatchPoleVector(InSourceGlobalPose, TargetSkeleton, OutTargetGlobalPose);
	}
}

void FIKRetargetAlignPoleVectorOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	// on initial setup, use the default source/target IK rigs
	const UIKRigDefinition* SourceIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source);
	const UIKRigDefinition* TargetIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target);
	ApplyIKRigs(SourceIKRig, TargetIKRig);

	// auto map
	static bool bForceRemap = true;
	ChainMapping.AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);
}

void FIKRetargetAlignPoleVectorOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	// get the newly assigned IK Rig
	const bool bAssignedSource = SourceOrTarget == ERetargetSourceOrTarget::Source;
	const UIKRigDefinition* SourceIKRig = bAssignedSource ? InIKRig : ChainMapping.GetIKRig(ERetargetSourceOrTarget::Source);
	const bool bAssignedTarget = SourceOrTarget == ERetargetSourceOrTarget::Target;
	const UIKRigDefinition* TargetIKRig = bAssignedTarget ? InIKRig : ChainMapping.GetIKRig(ERetargetSourceOrTarget::Target);
	ApplyIKRigs(SourceIKRig, TargetIKRig);

	// auto map, but don't force it (leave existing mappings alone)
	static bool bForceRemap = false;
	ChainMapping.AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);
}

FIKRetargetOpSettingsBase* FIKRetargetAlignPoleVectorOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetAlignPoleVectorOp::GetSettingsType() const
{
	return FIKRetargetAlignPoleVectorOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetAlignPoleVectorOp::GetType() const
{
	return FIKRetargetAlignPoleVectorOp::StaticStruct();
}

const UIKRigDefinition* FIKRetargetAlignPoleVectorOp::GetCustomTargetIKRig() const
{
	return Settings.IKRigAsset;
}

FRetargetChainMapping* FIKRetargetAlignPoleVectorOp::GetChainMapping()
{
	return &ChainMapping;
}

void FIKRetargetAlignPoleVectorOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	for (FRetargetPoleVectorSettings& ChainSettings : Settings.ChainsToAlign)
	{
		if (ChainSettings.TargetChainName == InOldChainName)
		{
			ChainSettings.TargetChainName = InNewChainName;
		}
	}
}

void FIKRetargetAlignPoleVectorOp::OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent)
{
	const UIKRigDefinition* SourceIKRig = ChainMapping.GetIKRig(ERetargetSourceOrTarget::Source);
	const UIKRigDefinition* TargetIKRig = Settings.IKRigAsset;
	ApplyIKRigs(SourceIKRig, TargetIKRig);
}

#if WITH_EDITOR
FText FIKRetargetAlignPoleVectorOp::GetWarningMessage() const
{
	if (!bIsInitialized && Settings.ChainsToAlign.IsEmpty())
	{
		return LOCTEXT("NoChainsAssigned", "Not initialized. No chains assigned.");
	}
	
	return FIKRetargetOpBase::GetWarningMessage();
}

void FIKRetargetAlignPoleVectorOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.ChainsToAlign, InChainName);
}

bool FIKRetargetAlignPoleVectorOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.ChainsToAlign, InChainName);
}

uint8* FIKRetargetAlignPoleVectorOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.ChainsToAlign, InChainName);
}
#endif

void FIKRetargetAlignPoleVectorOp::ApplyIKRigs(const UIKRigDefinition* InSourceIKRig, const UIKRigDefinition* InTargetIKRig)
{
	// store IK Rig
	Settings.IKRigAsset = InTargetIKRig;
	
	// update chain mapping
	ChainMapping.ReinitializeWithIKRigs(InSourceIKRig, InTargetIKRig);

	// update settings only if we have a valid mapping
	if (!ChainMapping.IsReady())
	{
		// don't remove settings, instead we want to preserve existing settings at least until the next valid rig is loaded
		return;
	}
	
	// get the required target chains
	const TArray<FBoneChain>& AllTargetChains = Settings.IKRigAsset->GetRetargetChains();
	TArray<FName> RequiredTargetChains;
	for (const FBoneChain& ChainToRetarget : AllTargetChains)
	{
		RequiredTargetChains.Add(ChainToRetarget.ChainName);
	}
	
	// remove chains that are not required
	Settings.ChainsToAlign.RemoveAll([&RequiredTargetChains](const FRetargetPoleVectorSettings& InChainSettings)
	{
		return !RequiredTargetChains.Contains(InChainSettings.TargetChainName);
	});

	// add any required chains not already present
	for (const FName RequiredTargetChain : RequiredTargetChains)
	{
		bool bFound = false;
		for (const FRetargetPoleVectorSettings& ChainToAlign : Settings.ChainsToAlign)
		{
			if (ChainToAlign.TargetChainName == RequiredTargetChain)
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			Settings.ChainsToAlign.Emplace(RequiredTargetChain);
		}
	}
}

FIKRetargetAlignPoleVectorOpSettings UIKRetargetAlignPoleVectorController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetAlignPoleVectorOpSettings*>(OpSettingsToControl);
}

void UIKRetargetAlignPoleVectorController::SetSettings(FIKRetargetAlignPoleVectorOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
