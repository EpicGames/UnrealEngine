// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/BlendToSourceOp.h"

#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"

#if WITH_EDITOR
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"
#endif

bool FIKRetargetBlendToSourceChainSettings::operator==(const FIKRetargetBlendToSourceChainSettings& Other) const
{
	return FMath::IsNearlyEqualByULP(BlendToSource, Other.BlendToSource)
		&& FMath::IsNearlyEqualByULP(ApplyPelvisOffset, Other.ApplyPelvisOffset)
		&& TranslationPerAxisAlpha.Equals(Other.TranslationPerAxisAlpha);
}

const UClass* FIKRetargetBlendToSourceOpSettings::GetControllerType() const
{
	return UIKRetargetBlendToSourceController::StaticClass();
}

void FIKRetargetBlendToSourceOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the chains
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetBlendToSourceOpSettings, Chains)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetBlendToSourceOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);

	// copy chains
	const FIKRetargetBlendToSourceOpSettings* NewSettings = reinterpret_cast<const FIKRetargetBlendToSourceOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->Chains, Chains);
}

void FIKChainToBlend::Run(
	FIKRigGoalContainer& InGoalContainer,
	const TArray<FTransform>& InSourceGlobalPose,
	const FIKRetargetBlendToSourceOpSettings& InOpSettings,
	const FIKRetargetPelvisMotionOp* InPelvisMotionOp) const
{
	if (Settings->BlendToSource <= UE_KINDA_SMALL_NUMBER)
	{
		return; // disabled on this chain
	}
	
	if (InOpSettings.Alpha <= UE_KINDA_SMALL_NUMBER)
	{
		return; // whole op disabled
	}

	FIKRigGoal* Goal = InGoalContainer.FindGoalByName(TargetBoneChain->IKGoalName);
	if (!ensure(Goal))
	{
		return; // goal should be validated at initialization, should not happen
	}

	// blend to source position
	const double FinalPositionAlpha = Settings->BlendToSource * Settings->TranslationAlpha * InOpSettings.Alpha;
	if (FinalPositionAlpha > UE_KINDA_SMALL_NUMBER)
	{
		// get source location to blend towards
		const int32 SourceEndBoneIndex = SourceBoneChain->BoneIndices.Last();
		FVector SourceLocation = InSourceGlobalPose[SourceEndBoneIndex].GetLocation();
		if (InPelvisMotionOp && Settings->ApplyPelvisOffset > UE_KINDA_SMALL_NUMBER)
		{
			const FVector WeightedPelvisOffset = InPelvisMotionOp->GetPelvisTranslationOffset() * InPelvisMotionOp->GetAffectIKWeightAsVector();
			SourceLocation += WeightedPelvisOffset * Settings->ApplyPelvisOffset;
		}

		// blend and apply to goal
		const FVector PerAxisWeight = FinalPositionAlpha * Settings->TranslationPerAxisAlpha;
		FVector& GoalPosition = Goal->Position;
		GoalPosition.X = FMath::Lerp(GoalPosition.X, SourceLocation.X, PerAxisWeight.X);
		GoalPosition.Y = FMath::Lerp(GoalPosition.Y, SourceLocation.Y, PerAxisWeight.Y);
		GoalPosition.Z = FMath::Lerp(GoalPosition.Z, SourceLocation.Z, PerAxisWeight.Z);
	}

	// blend to source rotation
	const double FinalRotationAlpha = InOpSettings.Alpha * Settings->RotationAlpha * Settings->BlendToSource;
	if (FinalRotationAlpha > UE_KINDA_SMALL_NUMBER)
	{
		// get relevant bone transforms
		const int32 SourceEndBoneIndex = SourceBoneChain->BoneIndices.Last();
		const FTransform& InitialSourceEndGlobal = SourceBoneChain->RefPoseGlobalTransforms.Last();
		const FTransform& InitialTargetEndGlobal = TargetBoneChain->RefPoseGlobalTransforms.Last();
		const FTransform& CurrentSourceEndGlobal = InSourceGlobalPose[SourceEndBoneIndex];
		
		// apply delta rotation from source input
		const FQuat CurrentSourceEndRotation = CurrentSourceEndGlobal.GetRotation();
		const FQuat SourceDeltaRotation = (CurrentSourceEndRotation * InitialSourceEndGlobal.GetRotation().Inverse()).GetNormalized();
		FQuat NewGoalRotation = SourceDeltaRotation * InitialTargetEndGlobal.GetRotation();

		// blend to source rotation
		NewGoalRotation = FQuat::FastLerp(Goal->Rotation.Quaternion(), NewGoalRotation, FinalRotationAlpha);
		NewGoalRotation.Normalize();

		// apply to goal
		Goal->Rotation = NewGoalRotation.Rotator();
	}

#if WITH_EDITOR
	{
		DebugOutputGoal = FTransform(Goal->Rotation.Quaternion(), Goal->Position);
		DebugSourceGoal = InSourceGlobalPose[SourceBoneChain->BoneIndices.Last()];
	}
#endif
}

#if WITH_EDITOR
void FIKChainToBlend::DebugDraw(FPrimitiveDrawInterface* InPDI, const FTransform& InComponentTransform, const double InDrawSize) const
{
	const FTransform Source = DebugSourceGoal * InComponentTransform;
	const FTransform Output = DebugOutputGoal * InComponentTransform;
	
	DrawWireSphere(
				InPDI,
				Source,
				FLinearColor::Blue,
				InDrawSize * 0.5f,
				12,
				SDPG_MAX,
				0.0f,
				0.001f,
				false);

	DrawWireSphere(
			InPDI,
			Output,
			FLinearColor::Yellow,
			InDrawSize,
			12,
			SDPG_MAX,
			0.0f,
			0.001f,
			false);

	DrawDashedLine(
					InPDI,
					Source.GetLocation(),
					Output.GetLocation(),
					FLinearColor::Black,
					1.0f,
					SDPG_MAX);
}
#endif

bool FIKRetargetBlendToSourceOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;
	
	// go through all chains to retarget and load them
	ChainsToBlend.Reset();
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (FIKRetargetBlendToSourceChainSettings& ChainSettings : Settings.Chains)
	{
		FName TargetChainName = ChainSettings.TargetChainName;
		const FResolvedBoneChain* SourceBoneChain = nullptr;
		const FResolvedBoneChain* TargetBoneChain = nullptr;
		if (!FIKRetargetRunIKRigOp::ValidateIKChain(
			TargetChainName,
			GoalContainer,
			BoneChains,
			this,
			InParentOp,
			InLog,
			SourceBoneChain /*out*/,
			TargetBoneChain /*out*/))
		{
			continue;
		}

		ChainsToBlend.Emplace(*SourceBoneChain, *TargetBoneChain, ChainSettings);
	}

	// cache pelvis op
	PelvisMotionOp = InProcessor.GetFirstRetargetOpOfType<FIKRetargetPelvisMotionOp>();

	bIsInitialized = true;
	return true;
}

void FIKRetargetBlendToSourceOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	if (Settings.Alpha <= UE_KINDA_SMALL_NUMBER)
	{
		return; // fast skip whole op
	}
	
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (const FIKChainToBlend& Chain : ChainsToBlend)
	{
		Chain.Run(GoalContainer, InSourceGlobalPose, Settings, PelvisMotionOp);
	}
}

FIKRetargetOpSettingsBase* FIKRetargetBlendToSourceOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetBlendToSourceOp::GetSettingsType() const
{
	return FIKRetargetBlendToSourceOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetBlendToSourceOp::GetType() const
{
	return FIKRetargetBlendToSourceOp::StaticStruct();
}

const UScriptStruct* FIKRetargetBlendToSourceOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

void FIKRetargetBlendToSourceOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetBlendToSourceOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetBlendToSourceOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	IKRetargetOpUtils::OnRetargetChainRenamed(Settings.Chains, InOldChainName, InNewChainName);
}

void FIKRetargetBlendToSourceOp::OnParentReinitPropertyEdited(
	const FIKRetargetOpBase& InParentOp,
	const FPropertyChangedEvent* InPropertyChangedEvent)
{
	RegenerateChainSettings(&InParentOp);	
}

void FIKRetargetBlendToSourceOp::RegenerateChainSettings(const FIKRetargetOpBase* InParentOp)
{
	// regenerate chain settings
	constexpr bool bSkipUnmappedChains = false;
	constexpr bool bSkipNonIKChains = true;
	IKRetargetOpUtils::SynchronizeChainSettingsWithIKRig(Settings.Chains, InParentOp, bSkipUnmappedChains, bSkipNonIKChains);
}

FIKRetargetBlendToSourceOpSettings UIKRetargetBlendToSourceController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetBlendToSourceOpSettings*>(OpSettingsToControl);
}

void UIKRetargetBlendToSourceController::SetSettings(FIKRetargetBlendToSourceOpSettings InSettings)
{
	reinterpret_cast<FIKRetargetBlendToSourceOpSettings*>(OpSettingsToControl)->CopySettingsAtRuntime(&InSettings);
}

#if WITH_EDITOR
void FIKRetargetBlendToSourceOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	for (const FIKChainToBlend& Chain : ChainsToBlend)
	{
		Chain.DebugDraw(InPDI, InComponentTransform, Settings.DebugDrawSize);
	}
}

void FIKRetargetBlendToSourceOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.Chains, InChainName);
}

bool FIKRetargetBlendToSourceOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.Chains, InChainName);
}

uint8* FIKRetargetBlendToSourceOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.Chains, InChainName);
}
#endif
