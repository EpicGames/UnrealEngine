// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/CopyBasePoseOp.h"

#include "IKRigObjectVersion.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CopyBasePoseOp)

#define LOCTEXT_NAMESPACE "CopyBasePoseOp"

const UClass* FIKRetargetCopyBasePoseOpSettings::GetControllerType() const
{
	return UIKRetargetCopyBasePoseController::StaticClass();
}

void FIKRetargetCopyBasePoseOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies all properties
	const TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetCopyBasePoseOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

void FIKRetargetCopyBasePoseOpSettings::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	FIKRetargetOpSettingsBase::PostLoad(InVersion);

	// custom serialization to handle conversion from deprecated CopyBasePoseRoot to CopyFromStart
	if (InVersion < FIKRigObjectVersion::CopyBasePoseConvertedToBoneReference)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CopyFromStart.BoneName = CopyBasePoseRoot_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

bool FIKRetargetCopyBasePoseOpSettings::Serialize(FArchive& Ar)
{
	return SerializeOpWithVersion(Ar, *this);
}

bool FIKRetargetCopyBasePoseOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& Log)
{
	bIsInitialized = true;
	
	// find the roots to copy from
	const FName RootBoneToCopy = Settings.CopyFromStart.BoneName;
	const int32 SourceRootBoneToCopyIndex = RootBoneToCopy == NAME_None ? 0 : InSourceSkeleton.FindBoneIndexByName(RootBoneToCopy);
	const int32 TargetRootBoneToCopyIndex = RootBoneToCopy == NAME_None ? 0 : InTargetSkeleton.FindBoneIndexByName(RootBoneToCopy);

	// warn if user specified a root bone but it wasn't found
	if (RootBoneToCopy != NAME_None)
	{
		if (SourceRootBoneToCopyIndex == INDEX_NONE)
		{
			Log.LogWarning( FText::Format(
				LOCTEXT("MissingSourceRootToCopyFrom", "The root bone to copy from, {0} was not found in the source mesh {1}"),
				FText::FromName(RootBoneToCopy), FText::FromString(InSourceSkeleton.SkeletalMesh->GetName())));
		}

		if (TargetRootBoneToCopyIndex == INDEX_NONE)
		{
			Log.LogWarning( FText::Format(
				LOCTEXT("MissingTargetRootToCopyFrom", "The root bone to copy from, {0} was not found in the target mesh {1}"),
				FText::FromName(RootBoneToCopy), FText::FromString(InSourceSkeleton.SkeletalMesh->GetName())));
		}
	}

	// gather list of all bones to exclude / filter out of the copy operation
	TArray<int32> BoneIndicesToExclude;
	for (const FBoneReference& BoneToExclude : Settings.BonesToExclude)
	{
		const int32 BoneToExcludeIndex = InTargetSkeleton.FindBoneIndexByName(BoneToExclude.BoneName);
		if (BoneToExcludeIndex != INDEX_NONE)
		{
			BoneIndicesToExclude.Add(BoneToExcludeIndex);
			InTargetSkeleton.GetChildrenIndicesRecursive(BoneToExcludeIndex, BoneIndicesToExclude);
		}
	}

	// update the mapping from source-to-target bones (by name)
	SourceToTargetBoneIndexMap.Reset();
	for (int32 SourceBoneIndex=0; SourceBoneIndex<InSourceSkeleton.BoneNames.Num(); ++SourceBoneIndex)
	{
		// filter out bones above the RootBoneToCopy
		if (SourceRootBoneToCopyIndex != INDEX_NONE &&
			SourceBoneIndex != SourceRootBoneToCopyIndex &&
			!InSourceSkeleton.IsParentOf(SourceRootBoneToCopyIndex, SourceBoneIndex))
		{
			continue;
		}
	
		// filter out bones not in the target skeleton
		const int32 TargetBoneIndex = InTargetSkeleton.FindBoneIndexByName(InSourceSkeleton.BoneNames[SourceBoneIndex]);
		if (TargetBoneIndex == INDEX_NONE)
		{
			continue;
		}

		// filter out excluded bones
		if (BoneIndicesToExclude.Contains(TargetBoneIndex))
		{
			continue;
		}
	
		// store map of source/target bones in common
		SourceToTargetBoneIndexMap.Emplace(TargetBoneIndex, SourceBoneIndex);
	}

	// cache list of children that need updated
	ChildrenToUpdate.Reset();
	const int32 LastBranchIndex = InTargetSkeleton.GetCachedEndOfBranchIndex(TargetRootBoneToCopyIndex);
	const bool bIsLeafBone = LastBranchIndex == INDEX_NONE;
	if (!bIsLeafBone)
	{
		for (int32 ChildBoneIndex = TargetRootBoneToCopyIndex + 1; ChildBoneIndex <= LastBranchIndex; ChildBoneIndex++)
		{
			if (SourceToTargetBoneIndexMap.Contains(ChildBoneIndex))
			{
				continue;
			}

			ChildrenToUpdate.Add(ChildBoneIndex);
		}
	}
	
	return true;
}

void FIKRetargetCopyBasePoseOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	// skip if disabled
	if (!Settings.bCopyBasePose)
	{
		return;
	}
	
	const FRetargetSkeleton& SourceSkeleton = InProcessor.GetSkeleton(ERetargetSourceOrTarget::Source);
	FTargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
	
	// copy source bones directly to the target (no retargeting)
	for (const TPair<int32, int32>& Pair : SourceToTargetBoneIndexMap)
	{
		const int32 SourceIndex = Pair.Value;
		const int32 TargetIndex = Pair.Key;

		// copy the pose in local space (this matches CopyPoseFromMesh behavior)
		const int32 TargetParentIndex = TargetSkeleton.GetParentIndex(TargetIndex);
		const FTransform TargetParentGlobal = TargetParentIndex != INDEX_NONE ? OutTargetGlobalPose[TargetParentIndex] : FTransform::Identity;
		const int32 SourceParentIndex = SourceSkeleton.GetParentIndex(SourceIndex);
		const FTransform SourceParentGlobal = SourceParentIndex != INDEX_NONE ? InSourceGlobalPose[SourceParentIndex] : FTransform::Identity;
		const FTransform& SourceGlobal = InSourceGlobalPose[SourceIndex];
		const FTransform& SourceLocal = SourceGlobal.GetRelativeTransform(SourceParentGlobal);

		// convert to global space and store
		OutTargetGlobalPose[TargetIndex] = SourceLocal * TargetParentGlobal;
	}

	// update children of RootBoneToCopy that were not in the source
	for (int32 TargetChildIndex : ChildrenToUpdate)
	{
		const int32 ParentIndex = TargetSkeleton.GetParentIndex(TargetChildIndex);
		const FTransform ParentGlobalTransform = ParentIndex != INDEX_NONE ? OutTargetGlobalPose[ParentIndex] : FTransform::Identity;
		const FTransform ChildLocalTransform = TargetSkeleton.InputLocalPose[TargetChildIndex];
		OutTargetGlobalPose[TargetChildIndex] = ChildLocalTransform * ParentGlobalTransform;
	}
	
	// update input local pose to reflect the newly copied pose
	// (this local pose is used when updating intermediate bones in subsequent retargeting ops)
	const int32 TargetRootBoneToCopyIndex = TargetSkeleton.FindBoneIndexByName(Settings.CopyFromStart.BoneName);
	const int32 ParentOfCopyRootIndex = TargetSkeleton.GetParentIndex(TargetRootBoneToCopyIndex);
	TargetSkeleton.UpdateLocalTransformsBelowBone(ParentOfCopyRootIndex, TargetSkeleton.InputLocalPose, OutTargetGlobalPose);
}

FIKRetargetOpSettingsBase* FIKRetargetCopyBasePoseOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetCopyBasePoseOp::GetSettingsType() const
{
	return FIKRetargetCopyBasePoseOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetCopyBasePoseOp::GetType() const
{
	return FIKRetargetCopyBasePoseOp::StaticStruct();
}

FIKRetargetCopyBasePoseOpSettings UIKRetargetCopyBasePoseController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetCopyBasePoseOpSettings*>(OpSettingsToControl);
}

void UIKRetargetCopyBasePoseController::SetSettings(FIKRetargetCopyBasePoseOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

void UIKRetargetCopyBasePoseController::SetCopyFromStart(const FName InBoneName) const
{
	FIKRetargetCopyBasePoseOpSettings* Settings = reinterpret_cast<FIKRetargetCopyBasePoseOpSettings*>(OpSettingsToControl);
	if (!Settings)
	{
		return;
	}

	Settings->CopyFromStart.BoneName = InBoneName;
}

FName UIKRetargetCopyBasePoseController::GetCopyFromStart() const
{
	FIKRetargetCopyBasePoseOpSettings* Settings = reinterpret_cast<FIKRetargetCopyBasePoseOpSettings*>(OpSettingsToControl);
	if (!Settings)
	{
		return NAME_None;
	}

	return Settings->CopyFromStart.BoneName;
}

void UIKRetargetCopyBasePoseController::AddBoneToExclude(const FName InBoneName) const
{
	FIKRetargetCopyBasePoseOpSettings* Settings = reinterpret_cast<FIKRetargetCopyBasePoseOpSettings*>(OpSettingsToControl);
	if (!Settings)
	{
		return;
	}

	Settings->BonesToExclude.Add(InBoneName);
}

TArray<FName> UIKRetargetCopyBasePoseController::GetBonesToExclude() const
{
	FIKRetargetCopyBasePoseOpSettings* Settings = reinterpret_cast<FIKRetargetCopyBasePoseOpSettings*>(OpSettingsToControl);
	if (!Settings)
	{
		return TArray<FName>();
	}

	// convert to FNames (FBoneReference has no BP support)
	TArray<FName> ExcludedBoneNames;
	for (const FBoneReference& BoneToExclude : Settings->BonesToExclude)
	{
		ExcludedBoneNames.Add(BoneToExclude.BoneName);
	}
	
	return ExcludedBoneNames;
}

void UIKRetargetCopyBasePoseController::ResetBonesToExclude() const
{
	FIKRetargetCopyBasePoseOpSettings* Settings = reinterpret_cast<FIKRetargetCopyBasePoseOpSettings*>(OpSettingsToControl);
	if (!Settings)
	{
		return;
	}

	Settings->BonesToExclude.Reset();
}

#undef LOCTEXT_NAMESPACE
