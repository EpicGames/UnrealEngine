// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/CurveRemapOp.h"

#include "Animation/AnimCurveUtils.h"
#include "Animation/AnimCurveTypes.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimNodeBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveRemapOp)

const UClass* FIKRetargetCurveRemapOpSettings::GetControllerType() const
{
	return UIKRetargetCurveRemapController::StaticClass();
}

void FIKRetargetCurveRemapOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies all properties
	const TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetCurveRemapOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

void FIKRetargetCurveRemapOp::AnimGraphPreUpdateMainThread(
	USkeletalMeshComponent& SourceMeshComponent,
    USkeletalMeshComponent& TargetMeshComponent)
{
	if (!IsEnabled())
	{
		return;
	}
	
	SourceCurves.Empty();
	
	// get the source curves out of the source anim instance
	const UAnimInstance* SourceAnimInstance = SourceMeshComponent.GetAnimInstance();
	if (!SourceAnimInstance)
	{
		return;
	}
	
	// Potential optimization/tradeoff: If we stored the curve results on the mesh component in non-editor scenarios, this would be
	// much faster (but take more memory). As it is, we need to translate the map stored on the anim instance.
	const TMap<FName, float>& AnimCurveList = SourceAnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve);
	UE::Anim::FCurveUtils::BuildUnsorted(SourceCurves, AnimCurveList);
}

void FIKRetargetCurveRemapOp::AnimGraphEvaluateAnyThread(FPoseContext& Output)
{
	if (!IsEnabled())
	{
		return;
	}
	
	FBlendedCurve& OutputCurves = Output.Curve;

	TMap<FName, float> CurvesToRemapValues;
	if (Settings.bCopyAllSourceCurves)
	{
		// copy curves over from anim instance
		if (GetTakeInputCurvesFromSourceAnimInstance())
		{
			OutputCurves.CopyFrom(SourceCurves);
		}
	}

	if (!GetTakeInputCurvesFromSourceAnimInstance())
	{
		// take a copy of the source curves we need to remap
		for (const FCurveRemapPair& CurveToRemap : Settings.CurvesToRemap)
		{
			bool bOutIsValid = false;
			const float SourceValue = OutputCurves.Get(CurveToRemap.SourceCurve, bOutIsValid);
			if (bOutIsValid)
			{
				CurvesToRemapValues.Add(CurveToRemap.SourceCurve, SourceValue);
			}
		}

		// clear the outputs if needed
		if (!Settings.bCopyAllSourceCurves)
		{
			OutputCurves.ForEachElement([&OutputCurves](const UE::Anim::FCurveElement& InCurveElement)
				{
					OutputCurves.Set(InCurveElement.Name, 0.0f);
				});
		}
	}

	// copy curves over with different names (remap)
	if (Settings.bRemapCurves)
	{
		FBlendedCurve RemapedCurves;
		if (GetTakeInputCurvesFromSourceAnimInstance())
		{
			for (const FCurveRemapPair& CurveToRemap : Settings.CurvesToRemap)
			{
				bool bOutIsValid = false;
				const float SourceValue = SourceCurves.Get(CurveToRemap.SourceCurve, bOutIsValid);
				if (bOutIsValid)
				{
					RemapedCurves.Add(CurveToRemap.TargetCurve, SourceValue);
				}
			}
		}
		else
		{
			for (const FCurveRemapPair& CurveToRemap : Settings.CurvesToRemap)
			{
				float* SourceValuePtr = CurvesToRemapValues.Find(CurveToRemap.SourceCurve);
				if (SourceValuePtr)
				{
					RemapedCurves.Add(CurveToRemap.TargetCurve, *SourceValuePtr);
				}
			}
		}
		
		OutputCurves.Combine(RemapedCurves);
	}
}

void FIKRetargetCurveRemapOp::ProcessAnimSequenceCurves(FIKRetargetCurvesOpBase::FCurveData InCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues,
	FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues) const
{
	FIKRetargetCurvesOpBase::FCurveData SourceCurveMetaData;
	FIKRetargetCurvesOpBase::FFrameValues SourceCurveFrameValues;
	TArray<FName> ValidSourceTargetNames;

	if (Settings.bRemapCurves)
	{
		// first copy the source curve data
		SourceCurveFrameValues.SetNum(InCurveFrameValues.Num());
		for (int32 Frame = 0; Frame < InCurveFrameValues.Num(); ++Frame)
		{
			SourceCurveFrameValues[Frame].Reserve(Settings.CurvesToRemap.Num());
		}
		SourceCurveMetaData.Names.Reserve(Settings.CurvesToRemap.Num());
		SourceCurveMetaData.Flags.Reserve(Settings.CurvesToRemap.Num());
		SourceCurveMetaData.Colors.Reserve(Settings.CurvesToRemap.Num());
		ValidSourceTargetNames.Reserve(Settings.CurvesToRemap.Num());
		int32 RemapCurveCounter = 0;

		for (const FCurveRemapPair& CurveToRemap : Settings.CurvesToRemap)
		{
			int32 FoundSourceIndex = InCurveMetaData.Names.IndexOfByKey(CurveToRemap.SourceCurve);
			if (FoundSourceIndex != INDEX_NONE)
			{
				SourceCurveMetaData.Names.Add(InCurveMetaData.Names[FoundSourceIndex]);
				SourceCurveMetaData.Flags.Add(InCurveMetaData.Flags[FoundSourceIndex]);
				SourceCurveMetaData.Colors.Add(InCurveMetaData.Colors[FoundSourceIndex]);
				for (int32 Frame = 0; Frame < InCurveFrameValues.Num(); ++Frame)
				{
					SourceCurveFrameValues[Frame].Add(InCurveFrameValues[Frame][FoundSourceIndex]);
				}
				ValidSourceTargetNames.Add(Settings.CurvesToRemap[RemapCurveCounter].TargetCurve);
			}
			RemapCurveCounter++;
		}
	}

	if (Settings.bCopyAllSourceCurves)
	{
		OutCurveFrameValues = MoveTemp(InCurveFrameValues);
		OutCurveMetaData = MoveTemp(InCurveMetaData);
	}
	else
	{
		OutCurveFrameValues = {};
		OutCurveFrameValues.SetNum(InCurveFrameValues.Num());
		for (int32 Frame = 0; Frame < InCurveFrameValues.Num(); ++Frame)
		{
			OutCurveFrameValues[Frame].Reserve(ValidSourceTargetNames.Num());
		}
		OutCurveMetaData = {};
		OutCurveMetaData.Names.Reserve(ValidSourceTargetNames.Num());
		OutCurveMetaData.Flags.Reserve(ValidSourceTargetNames.Num());
		OutCurveMetaData.Colors.Reserve(ValidSourceTargetNames.Num());
	}

	if (Settings.bRemapCurves)
	{
		int32 RemapCurveCounter = 0;
		for (const FName& TargetCurve : ValidSourceTargetNames)
		{
			int32 FoundTargetIndex = OutCurveMetaData.Names.IndexOfByKey(TargetCurve);
			if (FoundTargetIndex != INDEX_NONE)
			{
				// overwrite the target data with the source
				for (int32 Frame = 0; Frame < OutCurveFrameValues.Num(); ++Frame)
				{
					OutCurveFrameValues[Frame][FoundTargetIndex] = SourceCurveFrameValues[Frame][RemapCurveCounter];
				}
				OutCurveMetaData.Names[FoundTargetIndex] = TargetCurve;
				OutCurveMetaData.Flags[FoundTargetIndex] = SourceCurveMetaData.Flags[RemapCurveCounter];
				OutCurveMetaData.Colors[FoundTargetIndex] = SourceCurveMetaData.Colors[RemapCurveCounter];
			}
			else
			{
				// add the target data from the source
				for (int32 Frame = 0; Frame < OutCurveFrameValues.Num(); ++Frame)
				{
					OutCurveFrameValues[Frame].Add(SourceCurveFrameValues[Frame][RemapCurveCounter]);
				}
				OutCurveMetaData.Names.Add(TargetCurve);
				OutCurveMetaData.Flags.Add(SourceCurveMetaData.Flags[RemapCurveCounter]);
				OutCurveMetaData.Colors.Add(SourceCurveMetaData.Colors[RemapCurveCounter]);
			}

			RemapCurveCounter++;
		}
	}
}


FIKRetargetCurveRemapOpSettings UIKRetargetCurveRemapController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetCurveRemapOpSettings*>(OpSettingsToControl);
}

void UIKRetargetCurveRemapController::SetSettings(FIKRetargetCurveRemapOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}
