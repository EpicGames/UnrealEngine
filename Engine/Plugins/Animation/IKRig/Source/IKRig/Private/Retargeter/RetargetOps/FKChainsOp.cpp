// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/FKChainsOp.h"

#include "IKRigObjectVersion.h"
#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FKChainsOp)

#if WITH_EDITOR
#include "PrimitiveDrawInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"

IMPLEMENT_HIT_PROXY(HIKRetargetEditorChainProxy, HHitProxy);
#endif

#define LOCTEXT_NAMESPACE "FKChainsOp"

void FChainEncoderFK::Initialize(const FResolvedBoneChain* InBoneChain)
{
	check(InBoneChain);
	BoneChain = InBoneChain;
	CurrentGlobalTransforms = BoneChain->RefPoseGlobalTransforms;
	InBoneChain->CurrentLocalTransforms = BoneChain->RefPoseLocalTransforms;
}

void FChainEncoderFK::EncodePose(
	const FRetargetSkeleton& SourceSkeleton,
	const TArray<int32>& SourceBoneIndices,
    const TArray<FTransform> &InSourceGlobalPose)
{
	check(SourceBoneIndices.Num() == CurrentGlobalTransforms.Num());
	
	// copy the global input pose for the chain
	for (int32 ChainIndex=0; ChainIndex<SourceBoneIndices.Num(); ++ChainIndex)
	{
		const int32 BoneIndex = SourceBoneIndices[ChainIndex];
		CurrentGlobalTransforms[ChainIndex] = InSourceGlobalPose[BoneIndex];
	}
	
	FResolvedBoneChain::FillTransformsWithLocalSpaceOfChain(SourceSkeleton, InSourceGlobalPose, SourceBoneIndices, BoneChain->CurrentLocalTransforms);
	
	ChainParentCurrentGlobalTransform = BoneChain->ChainParentBoneIndex != INDEX_NONE ? InSourceGlobalPose[BoneChain->ChainParentBoneIndex] : FTransform::Identity;
}

void FChainEncoderFK::TransformCurrentChainTransforms(const FTransform& NewParentTransform)
{
	for (int32 ChainIndex=0; ChainIndex<CurrentGlobalTransforms.Num(); ++ChainIndex)
	{
		if (ChainIndex == 0)
		{
			CurrentGlobalTransforms[ChainIndex] = BoneChain->CurrentLocalTransforms[ChainIndex] * NewParentTransform;
		}
		else
		{
			CurrentGlobalTransforms[ChainIndex] = BoneChain->CurrentLocalTransforms[ChainIndex] * CurrentGlobalTransforms[ChainIndex-1];
		}
	}
}

void FChainDecoderCachedData::Initialize(
	const FResolvedBoneChain* InSourceBoneChain,
	const FResolvedBoneChain* InTargetBoneChain,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton)
{
	// reset all data
	*this = FChainDecoderCachedData();

	// cache data for decoding the chain pose
	for (int32 TargetChainIndex=0; TargetChainIndex<InTargetBoneChain->BoneIndices.Num(); ++TargetChainIndex)
	{
		// cache the source bone associated with each target bone in the chain
		const int32 SourceBoneChainIndex = InTargetBoneChain->GetEquivalentBoneInOtherChain(TargetChainIndex, *InSourceBoneChain);
		const int32 SourceBoneIndex = InSourceBoneChain->BoneIndices[SourceBoneChainIndex];
		SourceBoneIndices.Add(SourceBoneIndex);

		// calculate bone direction in SOURCE ref pose
		const FTransform& SourceInitialGlobal = InSourceSkeleton.RetargetPoses.GetGlobalRetargetPose()[SourceBoneIndex];
		const FTransform& SourceParentInitialGlobal = InSourceSkeleton.GetParentTransform(SourceBoneIndex, InSourceSkeleton.RetargetPoses.GetGlobalRetargetPose());
		const FVector SourceInitialPosition = SourceInitialGlobal.GetTranslation();
		const FVector SourceParentInitialPosition = SourceParentInitialGlobal.GetTranslation();
		const FVector SourceInitialDirection = (SourceInitialPosition - SourceParentInitialPosition).GetSafeNormal();
		// calculate bone direction in TARGET ref pose
		const int32 TargetBoneIndex = InTargetBoneChain->BoneIndices[TargetChainIndex];
		const FVector TargetInitialPosition = InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose()[TargetBoneIndex].GetTranslation();
		const FTransform& TargetParentInitialGlobal = InTargetSkeleton.GetParentTransform(TargetBoneIndex, InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose());
		const FVector TargetParentInitialPosition = TargetParentInitialGlobal.GetTranslation();
		const FVector TargetInitialDirection = (TargetInitialPosition - TargetParentInitialPosition).GetSafeNormal();
		// calculate FromTo rotation from source bone direction to target bone direction
		const FQuat RefPoseGlobalDeltaRotation = FQuat::FindBetweenNormals(SourceInitialDirection, TargetInitialDirection);
		// cache FromTo orientation delta in ref pose
		FromToGlobalRotationDeltas.Add(RefPoseGlobalDeltaRotation);

		// cache rest pose bone lengths
		SourceRefPoseLengths.Add(FMath::Max(KINDA_SMALL_NUMBER, (SourceInitialPosition - SourceParentInitialPosition).Size()));
		TargetRefPoseLengths.Add(FMath::Max(KINDA_SMALL_NUMBER, (TargetInitialPosition - TargetParentInitialPosition).Size()));
	}	
}

void FChainDecoderFK::Initialize(
	const FResolvedBoneChain* InSourceBoneChain,
	const FResolvedBoneChain* InTargetBoneChain,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton)
{
	check(InSourceBoneChain && InTargetBoneChain);
	BoneChain = InTargetBoneChain;
	CachedData.Initialize(InSourceBoneChain, InTargetBoneChain, InSourceSkeleton, InTargetSkeleton);
}

void FChainDecoderFK::InitializeIntermediateParentIndices(
	const int32 InRetargetRootBoneIndex,
	const int32 InChainRootBoneIndex,
	const FTargetSkeleton& InTargetSkeleton)
{
	IntermediateParentIndices.Reset();
	
	const TArray<bool>& RetargetedBonesMask = InTargetSkeleton.GetRetargetedBonesMask();

	int32 ParentBoneIndex = InTargetSkeleton.ParentIndices[InChainRootBoneIndex];
	while (true)
	{
		if (ParentBoneIndex < 0 || ParentBoneIndex == InRetargetRootBoneIndex)
		{
			break; // reached root of skeleton
		}

		if (RetargetedBonesMask[ParentBoneIndex])
		{
			break; // reached the start of another retargeted chain
		}

		IntermediateParentIndices.Add(ParentBoneIndex);
		ParentBoneIndex = InTargetSkeleton.ParentIndices[ParentBoneIndex];
	}

	Algo::Reverse(IntermediateParentIndices);
}

void FChainDecoderFK::DecodePose(
	const FIKRetargetPelvisMotionOp* PelvisMotionOp,
	const FRetargetFKChainSettings& Settings,
	const TArray<int32>& TargetBoneIndices,
    FChainEncoderFK& SourceChain,
    const TArray<FTransform>& InSourceGlobalPose,
    const FRetargetSkeleton& SourceSkeleton,
    const FTargetSkeleton& TargetSkeleton,
    TArray<FTransform> &InOutGlobalPose)
{
	check(TargetBoneIndices.Num() == BoneChain->Params.Num());

	// Before setting this chain pose, we need to ensure that any
	// intermediate (between chains) NON-retargeted parent bones have had their
	// global transforms updated.
	// 
	// For example, if this chain is retargeting a single head bone, AND the spine was
	// retargeted in the prior step, then the neck bones will need updating first.
	// Otherwise the neck bones will remain at their location prior to the spine update.
	UpdateIntermediateParents(TargetSkeleton,InOutGlobalPose);
	
	// transform entire source chain from it's root to match target's current root orientation (maintaining offset from retarget pose)
	// this ensures children are retargeted in a "local" manner free from skewing that will happen if source and target
	// become misaligned as can happen if parent chains were not retargeted
	FTransform SourceChainParentInitialDelta = SourceChain.BoneChain->ChainParentInitialGlobalTransform.GetRelativeTransform(BoneChain->ChainParentInitialGlobalTransform);
	FTransform TargetChainParentCurrentGlobalTransform = BoneChain->ChainParentBoneIndex == INDEX_NONE ? FTransform::Identity : InOutGlobalPose[BoneChain->ChainParentBoneIndex]; 
	FTransform SourceChainParentTransform = SourceChainParentInitialDelta * TargetChainParentCurrentGlobalTransform;
	// apply delta to the source chain's current transforms before transferring rotations to the target
	SourceChain.TransformCurrentChainTransforms(SourceChainParentTransform);

	// if FK retargeting has been disabled for this chain, then simply set it to the retarget pose
	if (!Settings.EnableFK)
	{
		// put the chain in the global ref pose (globally rotated by parent bone in it's currently retargeted state)
		TArray<FTransform> RefPoseGlobalTransforms;
		BoneChain->FillTransformsWithGlobalRetargetPoseOfChain(TargetSkeleton, InOutGlobalPose, TargetBoneIndices, RefPoseGlobalTransforms);
		
		for (int32 ChainIndex=0; ChainIndex<TargetBoneIndices.Num(); ++ChainIndex)
		{
			const int32 BoneIndex = TargetBoneIndices[ChainIndex];
			InOutGlobalPose[BoneIndex] = RefPoseGlobalTransforms[ChainIndex];
		}

		return;
	}

	// now retarget the pose of each bone in the chain, copying from source to target
	for (int32 ChainIndex=0; ChainIndex<TargetBoneIndices.Num(); ++ChainIndex)
	{
		FTransform SourceCurrentTransform;
		FTransform SourceInitialTransform;

		// retarget rotation
		const FQuat OutRotation = RetargetRotation(
			ChainIndex,
			Settings,
			TargetBoneIndices,
			TargetSkeleton,
			InOutGlobalPose,
			SourceChain,
			SourceCurrentTransform,
			SourceInitialTransform);

		// retarget position
		const FVector OutPosition = RetargetPosition(
			ChainIndex,
			OutRotation,
			PelvisMotionOp,
			Settings,
			TargetBoneIndices,
			SourceChain,
			InSourceGlobalPose,
			SourceSkeleton,
			TargetSkeleton,
			InOutGlobalPose,
			SourceCurrentTransform);

		// retarget scale
		const FVector OutScale = RetargetScale(ChainIndex, SourceCurrentTransform, SourceInitialTransform);
		
		// apply output transform
		const int32 BoneIndex = TargetBoneIndices[ChainIndex];
		InOutGlobalPose[BoneIndex] = FTransform(OutRotation, OutPosition, OutScale);
	}

	// apply match chain operation on-top of translated and rotated result
	if (Settings.RotationMode == EFKChainRotationMode::MatchChain ||
		Settings.RotationMode == EFKChainRotationMode::MatchScaledChain)
	{
		const bool bScaleSourceChain = Settings.RotationMode == EFKChainRotationMode::MatchScaledChain;
		FVector TargetChainOrigin = InOutGlobalPose[TargetBoneIndices[0]].GetTranslation();
		MatchChain(bScaleSourceChain, SourceChain, TargetBoneIndices, TargetChainOrigin, InOutGlobalPose);
	}

	// apply final blending between retarget pose of chain and newly retargeted pose
	BlendByAlpha(Settings, TargetBoneIndices, TargetSkeleton, InOutGlobalPose);
}

FQuat FChainDecoderFK::RetargetRotation(
	const int32 InChainIndex,
	const FRetargetFKChainSettings& Settings,
	const TArray<int32>& TargetBoneIndices,
	const FTargetSkeleton& TargetSkeleton,
	const TArray<FTransform>& InOutGlobalPose,
	FChainEncoderFK& SourceChain,
	FTransform& OutSourceCurrentTransform,
	FTransform& OutSourceInitialTransform) const
{
	// special case, copy in local space (assumes user knows the local axes are desirable)
	if (Settings.RotationMode == EFKChainRotationMode::CopyLocal)
	{
		// use the closest bone transform, cannot reliably interpolate local rotations
		const float Param = BoneChain->Params[InChainIndex];
		int32 ClosestBoneChainIndex = SourceChain.BoneChain->GetBoneClosestToParam(Param);
		OutSourceCurrentTransform = SourceChain.CurrentGlobalTransforms[ClosestBoneChainIndex];
		OutSourceInitialTransform = SourceChain.BoneChain->RefPoseGlobalTransforms[ClosestBoneChainIndex];

		// apply the local rotation relative to the target parent's global pose
		const int32 BoneIndex = TargetBoneIndices[InChainIndex];
		FQuat ParentGlobalRotation = FQuat::Identity;
		const int32 ParentIndex = TargetSkeleton.ParentIndices[BoneIndex];
		if (ParentIndex != INDEX_NONE)
		{
			ParentGlobalRotation = InOutGlobalPose[ParentIndex].GetRotation();
		}

		// calculate the local delta on the source
		const FQuat SourceCurrentLocalRotation = SourceChain.BoneChain->CurrentLocalTransforms[ClosestBoneChainIndex].GetRotation();
		const FQuat SourceInitialLocalRotation = SourceChain.BoneChain->RefPoseLocalTransforms[ClosestBoneChainIndex].GetRotation();
		const FQuat SourceLocalDelta = SourceCurrentLocalRotation * SourceInitialLocalRotation.Inverse();
		// add the delta to the target local ref rotation
		const FQuat TargetLocalRefRotation = BoneChain->RefPoseLocalTransforms[InChainIndex].GetRotation();
		
		// apply local space offset to global space
		FQuat OutRotation = ParentGlobalRotation * (SourceLocalDelta * TargetLocalRefRotation);
		return MoveTemp(OutRotation);
	}
	
	// get source current / initial transforms for this bone
	switch (Settings.RotationMode)
	{
		case EFKChainRotationMode::Interpolated:
		case EFKChainRotationMode::MatchChain:
		case EFKChainRotationMode::MatchScaledChain:
		{
			// get the initial and current transform of source chain at param
			// this is the interpolated transform along the chain
			const float Param = BoneChain->Params[InChainIndex];
			OutSourceCurrentTransform = SourceChain.BoneChain->GetTransformAtChainParam(SourceChain.CurrentGlobalTransforms, Param);
			OutSourceInitialTransform = SourceChain.BoneChain->GetTransformAtChainParam(SourceChain.BoneChain->RefPoseGlobalTransforms, Param);
			break;
		}
		case EFKChainRotationMode::OneToOne:
		{
			const int32 NumBonesInSourceChain = SourceChain.CurrentGlobalTransforms.Num();
			if (InChainIndex < NumBonesInSourceChain)
			{
				OutSourceCurrentTransform = SourceChain.CurrentGlobalTransforms[InChainIndex];
				OutSourceInitialTransform = SourceChain.BoneChain->RefPoseGlobalTransforms[InChainIndex];
			}else
			{
				OutSourceCurrentTransform = SourceChain.CurrentGlobalTransforms.Last();
				OutSourceInitialTransform = SourceChain.BoneChain->RefPoseGlobalTransforms.Last();
			}
			break;
		}
		case EFKChainRotationMode::OneToOneReversed:
		{
			const int32 NumBonesInSourceChain = SourceChain.CurrentGlobalTransforms.Num();
			const int32 NumBonesInTargetChain = TargetBoneIndices.Num();
			const int32 TargetStartIndex = FMath::Max(0, NumBonesInTargetChain - NumBonesInSourceChain);
			const int32 SourceStartIndex = FMath::Max(0,NumBonesInSourceChain - NumBonesInTargetChain);
				
			if (InChainIndex < TargetStartIndex)
			{
				OutSourceCurrentTransform = SourceChain.BoneChain->RefPoseGlobalTransforms[0];
				OutSourceInitialTransform = SourceChain.BoneChain->RefPoseGlobalTransforms[0];
			}
			else
			{
				const int32 SourceChainIndex = SourceStartIndex + (InChainIndex - TargetStartIndex);
				OutSourceCurrentTransform = SourceChain.CurrentGlobalTransforms[SourceChainIndex];
				OutSourceInitialTransform = SourceChain.BoneChain->RefPoseGlobalTransforms[SourceChainIndex];
			}
			break;
		}
		case EFKChainRotationMode::None:
		{
			// in order to induce no rotation on the FK chain, we rotate the chain rigidly from the root of the chain
			OutSourceInitialTransform = SourceChain.BoneChain->RefPoseGlobalTransforms[0];
			// use the current global space retarget pose as the "current" transform, so chain rotates with parent
			OutSourceCurrentTransform = SourceChain.BoneChain->RefPoseLocalTransforms[0] * SourceChain.ChainParentCurrentGlobalTransform;
			break;
		}
		case EFKChainRotationMode::CopyLocal:
		default:
			checkNoEntry();
		break;
	}
	
	// calculate rotation delta
	const FQuat SourceCurrentRotation = OutSourceCurrentTransform.GetRotation();
	const FQuat SourceInitialRotation = OutSourceInitialTransform.GetRotation();
	const FQuat RotationDelta = SourceCurrentRotation * SourceInitialRotation.Inverse();

	// apply delta to the initial target rotation
	const FTransform& TargetInitialTransform = BoneChain->RefPoseGlobalTransforms[InChainIndex];
	const FQuat TargetInitialRotation = TargetInitialTransform.GetRotation();
	FQuat Rotation = RotationDelta * TargetInitialRotation;
	return MoveTemp(Rotation);
}

FVector FChainDecoderFK::RetargetPosition(
	const int32 InChainIndex,
	const FQuat& NewRotation,
	const FIKRetargetPelvisMotionOp* PelvisMotionOp,
	const FRetargetFKChainSettings& Settings,
	const TArray<int32>& TargetBoneIndices,
	FChainEncoderFK& SourceChain,
	const TArray<FTransform>& InSourceGlobalPose,
	const FRetargetSkeleton& SourceSkeleton,
	const FTargetSkeleton& TargetSkeleton,
	TArray<FTransform>& InOutGlobalPose,
	const FTransform& InCurrentSourceGlobal)
{
	// the position this function generates
	FVector OutPosition;
	
	const int32 BoneIndex = TargetBoneIndices[InChainIndex];
	
	// calculate output POSITION based on translation mode setting
	FTransform ParentGlobalTransform = FTransform::Identity;
	const int32 ParentIndex = TargetSkeleton.ParentIndices[BoneIndex];
	if (ParentIndex != INDEX_NONE)
	{
		ParentGlobalTransform = InOutGlobalPose[ParentIndex];
	}
	
	switch (Settings.TranslationMode)
	{
	case EFKChainTranslationMode::None:
		{
			const FVector InitialLocalOffset = TargetSkeleton.RetargetPoses.GetLocalRetargetPose()[BoneIndex].GetTranslation();
			OutPosition = ParentGlobalTransform.TransformPosition(InitialLocalOffset);
			break;
		}
	case EFKChainTranslationMode::GloballyScaled:
		{
			FVector GlobalScale = PelvisMotionOp ? PelvisMotionOp->GetGlobalScaleVector() : FVector::OneVector;
			OutPosition = InCurrentSourceGlobal.GetTranslation() * GlobalScale;
			break;
		}
	case EFKChainTranslationMode::Absolute:
		{
			OutPosition = InCurrentSourceGlobal.GetTranslation();
			break;	
		}
	case EFKChainTranslationMode::StretchBoneLengthUniformly:
		{
			if (InChainIndex==0)
			{
				const FVector InitialLocalOffset = TargetSkeleton.RetargetPoses.GetLocalRetargetPose()[BoneIndex].GetTranslation();
				OutPosition = ParentGlobalTransform.TransformPosition(InitialLocalOffset);
				break;
			}
			// initial chain length
			double SourceChainLengthInitial = BoneChain->GetChainLength(SourceChain.BoneChain->RefPoseGlobalTransforms);
			double SourceChainLengthCurrent = BoneChain->GetChainLength(SourceChain.CurrentGlobalTransforms);
			double StretchRatio = SourceChainLengthInitial < UE_KINDA_SMALL_NUMBER ? 1.0f : SourceChainLengthCurrent / SourceChainLengthInitial;
			// stretch local translation
			const FVector InitialLocalOffset = TargetSkeleton.RetargetPoses.GetLocalRetargetPose()[BoneIndex].GetTranslation();
			OutPosition = ParentGlobalTransform.TransformPosition(InitialLocalOffset * StretchRatio);
			break;
		}
	case EFKChainTranslationMode::StretchBoneLengthNonUniformly:
		{
			const double Param = BoneChain->Params[InChainIndex];
			double StretchRatio = SourceChain.BoneChain->GetStretchAtParam( SourceChain.BoneChain->RefPoseGlobalTransforms, SourceChain.CurrentGlobalTransforms, Param);
			// stretch local translation
			const FVector InitialLocalOffset = TargetSkeleton.RetargetPoses.GetLocalRetargetPose()[BoneIndex].GetTranslation();
			OutPosition = ParentGlobalTransform.TransformPosition(InitialLocalOffset * StretchRatio);
			break;
		}
	case EFKChainTranslationMode::OrientAndScale:
		{
			const double TargetRefLength = CachedData.TargetRefPoseLengths[InChainIndex];
			const double SourceRefLength = CachedData.SourceRefPoseLengths[InChainIndex];

			// calculate current direction of source bone
			const int32 SourceBoneIndex = CachedData.SourceBoneIndices[InChainIndex];
			const FTransform& SourceCurrentGlobal = InSourceGlobalPose[SourceBoneIndex];
			const FTransform& SourceParentCurrentGlobal = SourceSkeleton.GetParentTransform(SourceBoneIndex, InSourceGlobalPose);
			const FVector SourceCurrentTranslation = SourceCurrentGlobal.GetTranslation() - SourceParentCurrentGlobal.GetTranslation();

			// calculate current stretch of source bone
			double SourceCurrentLength;
			FVector SourceCurrentNorm;
			SourceCurrentTranslation.ToDirectionAndLength(SourceCurrentNorm, SourceCurrentLength);

			// generate a translational offset for the target bone
			FVector TargetNewGlobalOffset = FVector::ZeroVector;

			// in the absence of a proper source and target bone length, we treat translational deltas as additive rather than scaled
			constexpr double ThresholdLength = 0.1f;
			if (TargetRefLength <= ThresholdLength || SourceRefLength <= ThresholdLength)
			{
				// additive offset
				const FVector SourceCurrentGlobalOffset = SourceCurrentNorm * (SourceCurrentLength - SourceRefLength);
				// add it to the current local offset
				const FVector InitialLocalOffset = TargetSkeleton.RetargetPoses.GetLocalRetargetPose()[BoneIndex].GetTranslation();
				const FVector TargetCurrentGlobalOffset = ParentGlobalTransform.TransformPosition(InitialLocalOffset) - ParentGlobalTransform.GetLocation();
				// add the source offset to the target
				TargetNewGlobalOffset = TargetCurrentGlobalOffset + SourceCurrentGlobalOffset;
			}
			else
			{
				// scaled offset
				const double SourceLengthScaleFactor = SourceCurrentLength / SourceRefLength;
				// calculate equivalently stretched target bone in direction of source
				TargetNewGlobalOffset = SourceCurrentNorm * (TargetRefLength * SourceLengthScaleFactor);
				// apply rotational delta from the ref pose to the translational offset
				const FQuat& RefPoseGlobalDeltaRotation = CachedData.FromToGlobalRotationDeltas[InChainIndex];
				const FQuat CurrentGlobalRotationPlusDelta = RefPoseGlobalDeltaRotation * NewRotation;
				const FQuat CurrentGlobalRotationDelta = CurrentGlobalRotationPlusDelta * NewRotation.Inverse();
				TargetNewGlobalOffset = CurrentGlobalRotationDelta.RotateVector(TargetNewGlobalOffset);
			}

			// apply target offset to the parent global position
			OutPosition = ParentGlobalTransform.GetLocation() + TargetNewGlobalOffset;
			break;
		}
	default:
		{
			const FVector InitialLocalOffset = TargetSkeleton.RetargetPoses.GetLocalRetargetPose()[BoneIndex].GetTranslation();
			OutPosition = ParentGlobalTransform.TransformPosition(InitialLocalOffset);
			break;
		}
	}

	return OutPosition;
}

FVector FChainDecoderFK::RetargetScale(	
	const int32 InChainIndex,
	const FTransform& InSourceCurrentTransform,
	const FTransform& InSourceInitialTransform) const
{
	// calculate output SCALE
	const FVector SourceCurrentScale = InSourceCurrentTransform.GetScale3D();
	const FVector SourceInitialScale = InSourceInitialTransform.GetScale3D();
	const FVector TargetInitialScale = BoneChain->RefPoseGlobalTransforms[InChainIndex].GetScale3D();
	FVector Scale = SourceCurrentScale + (TargetInitialScale - SourceInitialScale);
	return MoveTemp(Scale);
}

void FChainDecoderFK::MatchChain(
	const bool bScaleSourceChain,
	const FChainEncoderFK& SourceChain,
	const TArray<int32>& TargetBoneIndices,
	const FVector& TargetChainOrigin,
	TArray<FTransform>& InOutGlobalPose)
{
	// "MatchChain" mode assumes the interpolated rotations (and any translation mode) as a starting point.
	//
	// The "spline-IK-like" method used below generates swing rotations, which fix-up the interpolated rotations
	// such that they align the bone positions to lie on the source chain.
	//
	// Typically, spline-ik is not "twist aware" because aligning a joint chain with swing rotations does not twist,
	// but by using the interpolated rotations/translation as a starting place we have a full twist/bend/stretch.

	// match chain feature requires at least 3 bones to work
	if (TargetBoneIndices.Num() < 3 || SourceChain.BoneChain->BoneIndices.Num() < 3)
	{
		return;
	}
	
	// convert source chain into a linear spline
	TArray<FVector> SourceSplinePoints;
	for (const FTransform& SourceGlobalTransform : SourceChain.CurrentGlobalTransforms)
	{
		SourceSplinePoints.Add(SourceGlobalTransform.GetTranslation());
	}
		
	// translate chain points to origin of target chain
	FVector ChainOffset = TargetChainOrigin - SourceSplinePoints[0];
	for (FVector& SplinePoint : SourceSplinePoints)
	{
		SplinePoint += ChainOffset;
	}

	// scale chain to match target length
	if (bScaleSourceChain)
	{
		TArray <FTransform> ChainGlobalTransforms;
		ChainGlobalTransforms.Reset(TargetBoneIndices.Num());
		for (int32 BoneIndex : TargetBoneIndices)
		{
			ChainGlobalTransforms.Add(InOutGlobalPose[BoneIndex]);
		}
		
		double TargetChainLengthCurrent = BoneChain->GetChainLength(ChainGlobalTransforms);
		double SourceChainLengthCurrent = BoneChain->GetChainLength(SourceChain.CurrentGlobalTransforms);
		double ScaleFactor = TargetChainLengthCurrent < UE_KINDA_SMALL_NUMBER ? 1.0f : SourceChainLengthCurrent / TargetChainLengthCurrent;
		for (FVector& SplinePoint : SourceSplinePoints)
		{
			SplinePoint =  TargetChainOrigin + (SplinePoint - TargetChainOrigin) * ScaleFactor;
		}
	}
	
	// snap orient each bone to lie on the source spline
	int32 OriginPointIndex = 0;
	double OriginPointAlpha = 0.0;
	for (int32 ChainIndex=0; ChainIndex<TargetBoneIndices.Num() - 1; ++ChainIndex)
	{
		// generate an aim rotation from the current joint vector to the point on the spline
		int32 BoneIndex = TargetBoneIndices[ChainIndex];
		int32 NextBoneIndex = TargetBoneIndices[ChainIndex + 1];
		const FVector StartBonePosition = InOutGlobalPose[BoneIndex].GetTranslation();
		const FVector EndBonePosition = InOutGlobalPose[NextBoneIndex].GetTranslation();
		const FVector EndBoneLocalPosition = InOutGlobalPose[BoneIndex].InverseTransformPosition(EndBonePosition);
		FVector BoneNorm;
		double BoneLength;
		(EndBonePosition - StartBonePosition).ToDirectionAndLength(BoneNorm, BoneLength);

		// get the first point along the spline that is "BoneLength" away from the joint location in a straight line
		GetPointOnSplineDistanceFromPoint(
			SourceSplinePoints,
			OriginPointIndex,
			OriginPointAlpha,
			BoneLength,
			OriginPointIndex,
			OriginPointAlpha);
		
		// convert spline coordinates to euclidean
		const FVector AimPointOnSpline = GetPointOnSplineFromIndexAndAlpha(SourceSplinePoints, OriginPointIndex, OriginPointAlpha);
		
		// generate from/to swing rotation to align child with spline
		const FVector AimNorm = (AimPointOnSpline - StartBonePosition).GetSafeNormal();
		const FQuat RotationOffset = FQuat::FindBetweenNormals(BoneNorm, AimNorm);

		// rotate bone to contact spline
		InOutGlobalPose[BoneIndex].SetRotation( RotationOffset * InOutGlobalPose[BoneIndex].GetRotation());

		// propagate translational offset of end bone to all it's children
		const FVector NewEndBonePosition = InOutGlobalPose[BoneIndex].TransformPosition(EndBoneLocalPosition);
		const FVector DeltaTranslation = NewEndBonePosition - EndBonePosition;
		for (int32 ChildChainIndex=ChainIndex+1; ChildChainIndex<TargetBoneIndices.Num(); ++ChildChainIndex)
		{
			int32 ChildBoneIndex = TargetBoneIndices[ChildChainIndex];
			InOutGlobalPose[ChildBoneIndex].AddToTranslation(DeltaTranslation);
		}
	}
}

void FChainDecoderFK::BlendByAlpha(
	const FRetargetFKChainSettings& Settings,
	const TArray<int32>& TargetBoneIndices,
	const FTargetSkeleton& TargetSkeleton,
	TArray<FTransform>& InOutGlobalPose)
{
	// apply final blending between retarget pose of chain and newly retargeted pose
	// blend must be done in local space, so we do it in a separate loop after full chain pose is generated
	const bool bShouldBlendRotation = !FMath::IsNearlyEqual(Settings.RotationAlpha, 1.0f);
	const bool bShouldBlendTranslation = !FMath::IsNearlyEqual(Settings.TranslationAlpha, 1.0f);
	if (!bShouldBlendRotation && !bShouldBlendTranslation)
	{
		return; // skipped if the alphas are near 1.0 (no blending needed)
	}

	// generate local space pose of chain
	BoneChain->FillTransformsWithLocalSpaceOfChain(TargetSkeleton, InOutGlobalPose, TargetBoneIndices, BoneChain->CurrentLocalTransforms);

	// blend each bone in chain with the retarget pose
	for (int32 ChainIndex=0; ChainIndex<BoneChain->RefPoseLocalTransforms.Num(); ++ChainIndex)
	{
		// blend between current local pose and initial local pose
		FTransform& NewLocalTransform = BoneChain->CurrentLocalTransforms[ChainIndex];
		const FTransform& RefPoseLocalTransform = BoneChain->RefPoseLocalTransforms[ChainIndex];
		NewLocalTransform.SetTranslation(FMath::Lerp(RefPoseLocalTransform.GetTranslation(), NewLocalTransform.GetTranslation(), Settings.TranslationAlpha));
		NewLocalTransform.SetRotation(FQuat::FastLerp(RefPoseLocalTransform.GetRotation(), NewLocalTransform.GetRotation(), Settings.RotationAlpha).GetNormalized());

		// put blended transforms back in global space and store in final output pose
		const int32 BoneIndex = TargetBoneIndices[ChainIndex];
		const int32 ParentIndex = TargetSkeleton.ParentIndices[BoneIndex];
		const FTransform& ParentGlobalTransform = ParentIndex == INDEX_NONE ? FTransform::Identity : InOutGlobalPose[ParentIndex];
		InOutGlobalPose[BoneIndex] = NewLocalTransform * ParentGlobalTransform;
	}
}

FVector FChainDecoderFK::GetPointOnSplineFromIndexAndAlpha(
	const TArray<FVector>& InSplinePoints,
	const int32 InPointIndex,
	const double InSegmentAlpha)
{
	if (!ensure(InPointIndex>=0))
	{
		return InSplinePoints[0];
	}
	
	if (InPointIndex == InSplinePoints.Num() - 1)
	{
		// extrapolate SegmentAlpha distance beyond the last segment
		const FVector& LastSegmentStart = InSplinePoints[InSplinePoints.Num() - 2];
		const FVector& LastSegmentEnd = InSplinePoints.Last();
		const FVector LastSegmentDirection = (LastSegmentEnd - LastSegmentStart).GetSafeNormal();
		return LastSegmentEnd + LastSegmentDirection * InSegmentAlpha;
	}

	// interpolate between point and next point by alpha
	return FMath::Lerp(InSplinePoints[InPointIndex], InSplinePoints[InPointIndex+1], InSegmentAlpha);
}

void FChainDecoderFK::GetPointOnSplineDistanceFromPoint(
	const TArray<FVector>& InSplinePoints,
	const int32 InOriginPointIndex,
	const double InOriginPointAlpha,
	const double InTargetDistanceFromOrigin,
	int32& OutPointIndex,
	double& OutPointAlpha)
{
	// NOTE: this returns the point on the ray, in RayDirection, that is DistanceFromPoint away from Point
	// It assumes that there is a point on the ray within TargetDistanceFromPointToRay
	auto GetPointOnRayDistanceFromPoint = [](
		const FVector& RayStart,
		const FVector& RayDirection,
		const FVector& Point,
		const double TargetDistanceFromPointToRay)
	{
		const FVector RayStartToPoint = Point - RayStart;
		const double ProjectionLength = FVector::DotProduct(RayStartToPoint, RayDirection);

		// get the point projected onto the line formed by the ray
		const FVector ProjectedPoint = RayStart + (RayDirection * ProjectionLength);
		const double DistancePointToRay = (ProjectedPoint - Point).Size();

		// check if point is further away from ray origin than TargetDistanceFromPointToRay (should not happen)
		if (!ensure(DistancePointToRay < TargetDistanceFromPointToRay))
		{
			return (RayStart - Point).GetClampedToMaxSize(TargetDistanceFromPointToRay);
		}
	
		// pythagorean theorem to find distance from projected point to point on ray
		// NOTE: imagine the right angle triangle formed by Point, ProjectedPoint and the unknown point on the ray
		const double DistanceFromProjPointToTargetPoint = FMath::Sqrt ((TargetDistanceFromPointToRay * TargetDistanceFromPointToRay) - (DistancePointToRay * DistancePointToRay));

		// calculate the point on the ray that is DistanceFromPoint away from Point
		return ProjectedPoint + RayDirection * DistanceFromProjPointToTargetPoint;
	};

	// convert spline coordinates to euclidean
	const FVector OriginPoint = GetPointOnSplineFromIndexAndAlpha(InSplinePoints, InOriginPointIndex, InOriginPointAlpha);
	
	// iterate down the chain until we find a bone that is beyond TargetDistanceFromOrigin away from the origin point
	for (OutPointIndex = InOriginPointIndex; OutPointIndex < InSplinePoints.Num()-1; ++OutPointIndex)
	{
		const FVector& SegmentEnd = InSplinePoints[OutPointIndex+1];
		const double DistanceOriginToSegmentEnd = (SegmentEnd - OriginPoint).Size();
		if (DistanceOriginToSegmentEnd < InTargetDistanceFromOrigin)
		{
			continue;
		}
		
		const FVector& RayStart = InSplinePoints[OutPointIndex];
		FVector RayDirection;
		double SegmentLength;
		(SegmentEnd - RayStart).ToDirectionAndLength(RayDirection, SegmentLength);
		const FVector PointOnRay = GetPointOnRayDistanceFromPoint(RayStart, RayDirection, OriginPoint, InTargetDistanceFromOrigin);
		OutPointAlpha = (PointOnRay - RayStart).Size() / SegmentLength;
		return;
	}

	// spline is too short, so extrapolate the last spline segment and find the point on that segment that is DistanceFromOrigin away
	const FVector& RayStart = InSplinePoints.Last();
	const FVector RayDirection = (InSplinePoints.Last() - InSplinePoints[InSplinePoints.Num() - 2]).GetSafeNormal();
	const FVector ExtrapolatedPointOnRay = GetPointOnRayDistanceFromPoint(RayStart, RayDirection, OriginPoint, InTargetDistanceFromOrigin);
	OutPointAlpha = (ExtrapolatedPointOnRay - RayStart).Size();
	OutPointIndex = InSplinePoints.Num() - 1;
}

void FChainDecoderFK::UpdateIntermediateParents(
	const FTargetSkeleton& TargetSkeleton,
	TArray<FTransform>& InOutGlobalPose)
{
	for (int32 IntermediateIndex=0; IntermediateIndex<IntermediateParentIndices.Num(); ++IntermediateIndex)
	{
		const int32 ParentIndex = IntermediateParentIndices[IntermediateIndex];
		const FTransform& ParentLocalTransform = IntermediateLocalTransforms[IntermediateIndex];
		TargetSkeleton.UpdateGlobalTransformOfSingleBone(ParentIndex, ParentLocalTransform, InOutGlobalPose);
	}
}

void FChainDecoderFK::UpdateIntermediateLocalTransforms(
	const FTargetSkeleton& TargetSkeleton,
	const TArray<FTransform>& InOutGlobalPose)
{
	IntermediateLocalTransforms.Reset(IntermediateParentIndices.Num());
	for (const int32& ParentIndex : IntermediateParentIndices)
	{
		const FTransform LocalTransform = TargetSkeleton.GetLocalTransformOfSingleBone(ParentIndex, InOutGlobalPose);
		IntermediateLocalTransforms.Add(LocalTransform);
	}
}

bool FChainPairFK::Initialize(
	const FResolvedBoneChain& InSourceBoneChain,
	const FResolvedBoneChain& InTargetBoneChain,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FRetargetFKChainSettings& InSettings,
	const FIKRigLogger& InLog)
{
	// bail out unless both chains were successfully resolved on the runtime skeleton
	if (!(InSourceBoneChain.IsValid() && InTargetBoneChain.IsValid()))
	{
		InLog.LogWarning( FText::Format(
			LOCTEXT("FKChainInvalid", "FK Chain Op unable to retarget source chain, {0} to target chain, {1}.'"),
			FText::FromName(InSourceBoneChain.ChainName), FText::FromName(InTargetBoneChain.ChainName)));
		return false;
	}
	
	SourceBoneChain = &InSourceBoneChain;
	TargetBoneChain = &InTargetBoneChain;
	Settings = &InSettings;
	
	FKEncoder.Initialize(SourceBoneChain);
	FKDecoder.Initialize(SourceBoneChain, TargetBoneChain, InSourceSkeleton, InTargetSkeleton);

	return true;
}

bool FRetargetFKChainSettings::operator==(const FRetargetFKChainSettings& Other) const
{
	return EnableFK == Other.EnableFK
		&& RotationMode == Other.RotationMode
		&& FMath::IsNearlyEqualByULP(RotationAlpha,Other.RotationAlpha)
		&& TranslationMode == Other.TranslationMode
		&& FMath::IsNearlyEqualByULP(TranslationAlpha, Other.TranslationAlpha);
}

const UClass* FIKRetargetFKChainsOpSettings::GetControllerType() const
{
	return UIKRetargetFKChainsController::StaticClass();
}

void FIKRetargetFKChainsOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the chains
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetFKChainsOpSettings, ChainsToRetarget)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetFKChainsOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);

	// copy chains
	const FIKRetargetFKChainsOpSettings* NewSettings = reinterpret_cast<const FIKRetargetFKChainsOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->ChainsToRetarget, ChainsToRetarget);
}

bool FIKRetargetFKChainsOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;

	ChainPairsFK.Reset();

	// spin through all the mapped retarget bone chains and load them
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	for (const FRetargetFKChainSettings& ChainSettings : Settings.ChainsToRetarget)
	{
		FName TargetChainName = ChainSettings.TargetChainName;
		const FResolvedBoneChain* TargetBoneChain = BoneChains.GetResolvedBoneChainByName(
			TargetChainName,
			ERetargetSourceOrTarget::Target,
			Settings.IKRigAsset);
		if (TargetBoneChain == nullptr)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("FKChainOpMissingChain", "FK Chain Op: chain data is out of sync with IK Rig. Missing target chain, '{0}."),
			FText::FromName(TargetChainName)));
			continue;
		}

		// which source chain was this target chain mapped to?
		const FName SourceChainName = Settings.ChainMapping.GetChainMappedTo(ChainSettings.TargetChainName, ERetargetSourceOrTarget::Target);
		const FResolvedBoneChain* SourceBoneChain = BoneChains.GetResolvedBoneChainByName(SourceChainName, ERetargetSourceOrTarget::Source);
		if (SourceBoneChain == nullptr)
		{
			// this target chain is not mapped to anything (don't spam user about it)
			continue;
		}
		
		// initialize the mapped pair of source/target bone chains
		FChainPairFK ChainPair;
		const bool bChainInitialized = ChainPair.Initialize(
			*SourceBoneChain,
			*TargetBoneChain,
			InSourceSkeleton,
			InTargetSkeleton,
			ChainSettings,
			InLog);
		if (!bChainInitialized)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("FKChainOpBadChain", "FK Chain Op: could not initialize a mapped retarget chain, '{0}."), FText::FromName(TargetChainName)));
			continue;
		}

		// store valid chain pair to be retargeted
		ChainPairsFK.Add(ChainPair);
	}

	// sort the chains based on their StartBone's index
	auto ChainsSorter = [this](const FChainPairFK& A, const FChainPairFK& B)
	{
		const int32 IndexA = A.TargetBoneChain->BoneIndices.Num() > 0 ? A.TargetBoneChain->BoneIndices[0] : INDEX_NONE;
		const int32 IndexB = B.TargetBoneChain->BoneIndices.Num() > 0 ? B.TargetBoneChain->BoneIndices[0] : INDEX_NONE;
		if (IndexA == IndexB)
		{
			// fallback to sorting alphabetically
			return A.TargetBoneChain->ChainName.LexicalLess(B.TargetBoneChain->ChainName);
		}
		return IndexA < IndexB;
	};
	ChainPairsFK.Sort(ChainsSorter);

	// gather all children bones that need updating after FK chains are retargeted
	const FName PelvisBone = InProcessor.GetPelvisBone(ERetargetSourceOrTarget::Target, ERetargetOpsToSearch::ProcessorOps);
	const int32 PelvisBoneIndex = InTargetSkeleton.FindBoneIndexByName(PelvisBone);
	auto IsBoneRetargeted = [this, PelvisBoneIndex](const int32 InBoneIndex)
	{
		if (PelvisBoneIndex == InBoneIndex)
		{
			return true; // never update the pelvis
		}
		
		for (const FChainPairFK& ChainPair : ChainPairsFK)
		{
			if (ChainPair.TargetBoneChain->BoneIndices.Contains(InBoneIndex))
			{
				return true;
			}
		}
		
		return false;
	};
	NonRetargetedChildrenToUpdate.Reset();
	for (const FChainPairFK& ChainPair : ChainPairsFK)
	{
		for (int32 ChainBone : ChainPair.TargetBoneChain->BoneIndices)
		{
			TArray<int32> AllChildren;
			InTargetSkeleton.GetChildrenIndicesRecursive(ChainBone, AllChildren);
			for (int32 ChildBoneIndex : AllChildren)
			{
				if (!IsBoneRetargeted(ChildBoneIndex))
				{
					NonRetargetedChildrenToUpdate.AddUnique(ChildBoneIndex);
				}
			}
		}
	}

	// consider initialized if at least 1 pair of bone chains were initialized
	bIsInitialized = !ChainPairsFK.IsEmpty();
	return bIsInitialized;
}

void FIKRetargetFKChainsOp::Run(
	FIKRetargetProcessor& Processor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	FTargetSkeleton& TargetSkeleton = Processor.GetTargetSkeleton();

	// update the local transforms of intermediate joints (in case prior op modified them)
	for (FChainPairFK& ChainPair : ChainPairsFK)
	{
		ChainPair.FKDecoder.UpdateIntermediateLocalTransforms(TargetSkeleton, OutTargetGlobalPose);
	}

	// update local transforms of all the non-retargeted children to update (in case prior op modified them)
	ChildrenToUpdateLocalTransforms.Reset();
	for (int32 NonRetargetedChildBoneIndex : NonRetargetedChildrenToUpdate)
	{
		const FTransform ChildLocalTransform = TargetSkeleton.GetLocalTransformOfSingleBone(NonRetargetedChildBoneIndex, OutTargetGlobalPose);
		ChildrenToUpdateLocalTransforms.Add(ChildLocalTransform);
	}
	
	// spin through chains and encode/decode them all using the input pose
	const FIKRetargetPelvisMotionOp* PelvisMotionOp = Processor.GetFirstRetargetOpOfType<FIKRetargetPelvisMotionOp>();
	const FRetargetSkeleton& SourceSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Source);
	for (FChainPairFK& ChainPair : ChainPairsFK)
	{
		ChainPair.FKEncoder.EncodePose(
			SourceSkeleton,
			ChainPair.SourceBoneChain->BoneIndices,
			InSourceGlobalPose);
		
		ChainPair.FKDecoder.DecodePose(
			PelvisMotionOp,
			*ChainPair.Settings,
			ChainPair.TargetBoneChain->BoneIndices,
			ChainPair.FKEncoder,
			InSourceGlobalPose,
			SourceSkeleton,
			TargetSkeleton,
			OutTargetGlobalPose);
	}

	// update non-retargeted children
	for (int32 ChildIndex=0; ChildIndex<NonRetargetedChildrenToUpdate.Num(); ++ChildIndex)
	{
		int32 ChildBoneIndex = NonRetargetedChildrenToUpdate[ChildIndex];
		const FTransform& ChildLocalTransform = ChildrenToUpdateLocalTransforms[ChildIndex];
		TargetSkeleton.UpdateGlobalTransformOfSingleBone(ChildBoneIndex, ChildLocalTransform, TargetSkeleton.OutputGlobalPose);
	}

#if WITH_EDITOR
	SaveDebugData(OutTargetGlobalPose);
#endif
}

void FIKRetargetFKChainsOp::PostInitialize(
	const FIKRetargetProcessor& Processor,
	const FRetargetSkeleton& SourceSkeleton,
	const FTargetSkeleton& TargetSkeleton,
	FIKRigLogger& Log)
{
	const FName TargetPelvisBoneName = Processor.GetPelvisBone(ERetargetSourceOrTarget::Target, ERetargetOpsToSearch::ProcessorOps);
	const int32 TargetPelvisBoneIndex = TargetSkeleton.FindBoneIndexByName(TargetPelvisBoneName);

	// record intermediate bones (non-retargeted bones located BETWEEN FK chains on the target skeleton)
	for (FChainPairFK& FKChainPair : ChainPairsFK)
	{
		FKChainPair.FKDecoder.InitializeIntermediateParentIndices(
			TargetPelvisBoneIndex,
			FKChainPair.TargetBoneChain->BoneIndices[0],
			TargetSkeleton);
	}
}

void FIKRetargetFKChainsOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	// on initial setup, use the default source/target IK rigs
	const UIKRigDefinition* SourceIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source);
	const UIKRigDefinition* TargetIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target);
	ApplyIKRigs(SourceIKRig, TargetIKRig);

	// auto map
	static bool bForceRemap = true;
	Settings.ChainMapping.AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);
}

void FIKRetargetFKChainsOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	// get the newly assigned IK Rig
	const bool bAssignedSource = SourceOrTarget == ERetargetSourceOrTarget::Source;
	const UIKRigDefinition* SourceIKRig = bAssignedSource ? InIKRig : Settings.ChainMapping.GetIKRig(ERetargetSourceOrTarget::Source);
	const bool bAssignedTarget = SourceOrTarget == ERetargetSourceOrTarget::Target;
	const UIKRigDefinition* TargetIKRig = bAssignedTarget ? InIKRig : Settings.ChainMapping.GetIKRig(ERetargetSourceOrTarget::Target);
	ApplyIKRigs(SourceIKRig, TargetIKRig);

	// auto map, but don't force it (leave existing mappings alone)
	static bool bForceRemap = false;
	Settings.ChainMapping.AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);
}

void FIKRetargetFKChainsOp::CollectRetargetedBones(TSet<int32>& OutRetargetedBones) const
{
	// all bones in an FK chain are retargeted
	for (const FChainPairFK& FKChainPair : ChainPairsFK)
	{
		OutRetargetedBones.Append(FKChainPair.TargetBoneChain->BoneIndices);
	}
}

const UIKRigDefinition* FIKRetargetFKChainsOp::GetCustomTargetIKRig() const
{
	return Settings.IKRigAsset;
}

FRetargetChainMapping* FIKRetargetFKChainsOp::GetChainMapping()
{
	return &Settings.ChainMapping;
}

void FIKRetargetFKChainsOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	for (FRetargetFKChainSettings& ChainSettings : Settings.ChainsToRetarget)
	{
		if (ChainSettings.TargetChainName == InOldChainName)
		{
			ChainSettings.TargetChainName = InNewChainName;
		}
	}
}

void FIKRetargetFKChainsOp::OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent)
{
	const UIKRigDefinition* SourceIKRig = Settings.ChainMapping.GetIKRig(ERetargetSourceOrTarget::Source);
	const UIKRigDefinition* TargetIKRig = Settings.IKRigAsset;
	ApplyIKRigs(SourceIKRig, TargetIKRig);
}

FIKRetargetOpSettingsBase* FIKRetargetFKChainsOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetFKChainsOp::GetSettingsType() const
{
	return FIKRetargetFKChainsOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetFKChainsOp::GetType() const
{
	return FIKRetargetFKChainsOp::StaticStruct();
}

void FIKRetargetFKChainsOp::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	Super::PostLoad(InVersion);
	
	if (InVersion >= FIKRigObjectVersion::OpsOwnChainMapping
		&& InVersion < FIKRigObjectVersion::MovedChainMappingToFKChainOpSettings)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Settings.ChainMapping = ChainMapping_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

#if WITH_EDITOR

FCriticalSection FIKRetargetFKChainsOp::DebugDataMutex;

void FIKRetargetFKChainsOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	// draw lines on each FK chain
	if (!(Settings.bDrawChainLines || Settings.bDrawSingleBoneChains))
	{
		return;
	}

	// locked because this is called from the main thread and debug data is modified on worker
	FScopeLock ScopeLock(&DebugDataMutex);
	
	for (const FFKChainDebugData& ChainDebugData : AllChainsDebugData)
	{
		bool bIsSelected = InEditorState.SelectedChains.Contains(ChainDebugData.TargetChainName);
		FLinearColor Color = bIsSelected ? InEditorState.MainColor : InEditorState.MainColor * InEditorState.NonSelected;
		FTransform Start = ChainDebugData.StartTransform *  InComponentTransform;
		FTransform End = ChainDebugData.EndTransform *  InComponentTransform;
	
		// draw a line from start to end of chain, or in the case of a chain with only 1 bone in it, draw a sphere
		InPDI->SetHitProxy(new HIKRetargetEditorChainProxy(ChainDebugData.TargetChainName));
		if (Settings.bDrawChainLines && !ChainDebugData.bIsSingleBoneChain)
		{
			InPDI->DrawLine(
			Start.GetLocation(),
			End.GetLocation(),
			Color,
			SDPG_Foreground,
			static_cast<float>(Settings.ChainDrawThickness * InComponentScale));
		}
		else if (Settings.bDrawSingleBoneChains)
		{
			// single bone chain, just draw a sphere on the start bone
			DrawWireSphere(
				InPDI,
				Start.GetLocation(),
				Color,
				Settings.ChainDrawSize,
				12,
				SDPG_World,
				static_cast<float>(Settings.ChainDrawThickness * InComponentScale),
				0.001f,
				false);
		}
	
		InPDI->SetHitProxy(nullptr);
	}
}

void FIKRetargetFKChainsOp::SaveDebugData(const TArray<FTransform>& OutTargetGlobalPose)
{
	FScopeLock ScopeLock(&DebugDataMutex);

	AllChainsDebugData.Reset();
	for (const FChainPairFK& ChainPair : ChainPairsFK)
	{
		FFKChainDebugData ChainDebugData;
		ChainDebugData.TargetChainName = ChainPair.TargetBoneChain->ChainName;
		ChainDebugData.StartTransform = OutTargetGlobalPose[ChainPair.TargetBoneChain->BoneIndices[0]];
		ChainDebugData.EndTransform = OutTargetGlobalPose[ChainPair.TargetBoneChain->BoneIndices.Last()];
		ChainDebugData.bIsSingleBoneChain = ChainPair.TargetBoneChain->BoneIndices.Num() <= 1;
		AllChainsDebugData.Add(ChainDebugData);
	}
}

void FIKRetargetFKChainsOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.ChainsToRetarget, InChainName);
}

bool FIKRetargetFKChainsOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.ChainsToRetarget, InChainName);
}

uint8* FIKRetargetFKChainsOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.ChainsToRetarget, InChainName);
}

#endif

void FIKRetargetFKChainsOp::ApplyIKRigs(const UIKRigDefinition* InSourceIKRig, const UIKRigDefinition* InTargetIKRig)
{
	// store IK Rig
	Settings.IKRigAsset = InTargetIKRig;
	
	// update chain mapping
	Settings.ChainMapping.ReinitializeWithIKRigs(InSourceIKRig, InTargetIKRig);

	// update settings only if we have a valid mapping
	if (!Settings.ChainMapping.IsReady())
	{
		// don't remove settings, instead we want to preserve existing settings at least until the next valid rig is loaded
		return;
	}
	
	// get the required target chains
	const TArray<FBoneChain>& AllTargetChains = Settings.IKRigAsset->GetRetargetChains();
	TArray<FName> RequiredTargetChains;
	for (const FBoneChain& ChainToRetarget : AllTargetChains)
	{
		RequiredTargetChains.Add(ChainToRetarget.ChainName);
	}
	
	// remove chains that are not required
	Settings.ChainsToRetarget.RemoveAll([&RequiredTargetChains](const FRetargetFKChainSettings& InChainSettings)
	{
		return !RequiredTargetChains.Contains(InChainSettings.TargetChainName);
	});

	// add any required chains not already present
	for (const FName RequiredTargetChain : RequiredTargetChains)
	{
		bool bFound = false;
		for (const FRetargetFKChainSettings& ChainToRetarget : Settings.ChainsToRetarget)
		{
			if (ChainToRetarget.TargetChainName == RequiredTargetChain)
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			Settings.ChainsToRetarget.Emplace(RequiredTargetChain);
		}
	}
}

FIKRetargetFKChainsOpSettings UIKRetargetFKChainsController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetFKChainsOpSettings*>(OpSettingsToControl);
}

void UIKRetargetFKChainsController::SetSettings(FIKRetargetFKChainsOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
