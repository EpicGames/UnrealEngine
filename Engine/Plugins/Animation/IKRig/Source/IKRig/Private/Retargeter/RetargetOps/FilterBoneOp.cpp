// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/FilterBoneOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FilterBoneOp)

#define LOCTEXT_NAMESPACE "FilterBoneOp"


const UClass* FIKRetargetFilterBoneOpSettings::GetControllerType() const
{
	return UIKRetargetFilterBoneController::StaticClass();
}

void FIKRetargetFilterBoneOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies everything except the bones we are operating on (those require reinit)
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetFilterBoneOpSettings, BonesToFilter)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetFilterBoneOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

bool FIKRetargetFilterBoneOp::Initialize(
	const FIKRetargetProcessor& Processor,
	const FRetargetSkeleton& SourceSkeleton,
	const FTargetSkeleton& TargetSkeleton,
	const FIKRetargetOpBase* InParentOp, FIKRigLogger& Log)
{
	bIsInitialized = false;
	
	for (FFilterBoneData& FilterBoneData : Settings.BonesToFilter)
	{
		// find the target bone
		FilterBoneData.TargetBone.BoneIndex = TargetSkeleton.FindBoneIndexByName(FilterBoneData.TargetBone.BoneName);
		FilterBoneData.bIsReady = FilterBoneData.TargetBone.BoneIndex != INDEX_NONE;
		if (!FilterBoneData.bIsReady)
		{
			Log.LogWarning(FText::Format(
				LOCTEXT("MissingSourceBone", "Filter Bone op refers to non-existant target bone, {0}."),
				FText::FromName(FilterBoneData.TargetBone.BoneName)));
		}
	}
	
	// always treat this op as "initialized", individual filters will only execute if their prerequisites are met
	bIsInitialized = true;
	// force to initialize bone rotations on next update
	bResetPlayback = true;
	return true;
}

void FIKRetargetFilterBoneOp::Run(
	FIKRetargetProcessor& Processor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	// true on the first run or if playback was reset or reversed
	if (bResetPlayback)
	{
		// do not filter on first run, just reset prev rotations
		ResetFilters(OutTargetGlobalPose);
		bResetPlayback = false;
	}

	const FRetargetSkeleton& TargetSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);
	for (FFilterBoneData& FilterBoneData : Settings.BonesToFilter)
	{
		if (!FilterBoneData.bIsReady)
		{
			continue;
		}

		FTransform TargetGlobalTransform = OutTargetGlobalPose[FilterBoneData.TargetBone.BoneIndex];
		const FQuat FilteredRotation = FilterBoneData.Filter.Update(TargetGlobalTransform.GetRotation(), InDeltaTime, Settings.FilterSettings);
		const FQuat FinalRotation = FQuat::FastLerp(TargetGlobalTransform.GetRotation(), FilteredRotation, Settings.Alpha).GetNormalized();
		TargetGlobalTransform.SetRotation(FinalRotation);

		// assign result and update children
		TargetSkeleton.SetGlobalTransformAndUpdateChildren(FilterBoneData.TargetBone.BoneIndex, TargetGlobalTransform,OutTargetGlobalPose);
	}
}

const UScriptStruct* FIKRetargetFilterBoneOp::GetSettingsType() const
{
	return FIKRetargetFilterBoneOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetFilterBoneOp::GetType() const
{
	return FIKRetargetFilterBoneOp::StaticStruct();
}

void FIKRetargetFilterBoneOp::OnPlaybackReset()
{
	if (Settings.bResetPlayback)
	{
		bResetPlayback = true;	
	}
}

void FIKRetargetFilterBoneOp::ResetFilters(TArray<FTransform>& OutTargetGlobalPose)
{
	for (FFilterBoneData& FilterBoneData : Settings.BonesToFilter)
	{
		if (!FilterBoneData.bIsReady)
		{
			continue;
		}

		FilterBoneData.Filter.Reset();
	}
}

FIKRetargetOpSettingsBase* FIKRetargetFilterBoneOp::GetSettings()
{
	return &Settings;
}

// BEGIN CONTROLLER

FIKRetargetFilterBoneOpSettings UIKRetargetFilterBoneController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetFilterBoneOpSettings*>(OpSettingsToControl);
}

void UIKRetargetFilterBoneController::SetSettings(FIKRetargetFilterBoneOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

void UIKRetargetFilterBoneController::ClearBonesToFilter()
{
	FIKRetargetFilterBoneOpSettings* Settings = reinterpret_cast<FIKRetargetFilterBoneOpSettings*>(OpSettingsToControl);
	Settings->BonesToFilter.Reset();
}

void UIKRetargetFilterBoneController::AddBoneToFilter(const FName InTargetBone)
{
	FIKRetargetFilterBoneOpSettings* Settings = reinterpret_cast<FIKRetargetFilterBoneOpSettings*>(OpSettingsToControl);

	// skip if bone already present
	for (FFilterBoneData& BonePair : Settings->BonesToFilter)
	{
		if (BonePair.TargetBone == InTargetBone)
		{
			return;
		}
	}

	// add new bone
	FFilterBoneData NewBonePair;
	NewBonePair.TargetBone.BoneName = InTargetBone;
	Settings->BonesToFilter.Add(NewBonePair);
}

TArray<FName> UIKRetargetFilterBoneController::GetAllBonesToFilter()
{
	TArray<FName> AllBonesToFilter;
	
	FIKRetargetFilterBoneOpSettings* Settings = reinterpret_cast<FIKRetargetFilterBoneOpSettings*>(OpSettingsToControl);
	for (FFilterBoneData& BoneToFilter : Settings->BonesToFilter)
	{
		AllBonesToFilter.Add(BoneToFilter.TargetBone.BoneName);
	}
	
	return MoveTemp(AllBonesToFilter);
}


#undef LOCTEXT_NAMESPACE

