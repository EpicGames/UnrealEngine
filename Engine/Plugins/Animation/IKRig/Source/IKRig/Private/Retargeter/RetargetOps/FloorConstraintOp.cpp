// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/FloorConstraintOp.h"

#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/IKChainsOp.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"

#if WITH_EDITOR
#include "PrimitiveDrawInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloorConstraintOp)

#define LOCTEXT_NAMESPACE "FloorConstraintOp"

bool FFloorConstraintChainSettings::operator==(const FFloorConstraintChainSettings& Other) const
{
	return EnableFloorConstraint == Other.EnableFloorConstraint
		&& FMath::IsNearlyEqualByULP(Alpha,Other.Alpha)
		&& FMath::IsNearlyEqualByULP(MaintainHeightOffset, Other.MaintainHeightOffset);
}

const UClass* FIKRetargetFloorConstraintOpSettings::GetControllerType() const
{
	return UIKRetargetFloorConstraintController::StaticClass();
}

void FIKRetargetFloorConstraintOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the chains
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetFloorConstraintOpSettings, ChainsToAffect)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetFloorConstraintOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
	
	// copy settings only for chains that the op has initialized
	const FIKRetargetFloorConstraintOpSettings* NewSettings = reinterpret_cast<const FIKRetargetFloorConstraintOpSettings*>(InSettingsToCopyFrom);
	for (const FFloorConstraintChainSettings& NewChainSettings : NewSettings->ChainsToAffect)
	{
		for (FFloorConstraintChainSettings& ChainSettings : ChainsToAffect)
		{
			if (ChainSettings.TargetChainName != NewChainSettings.TargetChainName)
			{
				continue;
			}
			
			// copy all chain settings except the foot definition
			const TArray<FName> IgnoreFoot = {GET_MEMBER_NAME_CHECKED(FFloorConstraintChainSettings, Foot)};
			FIKRetargetOpBase::CopyStructProperties(
			FFloorConstraintChainSettings::StaticStruct(),
			&NewChainSettings,
			&ChainSettings,
			IgnoreFoot);

			// copy the foot definition
			FIKRetargetOpBase::CopyStructProperties(
			FFloorConstraintFootDefinition::StaticStruct(),
			&NewChainSettings.Foot,
			&ChainSettings.Foot,
			TArray<FName>());

			// copy the toes definition except the toes array
			const TArray<FName> IgnoreToes = {GET_MEMBER_NAME_CHECKED(FFloorConstraintToesDefinition, AllToes)};
			FIKRetargetOpBase::CopyStructProperties(
			FFloorConstraintToesDefinition::StaticStruct(),
			&NewChainSettings.Toes,
			&ChainSettings.Toes,
			IgnoreToes);

			// copy only the toes that match
			for (const FFloorConstraintToeDefinition& NewToe : NewChainSettings.Toes.AllToes)
			{
				for (FFloorConstraintToeDefinition& Toe : ChainSettings.Toes.AllToes)
				{
					if (Toe.ToeBone == NewToe.ToeBone)
					{
						FIKRetargetOpBase::CopyStructProperties(
						FFloorConstraintToeDefinition::StaticStruct(),
						&NewToe,
						&Toe,
						TArray<FName>());
					}
				}
			}
		}
	}
}

#if WITH_EDITOR
void FIKRetargetFloorConstraintOpSettings::AssignSkeletonAssets(const USkeleton* InSourceSkeleton, const USkeleton* InTargetSkeleton)
{
	FIKRetargetOpSettingsBase::AssignSkeletonAssets(InSourceSkeleton, InTargetSkeleton);
	for (FFloorConstraintChainSettings& Chain : ChainsToAffect)
	{
		Chain.SourceSkeletonAsset = InSourceSkeleton;
		Chain.TargetSkeletonAsset = InTargetSkeleton;
	}
}
#endif

void FFootConstraint::Initialize(
	const FTargetSkeleton& InTargetSkeleton,
	const FFloorConstraintFootDefinition& InFootSettings,
	const int32 InAnkleBoneIndex,
	FIKRigLogger& InLog)
{
	bReadyToRun = false;
	
	AnkleBoneIndex = InAnkleBoneIndex;
	if (!InTargetSkeleton.BoneNames.IsValidIndex(InAnkleBoneIndex))
	{
		InLog.LogWarning(LOCTEXT("FootConstraintMissingAnkleBone", "Foot Constraint: missing target ankle bone."));
		return;
	}
	
	const TArray<FTransform>& TargetRetargetPose = InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	CurrentAnkleGlobal = TargetRetargetPose[AnkleBoneIndex];

	const FVector MedialDirection = FVector::XAxisVector * (CurrentAnkleGlobal.GetLocation().X > 0.0 ? -1.0 : 1.0);
	const FVector LateralDirection = MedialDirection * -1.0;
	const FVector ToeDirection = FVector(0,1,0);
	const FVector HeelDirection = -ToeDirection;
	const FVector MedialOffset = MedialDirection * InFootSettings.MedialOffset;
	const FVector LateralOffset = LateralDirection * InFootSettings.LateralOffset;
	const FVector ToeOffset = ToeDirection * InFootSettings.ToeOffset;
	const FVector HeelOffset = HeelDirection * InFootSettings.HeelOffset;
	const FVector VerticalOffset = FVector(0,0, InFootSettings.VerticalOffset);
	const FVector AnkleOnFloor = (CurrentAnkleGlobal.GetLocation() * FVector(1,1,0)) + VerticalOffset;

	// offset corner points from position of ankle projected on the floor
	PointsOrig[FrontMedial] = AnkleOnFloor + ToeOffset + MedialOffset;
	PointsOrig[FrontLateral] = AnkleOnFloor + ToeOffset + LateralOffset;
	PointsOrig[RearMedial] = AnkleOnFloor + HeelOffset + MedialOffset;
	PointsOrig[RearLateral] = AnkleOnFloor + HeelOffset + LateralOffset;

	// put in space of ankle
	PointsOrig[FrontMedial] = CurrentAnkleGlobal.InverseTransformPosition(PointsOrig[FrontMedial]);
	PointsOrig[FrontLateral] = CurrentAnkleGlobal.InverseTransformPosition(PointsOrig[FrontLateral]);
	PointsOrig[RearMedial] = CurrentAnkleGlobal.InverseTransformPosition(PointsOrig[RearMedial]);
	PointsOrig[RearLateral] = CurrentAnkleGlobal.InverseTransformPosition(PointsOrig[RearLateral]);

	bReadyToRun = true;
}

void FFootConstraint::UpdateBasePoints(const TArray<FTransform>& InGlobalPose, const FIKRigGoal* InGoal)
{
	// get latest foot/ball transforms
	if (InGoal)
	{
		CurrentAnkleGlobal.SetLocation(InGoal->Position);
		CurrentAnkleGlobal.SetRotation(InGoal->Rotation.Quaternion());
	}
	else
	{
		CurrentAnkleGlobal = InGlobalPose[AnkleBoneIndex];
	}
	
	Points[FrontMedial] = CurrentAnkleGlobal.TransformPosition(PointsOrig[FrontMedial]);
	Points[FrontLateral] = CurrentAnkleGlobal.TransformPosition(PointsOrig[FrontLateral]);
	Points[RearMedial] = CurrentAnkleGlobal.TransformPosition(PointsOrig[RearMedial]);
	Points[RearLateral] = CurrentAnkleGlobal.TransformPosition(PointsOrig[RearLateral]);
}

int32 FFootConstraint::GetIndexOfLowestPoint()
{
	int32 MinIndex = 0;
	float MinValue = TNumericLimits<float>::Max();

	for (int32 i=0; i<Points.Num(); ++i)
	{
		const float Z = Points[i].Z;
		if (Z < MinValue)
		{
			MinValue = Z;
			MinIndex = i;
		}
	}

	return MinIndex;
}

void FToeConstraint::Initialize(
	const FTargetSkeleton& InTargetSkeleton,
	const FFloorConstraintToesDefinition& InAllToeSettings,
	const FFloorConstraintToeDefinition& InToeSettings,
	const int32 InAnkleBoneIndex,
	FIKRigLogger& InLog)
{
	bToeReady = false;
	
	const FName ToeBoneName = InToeSettings.ToeBone.BoneName;
	ToeBoneIndex = InTargetSkeleton.FindBoneIndexByName(ToeBoneName);
	if (ToeBoneIndex == INDEX_NONE)
	{
		InLog.LogWarning(
			FText::Format(LOCTEXT("FootConstraintMissingToeBone", "Floor Constraint: missing target toe bone, '{0}'."),
				FText::FromName(ToeBoneName)));
		return;
	}

	AnkleBoneIndex = InAnkleBoneIndex;
	if (InAnkleBoneIndex == INDEX_NONE)
	{
		return; // logged inside Foot.Init
	}

	Settings = &InToeSettings;
	AllToeSettings = &InAllToeSettings;

	// initialize local offset and start/end points based on retarget pose
	const TArray<FTransform>& GlobalRetargetPose = InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	InitialToeGlobal = GlobalRetargetPose[ToeBoneIndex];
	constexpr bool bUpdateLocalOffset = true;
	UpdateFramePoints(GlobalRetargetPose,  nullptr, bUpdateLocalOffset);

	bToeReady = true;
}

void FToeConstraint::RunAfterParent(
	const FTargetSkeleton& InTargetSkeleton,
	TArray<FTransform>& OutGlobalPose,
	const FFloorConstraintChainSettings* InChainSettings,
	FIKRetargetFloorConstraintOpSettings& InOpSettings,
	const double InDeltaTime)
{
	if (!bToeReady)
	{
		return;
	}

	constexpr FIKRigGoal* NullGoal = nullptr; // this is running after the IK solve, so use the foot at the input pose transform
	bool bUpdateLocalOffset = false;
	UpdateFramePoints(OutGlobalPose, NullGoal, bUpdateLocalOffset);

	// solve for the angle to put toe on floor
	double ThetaRadians = CalcToeCorrectionAngle();

	// temporal filtering of the toe adjustment
	FOneEuroFilterSettings OneEuroFilterSettings;
	OneEuroFilterSettings.CutoffFrequency = AllToeSettings->CutoffFrequency;
	OneEuroFilterSettings.Responsiveness = AllToeSettings->Responsiveness;
	OneEuroFilterSettings.VelocityCutoffFrequency = AllToeSettings->VelocityCutoffFrequency;
	ThetaRadians = AngleFilter.Update(ThetaRadians, InDeltaTime, OneEuroFilterSettings);

	// alpha the whole effect
	const float Alpha = AllToeSettings->Alpha * InChainSettings->Alpha * InOpSettings.Alpha;
	ThetaRadians = FMath::Lerp(0, ThetaRadians, Alpha);

	// convert to a quaternion that rotates around the pitch axis by theta
	const FQuat DeltaRotation = FQuat(PitchAxis, ThetaRadians);
	
	// apply rotation to toe bone and update children
	FTransform NewToeGlobal = OutGlobalPose[ToeBoneIndex];
	NewToeGlobal.SetRotation(DeltaRotation * NewToeGlobal.GetRotation());
	InTargetSkeleton.SetGlobalTransformAndUpdateChildren(ToeBoneIndex, NewToeGlobal, OutGlobalPose);

#if WITH_EDITOR
	// update frame points after solve while in editor for debug drawing
	bUpdateLocalOffset = true;
	UpdateFramePoints(OutGlobalPose, NullGoal, bUpdateLocalOffset);
#endif
}

void FToeConstraint::UpdateFramePoints(
	const TArray<FTransform>& InGlobalPose,
	const FIKRigGoal* InGoal,
	bool bUpdateLocalOffset)
{
	FTransform CurrentAnkleGlobal;
	if (InGoal)
	{
		CurrentAnkleGlobal.SetLocation(InGoal->Position);
		CurrentAnkleGlobal.SetRotation(InGoal->Rotation.Quaternion());
	}
	else
	{
		CurrentAnkleGlobal = InGlobalPose[AnkleBoneIndex];
	}

	const FTransform CurrentLocalToe = InGlobalPose[ToeBoneIndex].GetRelativeTransform(InGlobalPose[AnkleBoneIndex]);
	CurrentToeGlobal = CurrentLocalToe * CurrentAnkleGlobal;

	if (bUpdateLocalOffset && Settings)
	{
		// calc toe end relative to the toe
		FVector EndOffset = FVector::UnitY() * Settings->Length;
		const FQuat YawRotation = FQuat(FVector::ZAxisVector, FMath::DegreesToRadians(Settings->YawOffset));
		EndOffset = YawRotation.RotateVector(EndOffset);
		FVector EndPoint = InitialToeGlobal.GetLocation() + EndOffset;
		EndPoint.Z = Settings->VerticalOffset;
		EndRelativeToToe = InitialToeGlobal.InverseTransformPosition(EndPoint);

		// calc pitch axis
		const FVector InitialGlobalPitchAxis = YawRotation.RotateVector(FVector::UnitX());
		PitchAxisRelativeToToe = InitialToeGlobal.InverseTransformVector(InitialGlobalPitchAxis);
	}
	
	Start = CurrentToeGlobal.GetLocation();
	End = CurrentToeGlobal.TransformPosition(EndRelativeToToe);
	PitchAxis = CurrentToeGlobal.TransformVector(PitchAxisRelativeToToe);
}

double FToeConstraint::CalcToeCorrectionAngle()
{
	if (End.Z >= 0)
	{
		return 0; // toe is above the floor already
	}

	const FVector r = End - Start;
	if (r.IsNearlyZero())
	{
		return 0; // toe has degenerate length
	}

	// NOTE:
	// The following solver code may, at first, look a bit magic, so I include this derivation.
	//
	// The overall idea is to treat this as a circle/plane collision check where the toe point (End) rotates
	// around the toe joint (Start) in the plane perpendicular to the PitchAxis (k).
	//
	// We need to find the rotation angle theta around axis k such that the rotated toe endpoint lies on the floor (Z = 0).
	//
	// Rodrigues' formula gives vector r' after rotating r by theta:
	//
	//     r'(theta) = r * cos(theta) + cross(k, r) * sin(theta) + k * dot(k, r) * (1 - cos(theta))
	//
	// Isolating the Z component gives a function:
	//
	//     z(theta) = A * cos(theta) + B * sin(theta) + C
	//
	// Explanation:
	// Each term in Rodrigues' formula contributes to the Z height of the rotated toe tip:
	//   - r * cos(theta): projects the original vector r onto its rotated position (scaled by cos(theta)).
	//     Its Z component varies as cos(theta), giving coefficient A.
	//   - cross(k, r) * sin(theta): represents rotation perpendicular to both k and r.
	//     Its Z component varies as sin(theta), giving coefficient B.
	//   - k * dot(k, r) * (1 - cos(theta)): accounts for the component of r already aligned with the axis k.
	//     This adds a constant offset to Z (independent of theta) captured by coefficient C.
	//
	// Summing these gives a trig dependence of Z on theta:
	//
	//     z(theta) = A * cos(theta) + B * sin(theta) + C
	//
	// Meaning the vertical position of the toe tip moves up and down as the vector r rotates in a circular arc about axis k.
	//
	// We want z(theta) + Start.Z = 0, which expands to:
	//
	//     A * cos(theta) + B * sin(theta) + C = 0
	//
	// Any linear combination of sine and cosine can be written as a single cosine with a phase shift:
	//
	//     A * cos(theta) + B * sin(theta) = R * cos(theta - phi)
	//
	// where:
	//     R = sqrt(A*A + B*B)   // amplitude of the combined wave
	//     phi = atan2(B, A)     // phase offset (angle of vector [A, B])
	//
	// Substituting back:
	//
	//     R * cos(theta - phi) = -C
	//
	// Now with a single cosine, we can isolate theta:
	//
	//     theta = phi +/- acos(-C / R)
	//
	// These are the two possible rotations that put the toe on the floor.
	// We clamp X = -C / R to [-1, 1] for safety and pick the smallest positive theta so the toe only bends upward.
	
	// a is z component of first term
	const double a = r.Z;
	// b is z component of second term
	const FVector& k = PitchAxis;
	const FVector kxr = FVector::CrossProduct(k, r);
	const double b = kxr.Z;
	// c is z component of third term
	const double c = k.Z * FVector::DotProduct(k, r);

	// now we need to solve for: Start.Z + z(theta) = 0
	// (a-c)*cos(theta) + b*sin(theta) + (c+Start.Z) = 0
	const double A = a - c;
	const double B = b;
	const double C = c + Start.Z;

	// if A and B are orthogonal, then rotation about k cannot change Z, bail.
	const double R = FMath::Sqrt(A*A + B*B);
	if (R < UE_SMALL_NUMBER) 
	{
		return 0; // toe Z coord is effectively independent of theta
	}
	
	const double Phi = FMath::Atan2(B, A);
	double X = -C / R;
	X = FMath::Clamp(X, -1.0, 1.0);
	const double AcosX = FMath::Acos(X);

	// two candidate angles: theta = phi +/- acos(x)
	const double ThetaA = FMath::UnwindRadians(Phi + AcosX); // wrap to (-pi,pi)
	const double ThetaB = FMath::UnwindRadians(Phi - AcosX);
	
	// pick the smallest positive theta
	// (this ensures toes only bend up, never backwards)
	if (ThetaA >= 0.0 && ThetaB >= 0.0)
	{
		// both up: pick smaller magnitude
		return (ThetaA <= ThetaB) ? ThetaA : ThetaB;
	}
	if (ThetaA >= 0.0)
	{
		return ThetaA;
	}
	if (ThetaB >= 0.0)
	{
		return ThetaB;
	}
	
	return 0; // both would rotate downward, bail out
}

void FFootConstraint::Run(
	FIKRigGoal& InOutGoal,
	const TArray<FTransform>& OutTargetGlobalPose,
	const FFloorConstraintChainSettings& InChainSettings,
	const FIKRetargetFloorConstraintOpSettings& InOpSettings)
{
	if (!bReadyToRun)
	{
		return;
	}
	
	// calc vertical delta to push all base points above the floor
	UpdateBasePoints(OutTargetGlobalPose, &InOutGoal);
	const int32 LowestBasePointIndex = GetIndexOfLowestPoint();
	const double BaseHeight = Points[LowestBasePointIndex].Z;
	const double BaseDelta = BaseHeight < 0.0f ? FMath::Abs(BaseHeight) : 0.0;
	
	// move the goal up
	const double Alpha = InChainSettings.Alpha * InOpSettings.Alpha;
	InOutGoal.Position.Z += BaseDelta * Alpha;
}

void FFloorConstraint::Initialize(
	const FResolvedBoneChain& InSourceBoneChain,
	const FResolvedBoneChain& InTargetBoneChain,
	const FFloorConstraintChainSettings& InSettings)
{
	IKGoalName = InTargetBoneChain.IKGoalName;
	SourceEndBoneIndex = InSourceBoneChain.BoneIndices.Last();
	TargetEndBoneIndex = InTargetBoneChain.BoneIndices.Last();
	Settings = &InSettings;

	// cache ref pose offset
	const FTransform& SourceEndBoneRefPose = InSourceBoneChain.RefPoseGlobalTransforms.Last();
	const FTransform& TargetEndBoneRefPose = InTargetBoneChain.RefPoseGlobalTransforms.Last();
	HeightOffsetInRefPose = TargetEndBoneRefPose.GetTranslation().Z - SourceEndBoneRefPose.GetTranslation().Z;
}

void FFloorConstraint::Run(
	const FTargetSkeleton& InTargetSkeleton,
	FIKRigGoalContainer& InGoalContainer,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose,
	const FIKRetargetFloorConstraintOpSettings& InOpSettings)
{
	if (!Settings->EnableFloorConstraint)
	{
		return; // constraint disabled
	}
		
	if (FMath::IsNearlyZero(Settings->Alpha))
	{
		return; // constraint weight near zero
	}

	// get the goal to adjust
	FIKRigGoal* GoalPtr = InGoalContainer.FindGoalByName(IKGoalName);
	if (!ensure(GoalPtr))
	{
		return; // goal missing (should not happen at runtime)
	}
	FIKRigGoal& Goal = *GoalPtr;

	// adjust height of IK Goal to match the source relative to floor
		
	// calculate falloff range
	const float FalloffStartHeight = InOpSettings.HeightFalloffOffset;
	float FalloffEndHeight = FalloffStartHeight + FMath::Max(0.0, InOpSettings.HeightFalloffDistance);

	// get the current height of the source bone
	const FTransform& CurrentSourceEndGlobal = InSourceGlobalPose[SourceEndBoneIndex];
	const float SourceHeight = FMath::Abs(CurrentSourceEndGlobal.GetLocation().Z);

	// constraint on while source is near ground
	if (SourceHeight < FalloffEndHeight)
	{
		// calculate the target height of the goal by blending towards the source height when close to floor
		const double FalloffRange = FMath::Max(1.0,FalloffEndHeight - FalloffStartHeight);
		const double Falloff = SourceHeight < FalloffStartHeight ? 1.0f : 1.0f - (SourceHeight - FalloffStartHeight) / FalloffRange;
		const double Weight = Falloff * Settings->Alpha * InOpSettings.Alpha;
		const double HeightOffset = FMath::Lerp(0.0f, HeightOffsetInRefPose, Settings->MaintainHeightOffset);
		const double TargetHeight = SourceHeight + HeightOffset;
		
		// lerp goal height from its current height to the target height by the weight
		Goal.Position.Z = FMath::Lerp(Goal.Position.Z, TargetHeight, Weight);
	}

	// run the foot/floor constraint based on new goal location
	FootConstraint.Run(Goal, OutTargetGlobalPose, *Settings, InOpSettings);
}

bool FIKRetargetFloorConstraintOp::Initialize(
    const FIKRetargetProcessor& InProcessor,
    const FRetargetSkeleton& InSourceSkeleton,
    const FTargetSkeleton& InTargetSkeleton,
    const FIKRetargetOpBase* InParentOp,
    FIKRigLogger& InLog)
{
	bIsInitialized = false;
	
	// validate that an IK rig has been assigned to the parent op
	const FIKRetargetRunIKRigOp* ParentOp = reinterpret_cast<const FIKRetargetRunIKRigOp*>(InParentOp);
	if (ParentOp->Settings.IKRigAsset == nullptr)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingIKRig", "{0}, is missing an IK rig. No chains can be retargeted."), FText::FromName(GetName())));
		return false;
	}

	// initialize / load floor constraints
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	FloorConstraints.Reset();
	for (const FFloorConstraintChainSettings& ChainSettings : Settings.ChainsToAffect)
	{
		const FName TargetChainName = ChainSettings.TargetChainName;
		const FResolvedBoneChain* TargetBoneChain = BoneChains.GetResolvedBoneChainByName(TargetChainName, ERetargetSourceOrTarget::Target,ParentOp->Settings.IKRigAsset);
		if (!TargetBoneChain)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("FloorConstraintChainNotFound", "Floor Constraint: target chain not found in IK Rig, '{0}."), FText::FromName(TargetChainName)));
			continue;
		}

		if (TargetBoneChain->IKGoalName == NAME_None)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("FloorConstraintIKNotFound", "Floor Constraint: target chain has no IK Goal, '{0}."), FText::FromName(TargetChainName)));
			continue;
		}

		// which source chain was this target chain mapped to?
		const FName SourceChainName = ParentOp->ChainMapping.GetChainMappedTo(TargetChainName, ERetargetSourceOrTarget::Target);
		const FResolvedBoneChain* SourceBoneChain = BoneChains.GetResolvedBoneChainByName(SourceChainName, ERetargetSourceOrTarget::Source);
		if (!SourceBoneChain)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("FloorConstraintChainNotMapped", "Floor Constraint: found IK chain that was not mapped to a source chain, '{0}."), FText::FromName(TargetChainName)));
			continue;
		}

		// create a FLOOR constraint
		if (ChainSettings.EnableFloorConstraint)
		{
			FFloorConstraint& FloorConstraint = FloorConstraints.Emplace_GetRef();
			FloorConstraint.Initialize(*SourceBoneChain, *TargetBoneChain, ChainSettings);
			
			// initialize foot constraint
			if (ChainSettings.bUseFoot || ChainSettings.bUseToes)
			{
				FloorConstraint.FootConstraint.Initialize(InTargetSkeleton, ChainSettings.Foot, FloorConstraint.TargetEndBoneIndex, InLog);
			}

			// initialize toe constraints
			if (ChainSettings.bUseToes)
			{
				for (const FFloorConstraintToeDefinition& ToeSettings : ChainSettings.Toes.AllToes)
				{
					FToeConstraint& Toe = FloorConstraint.ToeConstraints.Emplace_GetRef();
					Toe.Initialize(InTargetSkeleton, ChainSettings.Toes, ToeSettings, FloorConstraint.TargetEndBoneIndex, InLog);
				}
			}
		}
	}
	
	bIsInitialized = true;
    return true;
}

void FIKRetargetFloorConstraintOp::Run(
    FIKRetargetProcessor& InProcessor,
    const double InDeltaTime,
    const TArray<FTransform>& InSourceGlobalPose,
    TArray<FTransform>& OutTargetGlobalPose)
{
	// run all the floor constraints (1 per chain)
	// NOTE: in the main pass, all we do is affect the IK Goal to:
	// A. match the source height relative to the ground
	// B. if foot dimensions are available, push the goal upwards to keep foot out of the ground
	const FTargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (FFloorConstraint& FloorConstraint : FloorConstraints)
	{
		FloorConstraint.Run(TargetSkeleton,GoalContainer, InSourceGlobalPose, OutTargetGlobalPose, Settings);
	}
}

void FIKRetargetFloorConstraintOp::RunAfterParent(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	// NOTE: this pass runs AFTER the IK has solved.
	// At this point, we can adjust the toes on the post-IK pose to keep them above the floor
	const FTargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
	for (FFloorConstraint& FloorConstraint : FloorConstraints)
	{
		if (!FloorConstraint.FootConstraint.bReadyToRun)
		{
			continue;
		}

#if WITH_EDITOR
		// for accurate debug drawing, we update the foot base points AFTER IK solve
		FloorConstraint.FootConstraint.UpdateBasePoints(OutTargetGlobalPose, nullptr);
#endif
		
		for (FToeConstraint& Toe : FloorConstraint.ToeConstraints)
		{
			Toe.RunAfterParent(TargetSkeleton, OutTargetGlobalPose, FloorConstraint.Settings, Settings, InDeltaTime);
		}
	}
}

void FIKRetargetFloorConstraintOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	// regenerate chain settings
	constexpr bool bSkipUnmappedChains = false;
	constexpr bool bSkipNonIKChains = true;
	IKRetargetOpUtils::SynchronizeChainSettingsWithIKRig(Settings.ChainsToAffect, InParentOp, bSkipUnmappedChains, bSkipNonIKChains);
}

void FIKRetargetFloorConstraintOp::OnAssignIKRig(
    const ERetargetSourceOrTarget SourceOrTarget,
    const UIKRigDefinition* InIKRig,
    const FIKRetargetOpBase* InParentOp)
{
	// regenerate chain settings
	constexpr bool bSkipUnmappedChains = false;
	constexpr bool bSkipNonIKChains = true;
	IKRetargetOpUtils::SynchronizeChainSettingsWithIKRig(Settings.ChainsToAffect, InParentOp, bSkipUnmappedChains, bSkipNonIKChains);
}

FIKRetargetOpSettingsBase* FIKRetargetFloorConstraintOp::GetSettings()
{
    return &Settings;
}

const UScriptStruct* FIKRetargetFloorConstraintOp::GetSettingsType() const
{
    return FIKRetargetFloorConstraintOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetFloorConstraintOp::GetType() const
{
    return StaticStruct();
}

const UScriptStruct* FIKRetargetFloorConstraintOp::GetParentOpType() const
{
    return FIKRetargetRunIKRigOp::StaticStruct();
}

void FIKRetargetFloorConstraintOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	IKRetargetOpUtils::OnRetargetChainRenamed(Settings.ChainsToAffect, InOldChainName, InNewChainName);
}

void FIKRetargetFloorConstraintOp::OnParentReinitPropertyEdited(
    const FIKRetargetOpBase& InParentOp,
    const FPropertyChangedEvent* InPropertyChangedEvent)
{
	// regenerate chain settings
	constexpr bool bSkipUnmappedChains = false;
	constexpr bool bSkipNonIKChains = true;
	IKRetargetOpUtils::SynchronizeChainSettingsWithIKRig(Settings.ChainsToAffect, &InParentOp, bSkipUnmappedChains, bSkipNonIKChains);
}

// ------------------------------------------------------------
// Editor-only
// ------------------------------------------------------------
#if WITH_EDITOR

void FIKRetargetFloorConstraintOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.ChainsToAffect, InChainName);
}

bool FIKRetargetFloorConstraintOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.ChainsToAffect, InChainName);
}

uint8* FIKRetargetFloorConstraintOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.ChainsToAffect, InChainName);
}

void FIKRetargetFloorConstraintOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InTargetTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	static FLinearColor FootColor = FLinearColor::Green;
	static FLinearColor ToeColor = FLinearColor::Blue;
	constexpr float PointSize = 5.0f;
	constexpr float LineThickness = 0.5f;
	
	auto DrawFootFrame = [&](const FFootConstraint& InFoot, const FTransform& InComponentTransform)
		{
			const FVector FrontMedial = InComponentTransform.TransformPosition(InFoot.Points[FFootConstraint::FrontMedial]);
			const FVector FrontLateral = InComponentTransform.TransformPosition(InFoot.Points[FFootConstraint::FrontLateral]);
			const FVector RearMedial = InComponentTransform.TransformPosition(InFoot.Points[FFootConstraint::RearMedial]);
			const FVector RearLateral = InComponentTransform.TransformPosition(InFoot.Points[FFootConstraint::RearLateral]);
			
			InPDI->DrawPoint(RearMedial, FootColor, PointSize, SDPG_Foreground);
			InPDI->DrawPoint(RearLateral, FootColor, PointSize, SDPG_Foreground);
			InPDI->DrawPoint(FrontMedial, FootColor, PointSize, SDPG_Foreground);
			InPDI->DrawPoint(FrontLateral, FootColor, PointSize, SDPG_Foreground);
			
			DrawDashedLine(InPDI, RearMedial, RearLateral, FootColor, 1, SDPG_Foreground);
			DrawDashedLine(InPDI, RearMedial, FrontMedial, FootColor, 1, SDPG_Foreground);
			DrawDashedLine(InPDI, RearLateral, FrontLateral, FootColor, 1, SDPG_Foreground);
			DrawDashedLine(InPDI, FrontMedial, FrontLateral, FootColor, 1, SDPG_Foreground);
		};
	
	for (const FFloorConstraint& FloorConstraint : FloorConstraints)
	{
		if (FloorConstraint.FootConstraint.bReadyToRun)
		{
			DrawFootFrame(FloorConstraint.FootConstraint, InTargetTransform);
			
			for (const FToeConstraint& Toe : FloorConstraint.ToeConstraints)
			{
				const FVector ToeStart = InTargetTransform.TransformPosition(Toe.Start);
				const FVector ToeEnd = InTargetTransform.TransformPosition(Toe.End);
				InPDI->DrawPoint(ToeEnd, ToeColor, PointSize, SDPG_Foreground);
				InPDI->DrawLine(ToeStart, ToeEnd, ToeColor, SDPG_Foreground, LineThickness);
				// draw pitch axis
				InPDI->DrawLine(ToeStart, ToeStart + Toe.PitchAxis, FLinearColor::Red, SDPG_Foreground, LineThickness);
			}
		}
	}
}

#endif // WITH_EDITOR

FIKRetargetFloorConstraintOpSettings UIKRetargetFloorConstraintController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetFloorConstraintOpSettings*>(OpSettingsToControl);
}

void UIKRetargetFloorConstraintController::SetSettings(FIKRetargetFloorConstraintOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
