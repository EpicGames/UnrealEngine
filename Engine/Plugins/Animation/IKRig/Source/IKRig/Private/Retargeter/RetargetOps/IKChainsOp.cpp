// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/IKChainsOp.h"

#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "Retargeter/IKRetargetOpUtils.h"
#include "IKRigObjectVersion.h"

#if WITH_EDITOR
#include "PrimitiveDrawInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"
#include "IKRigDebugRendering.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKChainsOp)

#define LOCTEXT_NAMESPACE "IKChainsOp"

bool FIKChainRetargeter::Initialize(
	const FResolvedBoneChain& InSourceBoneChain,
	const FResolvedBoneChain& InTargetBoneChain,
	const FRetargetIKChainSettings& InSettings,
	FIKRigLogger& InLog)
{
	if (InSourceBoneChain.BoneIndices.Num() < 2)
	{
		InLog.LogWarning(LOCTEXT("SourceChainLessThanThree", "IK Chains Op: trying to retarget source bone chain with IK but it has less than 2 joints."));
		return false;
	}

	// store pointers to required data
	SourceBoneChain = &InSourceBoneChain;
	TargetBoneChain = &InTargetBoneChain;
	Settings = &InSettings;
	
	// cache initial length of SOURCE chain
	const FTransform& InitialSourceStartGlobal = InSourceBoneChain.RefPoseGlobalTransforms[0];
	const FTransform& InitialSourceEndGlobal = InSourceBoneChain.RefPoseGlobalTransforms.Last();
	const float InitialStartLength = static_cast<float>((InitialSourceStartGlobal.GetLocation() - InitialSourceEndGlobal.GetLocation()).Size());
	if (InitialStartLength <= UE_KINDA_SMALL_NUMBER)
	{
		InLog.LogWarning(LOCTEXT("SourceZeroLengthIK", "IK Chains Op: found zero-length source bone chain."));
		return false;
	}
	InvInitialLengthSource = 1.0f / InitialStartLength;

	// cache initial length of TARGET chain
	const FTransform& InitialTargetStartGlobal = InTargetBoneChain.RefPoseGlobalTransforms[0];
	const FTransform& InitialTargetEndGlobal = InTargetBoneChain.RefPoseGlobalTransforms.Last();
	InitialTargetChainLength = static_cast<float>((InitialTargetStartGlobal.GetTranslation() - InitialTargetEndGlobal.GetTranslation()).Size());
	if (InitialTargetChainLength <= UE_KINDA_SMALL_NUMBER)
	{
		InLog.LogWarning(LOCTEXT("TargetZeroLengthIK", "IK Retargeter trying to retarget target bone chain with IK, but it is zero length!"));
		return false;
	}

	// TODO
	InitialTargetEndRelativeToSourceEnd = InitialTargetEndGlobal.GetRelativeTransform(InitialSourceEndGlobal);

	return true;
}

const FTransform& FIKChainRetargeter::GenerateGoalTransform(
	const FIKRetargetPelvisMotionOp* InPelvisMotionOp,
	const TArray<FTransform>& InSourceGlobalPose,
	const TArray<FTransform>& InTargetGlobalPose,
	const FRetargetSkeleton& InSourceSkeleton,
	const FRetargetSkeleton& InTargetSkeleton)
{
	// calculate ROTATION of IK goal
	GoalOutput.SetRotation( CalculateGoalRotation(InSourceGlobalPose) );
	// calculate POSITION of IK goal ...
	GoalOutput.SetTranslation( CalculateGoalPosition(InPelvisMotionOp, InSourceGlobalPose, InTargetGlobalPose, InSourceSkeleton, InTargetSkeleton) );
	return GoalOutput;
}

FQuat FIKChainRetargeter::CalculateGoalRotation(const TArray<FTransform>& InSourceGlobalPose) const
{
	// get relevant bone transforms
	const int32 SourceEndBoneIndex = SourceBoneChain->BoneIndices.Last();
	const FTransform& InitialSourceEndGlobal = SourceBoneChain->RefPoseGlobalTransforms.Last();
	const FTransform& InitialTargetEndGlobal = TargetBoneChain->RefPoseGlobalTransforms.Last();
	const FTransform& CurrentSourceEndGlobal = InSourceGlobalPose[SourceEndBoneIndex];
	
	// apply delta rotation from source input
	const FQuat CurrentSourceEndRotation = CurrentSourceEndGlobal.GetRotation();
	const FQuat SourceDeltaRotation = (CurrentSourceEndRotation * InitialSourceEndGlobal.GetRotation().Inverse()).GetNormalized();
	FQuat GoalRotation = SourceDeltaRotation * InitialTargetEndGlobal.GetRotation();

	// blend to source rotation
	const double BlendToSourceRotation = Settings->BlendToSource * Settings->BlendToSourceRotation;
	if (BlendToSourceRotation > UE_KINDA_SMALL_NUMBER)
	{
		GoalRotation = FQuat::FastLerp(GoalRotation, CurrentSourceEndRotation, BlendToSourceRotation);
		GoalRotation.Normalize();
	}

	// apply static rotation offset in the local space of the foot (AFTER blend to source)
	GoalRotation = GoalRotation * Settings->StaticRotationOffset.Quaternion();
	
	return MoveTemp(GoalRotation);
}

FVector FIKChainRetargeter::CalculateGoalPosition(
	const FIKRetargetPelvisMotionOp* InPelvisMotionOp,
	const TArray<FTransform>& InSourceGlobalPose,
	const TArray<FTransform>& InTargetGlobalPose,
	const FRetargetSkeleton& InSourceSkeleton,
	const FRetargetSkeleton& InTargetSkeleton) const
{
	const int32 SourceStartBoneIndex = SourceBoneChain->BoneIndices[0];
	const int32 SourceEndBoneIndex = SourceBoneChain->BoneIndices.Last();
	const FTransform& CurrentSourceStartGlobal = InSourceGlobalPose[SourceStartBoneIndex];
	const FTransform& CurrentSourceEndGlobal = InSourceGlobalPose[SourceEndBoneIndex];
	
	// get the current normalized direction / length of the IK limb (how extended it is as percentage of original length)
	const FVector CurrentSourceChainVector = CurrentSourceEndGlobal.GetLocation() - CurrentSourceStartGlobal.GetLocation();
	double CurrentSourceChainLength;
	FVector CurrentSourceChainDirection;
	CurrentSourceChainVector.ToDirectionAndLength(CurrentSourceChainDirection, CurrentSourceChainLength);
	const double NormalizedLimbLength = CurrentSourceChainLength * InvInitialLengthSource;
	const FVector CurrentSourceEndDirectionNormalized = CurrentSourceChainDirection * NormalizedLimbLength;
	
	// set position to length-scaled direction from source limb
	const FVector PelvisTranslationDelta = InPelvisMotionOp ? InPelvisMotionOp->GetPelvisTranslationOffset() : FVector::ZeroVector;
	const FVector AffectIKWeights = InPelvisMotionOp ? InPelvisMotionOp->GetAffectIKWeightAsVector() : FVector::ZeroVector;
	const FVector InvAffectIKWeights = FVector::OneVector - AffectIKWeights;
	const FVector InvRootModification = PelvisTranslationDelta * InvAffectIKWeights;
	const int32 TargetStartBoneIndex = TargetBoneChain->BoneIndices[0];
	const FVector CurrentTargetStartLocation = InTargetGlobalPose[TargetStartBoneIndex].GetTranslation() - InvRootModification;
	FVector GoalPosition = CurrentTargetStartLocation + (CurrentSourceEndDirectionNormalized * InitialTargetChainLength);

	// blend to source location
	const double BlendToSourceTranslation = Settings->BlendToSource * Settings->BlendToSourceTranslation;
	if (BlendToSourceTranslation > UE_KINDA_SMALL_NUMBER)
	{
		const FVector Weight = BlendToSourceTranslation * Settings->BlendToSourceWeights;
		FVector SourceLocation = CurrentSourceEndGlobal.GetLocation();
		if (Settings->ApplyPelvisOffsetToSourceGoals)
		{
			SourceLocation += PelvisTranslationDelta * AffectIKWeights;
		}
		GoalPosition.X = FMath::Lerp(GoalPosition.X, SourceLocation.X, Weight.X);
		GoalPosition.Y = FMath::Lerp(GoalPosition.Y, SourceLocation.Y, Weight.Y);
		GoalPosition.Z = FMath::Lerp(GoalPosition.Z, SourceLocation.Z, Weight.Z);
	}

	// apply global static offset
	GoalPosition += Settings->StaticOffset;

	// apply local static offset
	GoalPosition += GoalOutput.GetRotation().RotateVector(Settings->StaticLocalOffset);

	// apply vertical scale
	GoalPosition.Z *= Settings->ScaleVertical;
	
	// apply extension
	if (!FMath::IsNearlyEqual(Settings->Extension, 1.0f))
	{
		GoalPosition = CurrentTargetStartLocation + (GoalPosition - CurrentTargetStartLocation) * Settings->Extension;	
	}

	return MoveTemp(GoalPosition);
}

bool FRetargetIKChainSettings::operator==(const FRetargetIKChainSettings& Other) const
{
	return EnableIK == Other.EnableIK
		&& FMath::IsNearlyEqualByULP(BlendToSource, Other.BlendToSource)
		&& BlendToSourceWeights.Equals(Other.BlendToSourceWeights)
		&& ApplyPelvisOffsetToSourceGoals == Other.ApplyPelvisOffsetToSourceGoals
		&& StaticOffset.Equals(Other.StaticOffset)
		&& StaticLocalOffset.Equals(Other.StaticLocalOffset)
		&& StaticRotationOffset.Equals(Other.StaticRotationOffset)
		&& FMath::IsNearlyEqualByULP(ScaleVertical, Other.ScaleVertical)
		&& FMath::IsNearlyEqualByULP(Extension, Other.Extension);
}

bool FRetargetIKChainSettings::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);
	StaticStruct()->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(this), StaticStruct(), nullptr, nullptr);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FIKRigObjectVersion::GUID) < FIKRigObjectVersion::RemovedPelvisMotionFromSourceGoals)
		{
			ApplyPelvisOffsetToSourceGoals = true;
		}
	}
		
	return true;
}

const UClass* FIKRetargetIKChainsOpSettings::GetControllerType() const
{
	return UIKRetargetIKChainsController::StaticClass();
}

void FIKRetargetIKChainsOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the chains
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetIKChainsOpSettings, ChainsToRetarget)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetIKChainsOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);

	// copy chains
	const FIKRetargetIKChainsOpSettings* NewSettings = reinterpret_cast<const FIKRetargetIKChainsOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->ChainsToRetarget, ChainsToRetarget);
}

void FIKRetargetIKChainsOpSettings::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	FIKRetargetOpSettingsBase::PostLoad(InVersion);
	
	if (InVersion < FIKRigObjectVersion::RemovedPelvisMotionFromSourceGoals)
	{
		for (FRetargetIKChainSettings& ChainSettings : ChainsToRetarget)
		{
			ChainSettings.ApplyPelvisOffsetToSourceGoals = true;
		}
	}
}

bool FIKRetargetIKChainsOpSettings::Serialize(FArchive& Ar)
{
	return SerializeOpWithVersion(Ar, *this);
}

bool FIKRetargetIKChainsOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;
	IKChainRetargeters.Reset();

	// go through all chains to retarget and load them
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (FRetargetIKChainSettings& ChainSettings : Settings.ChainsToRetarget)
	{
		FName TargetChainName = ChainSettings.TargetChainName;
		const FResolvedBoneChain* OutSourceBoneChain = nullptr;
		const FResolvedBoneChain* OutTargetBoneChain = nullptr;
		if (!FIKRetargetRunIKRigOp::ValidateIKChain(
			TargetChainName,
			GoalContainer,
			BoneChains,
			this,
			InParentOp,
			InLog,
			OutSourceBoneChain,
			OutTargetBoneChain))
		{
			continue;
		}

		// initialize the mapped pair of source/target bone chains
		FIKChainRetargeter IKChainRetargeter;
		const bool bChainInitialized = IKChainRetargeter.Initialize(*OutSourceBoneChain, *OutTargetBoneChain, ChainSettings, InLog);
		if (!bChainInitialized)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("IKChainOpBadChain", "IK Chain Op: could not initialize a mapped retarget chain for IK, '{0}."),
			FText::FromName(TargetChainName)));
			continue;
		}
		
		// store valid chain pair to be retargeted
		IKChainRetargeters.Add(IKChainRetargeter);
	}
	
	// consider initialized if at least 1 IK chain was initialized
	bIsInitialized = !IKChainRetargeters.IsEmpty();
	return bIsInitialized;
}

void FIKRetargetIKChainsOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	SCOPE_CYCLE_COUNTER(STAT_IKRetargetGoals);
	
	if (InProcessor.IsIKForcedOff())
	{
		return; // skip this op when IK is off
	}

	// used to optionally apply pelvis offsets to the goals
	const FIKRetargetPelvisMotionOp* PelvisMotionOp = InProcessor.GetFirstRetargetOpOfType<FIKRetargetPelvisMotionOp>();

	// retarget the IK goals to their new locations based on input pose 
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (FIKChainRetargeter& IKChain : IKChainRetargeters)
	{
		// generate the new goal transform for the chain
		const FTransform& GoalTransform = IKChain.GenerateGoalTransform(
			PelvisMotionOp,
			InSourceGlobalPose,
			OutTargetGlobalPose,
			InProcessor.GetSkeleton(ERetargetSourceOrTarget::Source),
			InProcessor.GetSkeleton(ERetargetSourceOrTarget::Target));

		// set the goal transform on the IK Rig
		constexpr double PositionAlpha = 1.0;
		constexpr double RotationAlpha = 1.0;
		FIKRigGoal Goal = FIKRigGoal(
			IKChain.GetTargetChain().IKGoalName,
			IKChain.GetTargetChain().EndBone,
			GoalTransform.GetLocation(),
			GoalTransform.GetRotation(),
			PositionAlpha,
			RotationAlpha,
			EIKRigGoalSpace::Component,
			EIKRigGoalSpace::Component,
			IKChain.GetSettings()->EnableIK);
		
		GoalContainer.SetIKGoal(Goal);
	}

#if WITH_EDITOR
	SaveDebugData(InProcessor, InSourceGlobalPose, OutTargetGlobalPose);
#endif
}

void FIKRetargetIKChainsOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetIKChainsOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

FIKRetargetOpSettingsBase* FIKRetargetIKChainsOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetIKChainsOp::GetSettingsType() const
{
	return FIKRetargetIKChainsOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetIKChainsOp::GetType() const
{
	return FIKRetargetIKChainsOp::StaticStruct();
}

const UScriptStruct* FIKRetargetIKChainsOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

void FIKRetargetIKChainsOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	for (FRetargetIKChainSettings& ChainSettings : Settings.ChainsToRetarget)
	{
		if (ChainSettings.TargetChainName == InOldChainName)
		{
			ChainSettings.TargetChainName = InNewChainName;
		}
	}
}

void FIKRetargetIKChainsOp::OnParentReinitPropertyEdited(
	const FIKRetargetOpBase& InParentOp,
	const FPropertyChangedEvent* InPropertyChangedEvent)
{
	RegenerateChainSettings(&InParentOp);
}

#if WITH_EDITOR

FCriticalSection FIKRetargetIKChainsOp::DebugDataMutex;

void FIKRetargetIKChainsOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	// draw IK goals on each IK chain
	if (!(Settings.bDrawFinalGoals || Settings.bDrawSourceLocations))
	{
		return;
	}

	// locked because this is called from the main thread and debug data is modified on worker
	FScopeLock ScopeLock(&DebugDataMutex);
	
	// spin through all IK chains
	for (const FChainDebugData& ChainDebugData : AllChainsDebugData)
	{
		FTransform FinalTransform = ChainDebugData.OutputTransformEnd * InComponentTransform;
		
		bool bIsSelected = InEditorState.SelectedChains.Contains(ChainDebugData.TargetChainName);

		InPDI->SetHitProxy(new HIKRetargetEditorChainProxy(ChainDebugData.TargetChainName));

		if (Settings.bDrawFinalGoals)
		{
			FLinearColor GoalColor = bIsSelected ? InEditorState.GoalColor : InEditorState.GoalColor * InEditorState.NonSelected;
			
			IKRigDebugRendering::DrawWireCube(
			InPDI,
			FinalTransform,
			GoalColor,
			static_cast<float>(Settings.GoalDrawSize),
			static_cast<float>(Settings.GoalDrawThickness * InComponentScale));
		}
	
		if (Settings.bDrawSourceLocations)
		{
			FTransform SourceGoalTransform;
			SourceGoalTransform.SetTranslation(ChainDebugData.SourceTransformEnd.GetLocation());
			SourceGoalTransform.SetRotation(ChainDebugData.SourceTransformEnd.GetRotation());
			SourceGoalTransform *= InComponentTransform;

			FLinearColor Color = bIsSelected ? InEditorState.SourceColor : InEditorState.SourceColor * InEditorState.NonSelected;

			DrawWireSphere(
				InPDI,
				SourceGoalTransform,
				Color,
				Settings.GoalDrawSize,
				12,
				SDPG_World,
				0.0f,
				0.001f,
				false);

			if (Settings.bDrawFinalGoals)
			{
				DrawDashedLine(
					InPDI,
					SourceGoalTransform.GetLocation(),
					FinalTransform.GetLocation(),
					Color,
					1.0f,
					SDPG_Foreground);
			}
		}

		// done drawing chain proxies
		InPDI->SetHitProxy(nullptr);
	}
}

void FIKRetargetIKChainsOp::SaveDebugData(
	const FIKRetargetProcessor& InProcessor,
	const TArray<FTransform>& InSourceGlobalPose,
	const TArray<FTransform>& OutTargetGlobalPose)
{
	FScopeLock ScopeLock(&DebugDataMutex);
	
	// get the root modification
	DebugRootModification = FVector::ZeroVector;
	if (const FIKRetargetPelvisMotionOp* PelvisMotionOp = InProcessor.GetFirstRetargetOpOfType<FIKRetargetPelvisMotionOp>())
	{
		DebugRootModification = PelvisMotionOp->GetPelvisTranslationOffset() * PelvisMotionOp->GetAffectIKWeightAsVector();
	}
	
	AllChainsDebugData.Reset();
	for (const FIKChainRetargeter& IKChainPair : IKChainRetargeters)
	{
		FChainDebugData NewChainData;
		const FResolvedBoneChain& SourceChain = IKChainPair.GetSourceChain();
		const FResolvedBoneChain& TargetChain = IKChainPair.GetTargetChain();
		NewChainData.TargetChainName = IKChainPair.GetSettings()->TargetChainName;
		NewChainData.InputTransformStart = OutTargetGlobalPose[TargetChain.BoneIndices[0]];
		NewChainData.InputTransformEnd = OutTargetGlobalPose[TargetChain.BoneIndices.Last()];
		NewChainData.OutputTransformEnd = IKChainPair.GetGoalTransform();
		NewChainData.SourceTransformEnd = InSourceGlobalPose[SourceChain.BoneIndices.Last()];
		const FVector RootModification = IKChainPair.GetSettings()->ApplyPelvisOffsetToSourceGoals ? DebugRootModification : FVector::ZeroVector;
		NewChainData.SourceTransformEnd.AddToTranslation(RootModification);
		AllChainsDebugData.Add(NewChainData);
	}
}

void FIKRetargetIKChainsOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.ChainsToRetarget, InChainName);
}

bool FIKRetargetIKChainsOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.ChainsToRetarget, InChainName);
}

uint8* FIKRetargetIKChainsOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.ChainsToRetarget, InChainName);
}

#endif

void FIKRetargetIKChainsOp::RegenerateChainSettings(const FIKRetargetOpBase* InParentOp)
{
	const FIKRetargetRunIKRigOp* ParentOp = reinterpret_cast<const FIKRetargetRunIKRigOp*>(InParentOp);
	if (!ensure(ParentOp))
	{
		return;
	}
	
	// find the target chains that require goal retargeting
	const TArray<FName> RequiredTargetChains = ParentOp->GetRequiredTargetChains();
	if (RequiredTargetChains.IsEmpty())
	{
		// NOTE: if there's no chains, we don't clear the settings
		// this allows users to clear and reassign a different rig and potentially retain/restore compatible settings
		return;
	}

	// remove chains that are not required
	Settings.ChainsToRetarget.RemoveAll([&RequiredTargetChains](const FRetargetIKChainSettings& InChainSettings)
	{
		return !RequiredTargetChains.Contains(InChainSettings.TargetChainName);
	});
	
	// add any required chains not already present
	for (FName RequiredTargetChain : RequiredTargetChains)
	{
		bool bFound = false;
		for (const FRetargetIKChainSettings& ChainToRetarget : Settings.ChainsToRetarget)
		{
			if (ChainToRetarget.TargetChainName == RequiredTargetChain)
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			Settings.ChainsToRetarget.Emplace(RequiredTargetChain);
		}
	}
}

FIKRetargetIKChainsOpSettings UIKRetargetIKChainsController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetIKChainsOpSettings*>(OpSettingsToControl);
}

void UIKRetargetIKChainsController::SetSettings(FIKRetargetIKChainsOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
