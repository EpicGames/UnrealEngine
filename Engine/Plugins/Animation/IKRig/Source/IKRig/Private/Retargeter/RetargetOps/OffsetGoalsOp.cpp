// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/OffsetGoalsOp.h"

#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"

bool FIKRetargetOffsetGoalsChainSettings::operator==(const FIKRetargetOffsetGoalsChainSettings& Other) const
{
	return GlobalTranslationOffset.Equals(Other.GlobalTranslationOffset)
		&& LocalTranslationOffset.Equals(Other.LocalTranslationOffset)
		&& GlobalRotationOffset.Equals(Other.GlobalRotationOffset)
		&& LocalRotationOffset.Equals(Other.LocalRotationOffset);
}

const UClass* FIKRetargetOffsetGoalsOpSettings::GetControllerType() const
{
	return UIKRetargetOffsetGoalsController::StaticClass();
}

void FIKRetargetOffsetGoalsOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the chains
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetOffsetGoalsOpSettings, Chains)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetOffsetGoalsOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);

	// copy chains
	const FIKRetargetOffsetGoalsOpSettings* NewSettings = reinterpret_cast<const FIKRetargetOffsetGoalsOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->Chains, Chains);
}

void FIKChainToOffset::Run(FIKRigGoalContainer& InGoalContainer, const FIKRetargetOffsetGoalsOpSettings& InOpSettings) const
{
	FIKRigGoal* Goal = InGoalContainer.FindGoalByName(TargetBoneChain->IKGoalName);
	if (!ensure(Goal))
	{
		return; // goal should have been validated inside Op::Initialize()
	}

	if (Settings->Alpha < UE_KINDA_SMALL_NUMBER)
	{
		return; // blended off
	}

	// modify rotation of goal
	{
		FQuat GoalRotation = Goal->Rotation.Quaternion();
		// apply global rotation offset
		GoalRotation = Settings->GlobalRotationOffset.Quaternion() * GoalRotation;
		// apply local rotation offset
		GoalRotation = GoalRotation * Settings->LocalRotationOffset.Quaternion();
		// alpha and apply
		const float Alpha = Settings->Alpha * InOpSettings.Alpha * InOpSettings.RotationAlpha;
		GoalRotation = FQuat::FastLerp(Goal->Rotation.Quaternion(), GoalRotation, Alpha);
		GoalRotation.Normalize();
		Goal->Rotation = GoalRotation.Rotator();
	}
	
	// modify position of goal
	{
		FVector GoalPosition = Goal->Position;
		// apply global static offset
		GoalPosition += Settings->GlobalTranslationOffset;
		// apply local static offset
		GoalPosition += Goal->Rotation.Quaternion().RotateVector(Settings->LocalTranslationOffset);
		// alpha and apply
		const float Alpha = Settings->Alpha * InOpSettings.Alpha * InOpSettings.TranslationAlpha;
		GoalPosition = FMath::Lerp(Goal->Position, GoalPosition, Alpha);
		Goal->Position = GoalPosition;
	}
}

bool FIKRetargetOffsetGoalsOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;
	
	// go through all chains to retarget and load them
	ChainsToOffset.Reset();
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (FIKRetargetOffsetGoalsChainSettings& ChainSettings : Settings.Chains)
	{
		FName TargetChainName = ChainSettings.TargetChainName;
		const FResolvedBoneChain* SourceBoneChain = nullptr;
		const FResolvedBoneChain* TargetBoneChain = nullptr;
		if (!FIKRetargetRunIKRigOp::ValidateIKChain(
			TargetChainName,
			GoalContainer,
			BoneChains,
			this,
			InParentOp,
			InLog,
			SourceBoneChain /*out*/,
			TargetBoneChain /*out*/))
		{
			continue;
		}

		ChainsToOffset.Emplace(*TargetBoneChain, ChainSettings);
	}

	bIsInitialized = true;
	return true;
}

void FIKRetargetOffsetGoalsOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	if (Settings.Alpha <= UE_KINDA_SMALL_NUMBER)
	{
		return; // fast skip whole op
	}
	
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (const FIKChainToOffset& Chain : ChainsToOffset)
	{
		Chain.Run(GoalContainer, Settings);
	}
}

FIKRetargetOpSettingsBase* FIKRetargetOffsetGoalsOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetOffsetGoalsOp::GetSettingsType() const
{
	return FIKRetargetOffsetGoalsOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetOffsetGoalsOp::GetType() const
{
	return FIKRetargetOffsetGoalsOp::StaticStruct();
}

const UScriptStruct* FIKRetargetOffsetGoalsOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

void FIKRetargetOffsetGoalsOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetOffsetGoalsOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetOffsetGoalsOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	IKRetargetOpUtils::OnRetargetChainRenamed(Settings.Chains, InOldChainName, InNewChainName);
}

void FIKRetargetOffsetGoalsOp::OnParentReinitPropertyEdited(
	const FIKRetargetOpBase& InParentOp,
	const FPropertyChangedEvent* InPropertyChangedEvent)
{
	RegenerateChainSettings(&InParentOp);	
}

void FIKRetargetOffsetGoalsOp::RegenerateChainSettings(const FIKRetargetOpBase* InParentOp)
{
	// regenerate chain settings
	constexpr bool bSkipUnmappedChains = false;
	constexpr bool bSkipNonIKChains = true;
	IKRetargetOpUtils::SynchronizeChainSettingsWithIKRig(Settings.Chains, InParentOp, bSkipUnmappedChains, bSkipNonIKChains);
}

FIKRetargetOffsetGoalsOpSettings UIKRetargetOffsetGoalsController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetOffsetGoalsOpSettings*>(OpSettingsToControl);
}

void UIKRetargetOffsetGoalsController::SetSettings(FIKRetargetOffsetGoalsOpSettings InSettings)
{
	reinterpret_cast<FIKRetargetOffsetGoalsOpSettings*>(OpSettingsToControl)->CopySettingsAtRuntime(&InSettings);
}

#if WITH_EDITOR

void FIKRetargetOffsetGoalsOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.Chains, InChainName);
}

bool FIKRetargetOffsetGoalsOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.Chains, InChainName);
}

uint8* FIKRetargetOffsetGoalsOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.Chains, InChainName);
}
#endif
