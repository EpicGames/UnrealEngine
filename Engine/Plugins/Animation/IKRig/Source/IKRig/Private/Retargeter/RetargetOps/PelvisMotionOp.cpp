// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/PelvisMotionOp.h"

#include "Retargeter/IKRetargetProcessor.h"
#include "Engine/SkeletalMesh.h"

#if WITH_EDITOR
#include "PrimitiveDrawInterface.h"
#include "PrimitiveDrawingUtils.h"
#endif

#include "Retargeter/RetargetOps/AdditivePoseOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PelvisMotionOp)

#define LOCTEXT_NAMESPACE "PelvisMotionOp"


const UClass* FIKRetargetPelvisMotionOpSettings::GetControllerType() const
{
	return UIKRetargetPelvisMotionController::StaticClass();
}

void FIKRetargetPelvisMotionOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies everything except the bones we are operating on (those require reinit)
	const TArray<FName> PropertiesToIgnore = {
		GET_MEMBER_NAME_CHECKED(FIKRetargetPelvisMotionOpSettings, SourcePelvisBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetPelvisMotionOpSettings, TargetPelvisBone)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetPelvisMotionOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

#if WITH_EDITOR
USkeleton* FIKRetargetPelvisMotionOpSettings::GetSkeleton(const FName InPropertyName)
{
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FIKRetargetPelvisMotionOpSettings, SourcePelvisBone))
	{
		return const_cast<USkeleton*>(SourceSkeletonAsset);
	}

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FIKRetargetPelvisMotionOpSettings, TargetPelvisBone))
	{
		return const_cast<USkeleton*>(TargetSkeletonAsset);
	}
	
	ensureMsgf(false, TEXT("Pelvis motion op unable to get skeleton for UI widget."));
	return nullptr;
}
#endif

void FIKRetargetPelvisMotionOpSettings::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	FIKRetargetOpSettingsBase::PostLoad(InVersion);

	if (InVersion < FIKRigObjectVersion::AddedDebugDrawTogglePerOp)
	{
		bDebugDraw = bEnableDebugDraw_DEPRECATED;
	}

	if (InVersion < FIKRigObjectVersion::AddedLocalGlobalOffsetsToPelvisOp)
	{
		TranslationOffsetGlobal = TranslationOffset_DEPRECATED;
		RotationOffsetLocal = RotationOffset_DEPRECATED;
	}
}

bool FIKRetargetPelvisMotionOpSettings::Serialize(FArchive& Ar)
{
	return SerializeOpWithVersion(Ar, *this);
}

bool FIKRetargetPelvisMotionOp::Initialize(
	const FIKRetargetProcessor& Processor,
	const FRetargetSkeleton& SourceSkeleton,
	const FTargetSkeleton& TargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& Log)
{
	bIsInitialized = false;
	
	// reset root data
	Reset();
	
	// initialize root encoder
	const FName SourcePelvisBoneName = Settings.SourcePelvisBone.BoneName;
	const bool bPelvisEncoderInit = InitializeSource(SourcePelvisBoneName, SourceSkeleton, Log);
	if (!bPelvisEncoderInit)
	{
		Log.LogWarning( FText::Format(
			LOCTEXT("NoSourceRoot", "IK Retargeter unable to initialize source root, '{0}' on skeletal mesh: '{1}'"),
			FText::FromName(SourcePelvisBoneName), FText::FromString(SourceSkeleton.SkeletalMesh->GetName())));
	}

	// initialize root decoder
	const FName TargetPelvisBoneName = Settings.TargetPelvisBone.BoneName;
	const bool bPelvisDecoderInit = InitializeTarget(TargetPelvisBoneName, TargetSkeleton, Log);
	if (!bPelvisDecoderInit)
	{
		Log.LogWarning( FText::Format(
			LOCTEXT("NoTargetRoot", "IK Retargeter unable to initialize target root, '{0}' on skeletal mesh: '{1}'"),
			FText::FromName(TargetPelvisBoneName), FText::FromString(TargetSkeleton.SkeletalMesh->GetName())));
	}

#if WITH_EDITOR
	// record skeletons for UI bone selector widgets
	Settings.SourceSkeletonAsset = SourceSkeleton.SkeletalMesh->GetSkeleton();
	Settings.TargetSkeletonAsset = TargetSkeleton.SkeletalMesh->GetSkeleton();
#endif

	bIsInitialized = bPelvisEncoderInit && bPelvisDecoderInit;
	return bIsInitialized;
}

void FIKRetargetPelvisMotionOp::Run(
	FIKRetargetProcessor& Processor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	FTransform NewPelvisGlobal;
	RetargetPelvisTransform(InSourceGlobalPose, NewPelvisGlobal);
	
	// update global transforms below pelvis
	FTargetSkeleton& TargetSkeleton = Processor.GetTargetSkeleton();
	TargetSkeleton.SetGlobalTransformAndUpdateChildren(Target.BoneIndex, NewPelvisGlobal, TargetSkeleton.OutputGlobalPose);

#if WITH_EDITOR
	CurrentPelvisTransform = NewPelvisGlobal;
#endif
}

void FIKRetargetPelvisMotionOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	// copy the source/target pelvis from the default IK Rig
	if (const UIKRigDefinition* SourceIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source))
	{
		Settings.SourcePelvisBone.BoneName = SourceIKRig->GetPelvis();
	}
	
	if (const UIKRigDefinition* TargetIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target))
	{
		Settings.TargetPelvisBone.BoneName = TargetIKRig->GetPelvis();
	}
}

void FIKRetargetPelvisMotionOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	if (!InIKRig)
	{
		return;
	}
	
	FName& PelvisNameSetting = SourceOrTarget == ERetargetSourceOrTarget::Source ? Settings.SourcePelvisBone.BoneName : Settings.TargetPelvisBone.BoneName;
	PelvisNameSetting = InIKRig->GetPelvis();
}

bool FIKRetargetPelvisMotionOp::InitializeSource(
	const FName SourcePelvisBoneName,
	const FRetargetSkeleton& SourceSkeleton,
	FIKRigLogger& Log)
{
	// validate target root bone exists
	Source.BoneName = SourcePelvisBoneName;
	Source.BoneIndex = SourceSkeleton.FindBoneIndexByName(SourcePelvisBoneName);
	if (Source.BoneIndex == INDEX_NONE)
	{
		Log.LogWarning( FText::Format(
			LOCTEXT("MissingSourceRoot", "IK Retargeter could not find source root bone, {0} in mesh {1}"),
			FText::FromName(SourcePelvisBoneName), FText::FromString(SourceSkeleton.SkeletalMesh->GetName())));
		return false;
	}
	
	// record initial root data
	const FTransform InitialTransform = SourceSkeleton.RetargetPoses.GetGlobalRetargetPose()[Source.BoneIndex]; 
	float InitialHeight = static_cast<float>(InitialTransform.GetTranslation().Z);
	Source.InitialPosition = Source.CurrentPosition = InitialTransform.GetLocation();
	Source.InitialRotation = Source.CurrentRotation = InitialTransform.GetRotation();

	// ensure root height is not at origin, this happens if user sets root to ACTUAL skeleton root and not pelvis
	if (InitialHeight < UE_KINDA_SMALL_NUMBER)
	{
		// warn user and push it up slightly to avoid divide by zero
		Log.LogError(LOCTEXT("BadPelvisHeight", "The source pelvis bone is very near the ground plane. This will cause the target to be moved very far. To resolve this, please create a retarget pose with the pelvis at the correct height off the ground."));
		InitialHeight = 1.0f;
	}

	// invert height
	Source.InitialHeightInverse = 1.0f / InitialHeight;

	return true;
}

bool FIKRetargetPelvisMotionOp::InitializeTarget(
	const FName TargetPelvisBoneName,
	const FTargetSkeleton& TargetSkeleton,
	FIKRigLogger& Log)
{
	// validate target root bone exists
	Target.BoneName = TargetPelvisBoneName;
	Target.BoneIndex = TargetSkeleton.FindBoneIndexByName(TargetPelvisBoneName);
	if (Target.BoneIndex == INDEX_NONE)
	{
		Log.LogWarning( FText::Format(
			LOCTEXT("CountNotFindRootBone", "IK Retargeter could not find target root bone, {0} in mesh {1}"),
			FText::FromName(TargetPelvisBoneName), FText::FromString(TargetSkeleton.SkeletalMesh->GetName())));
		return false;
	}

	const FTransform TargetInitialTransform = TargetSkeleton.RetargetPoses.GetGlobalRetargetPose()[Target.BoneIndex];
	Target.InitialHeight = static_cast<float>(TargetInitialTransform.GetTranslation().Z);
	Target.InitialRotation = TargetInitialTransform.GetRotation();
	Target.InitialPosition = TargetInitialTransform.GetTranslation();

	// initialize the global scale factor
	const float ScaleFactor = Source.InitialHeightInverse * Target.InitialHeight;
	GlobalScaleFactor.Set(ScaleFactor, ScaleFactor, ScaleFactor);
	
	return true;
}

FIKRetargetOpSettingsBase* FIKRetargetPelvisMotionOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetPelvisMotionOp::GetSettingsType() const
{
	return FIKRetargetPelvisMotionOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetPelvisMotionOp::GetType() const
{
	return FIKRetargetPelvisMotionOp::StaticStruct();
}

void FIKRetargetPelvisMotionOp::CollectRetargetedBones(TSet<int32>& OutRetargetedBones) const
{
	// the pelvis bone is retargeted
	if (Target.BoneIndex != INDEX_NONE)
	{
		OutRetargetedBones.Add(Target.BoneIndex);
	}
}

FName FIKRetargetPelvisMotionOp::GetPelvisBoneName(ERetargetSourceOrTarget SourceOrTarget) const
{
	if (!bIsInitialized)
	{
		return NAME_None;
	}

	return SourceOrTarget == ERetargetSourceOrTarget::Source ? Source.BoneName : Target.BoneName;
}

FVector FIKRetargetPelvisMotionOp::GetGlobalScaleVector() const
{
	return GlobalScaleFactor * FVector(Settings.ScaleHorizontal, Settings.ScaleHorizontal, Settings.ScaleVertical);
}

FVector FIKRetargetPelvisMotionOp::GetAffectIKWeightAsVector() const
{
	return FVector(Settings.AffectIKHorizontal, Settings.AffectIKHorizontal, Settings.AffectIKVertical);
}

FVector FIKRetargetPelvisMotionOp::GetPelvisTranslationOffset() const
{
	return Target.PelvisTranslationDelta;
}

int32 FIKRetargetPelvisMotionOp::GetPelvisBoneIndex(ERetargetSourceOrTarget SourceOrTarget) const
{
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? Source.BoneIndex : Target.BoneIndex;
}

void FIKRetargetPelvisMotionOp::Reset()
{
	Source = FPelvisSource();
	Target = FPelvisTarget();
}

void FIKRetargetPelvisMotionOp::RetargetPelvisTransform(
	const TArray<FTransform>& SourceGlobalPose,
	FTransform& OutPelvisGlobalPose)
{
	// get source pelvis transform
	const FTransform& SourceTransform = SourceGlobalPose[Source.BoneIndex];
	Source.CurrentPosition = SourceTransform.GetTranslation();
	Source.CurrentRotation = SourceTransform.GetRotation();
	
	// retarget rotation
	// NOTE: rotation must be done before translation to get proper orientation for local position offsets
	FQuat Rotation;
	{
		// calc offset between initial source/target root rotations
		const FQuat RotationDelta = Source.CurrentRotation * Source.InitialRotation.Inverse();
		// add retarget pose delta to the current source rotation
		const FQuat RetargetedRotation = RotationDelta * Target.InitialRotation;

		// apply static LOCAL rotation offset
		Rotation = RetargetedRotation * Settings.RotationOffsetLocal.Quaternion();
		// apply static GLOBAL rotation offset
		Rotation = Settings.RotationOffsetGlobal.Quaternion() * Rotation;

		// blend with alpha
		Rotation = FQuat::FastLerp(Target.InitialRotation, Rotation, Settings.RotationAlpha);
		Rotation.Normalize();

		// record the delta created by all the modifications made to the root rotation
		Target.PelvisRotationDelta = RetargetedRotation * Target.InitialRotation.Inverse();
	}
	
	// retarget position
	FVector Position;
	{
		// generate pelvis motion scaled by the relative height ratio of the target to the source height
		// ie, if source is 2x taller than target, motion is scaled by 0.5
		const FVector NormalizedSourcePosition = Source.CurrentPosition * Source.InitialHeightInverse;
		const FVector PositionScaled = NormalizedSourcePosition * Target.InitialHeight;

		// generate pelvis motion as absolute delta
		const FVector AbsoluteDelta = Source.CurrentPosition - Source.InitialPosition;
		const FVector PositionAbsolute = Target.InitialPosition + AbsoluteDelta;

		// blend between scaled and absolute motion
		Position = FMath::Lerp(PositionScaled, PositionAbsolute, Settings.BlendToAbsoluteOffset);

		// apply crotch-to-floor constraint
		if (!FMath::IsNearlyZero(Settings.FloorConstraintWeight))
		{
			if (Source.CurrentPosition.Z <= Source.InitialPosition.Z)
			{
				// calculate the constrained height as a proportional target height relative to the source
				const float SourceInitialCrotchHeight = Source.InitialPosition.Z - Settings.SourceCrotchOffset;
				const float SourceCurrentCrotchHeight = Source.CurrentPosition.Z - Settings.SourceCrotchOffset;
				const float SourceCrotchHeightPercent = SourceCurrentCrotchHeight * (1.0f / FMath::Max(1.0f, SourceInitialCrotchHeight));
				const float TargetInitialCrotchHeight = Target.InitialPosition.Z - Settings.TargetCrotchOffset;
				const float TargetCurrentCrotchHeight = TargetInitialCrotchHeight * SourceCrotchHeightPercent;
				const float ConstrainedHeight = TargetCurrentCrotchHeight + Settings.TargetCrotchOffset;

				// optionally apply ground proximity falloff to the floor constraint weight
				float FinalWeight = Settings.FloorConstraintWeight;
				if (Settings.bUseGroundFalloff)
				{
					// get how close we are to the ground (1.0 = Touching Floor, 0.0 = Standing)
					const float FalloffStartHeight = Target.InitialHeight * Settings.GroundFalloffHeightPercent;
					const float ProximityFalloff = 1.0f - FMath::Clamp(Position.Z, 0.0f, FalloffStartHeight) / FalloffStartHeight;
					FinalWeight *= ProximityFalloff;
				}
				
				Position.Z = FMath::Lerp(Position.Z, ConstrainedHeight, FinalWeight);
			}
		}

		// record the retargeted position before applying any modifications
		const FVector RetargetedPosition = Position;
		
		// blend the pelvis position towards the source pelvis position
		const FVector PerAxisAlpha = Settings.BlendToSourceTranslation * Settings.BlendToSourceTranslationWeights;
		Position = FMath::Lerp(Position, Source.CurrentPosition, PerAxisAlpha);

		// apply vertical / horizontal scaling of motion
		FVector ScaledRetargetedPosition = Position;
		ScaledRetargetedPosition.Z *= Settings.ScaleVertical;
		const FVector HorizontalOffset = (ScaledRetargetedPosition - Target.InitialPosition) * FVector(Settings.ScaleHorizontal, Settings.ScaleHorizontal, 1.0f);
		Position = Target.InitialPosition + HorizontalOffset;
		
		// apply a static GLOBAL offset
		Position += Settings.TranslationOffsetGlobal;

		// apply a static LOCAL offset
		Position += Rotation.RotateVector(Settings.TranslationOffsetLocal);

		// blend with alpha
		Position = FMath::Lerp(Target.InitialPosition, Position, Settings.TranslationAlpha);

		// record the delta created by all the modifications made to the root translation
		Target.PelvisTranslationDelta = Position - RetargetedPosition;
	}

	// apply to target
	OutPelvisGlobalPose.SetTranslation(Position);
	OutPelvisGlobalPose.SetRotation(Rotation);
}

#if WITH_EDITOR

FText FIKRetargetPelvisMotionOp::GetWarningMessage() const
{
	if (bIsInitialized)
	{
		return FText::Format(LOCTEXT("PelvisOpSuccess", "Running on {0}."), FText::FromName(Target.BoneName));
	}
	
	return FIKRetargetOpBase::GetWarningMessage();
}

FCriticalSection FIKRetargetPelvisMotionOp::DebugDataMutex;

void FIKRetargetPelvisMotionOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	FScopeLock ScopeLock(&DebugDataMutex);
	
	const FTransform SourcePelvis = FTransform(Source.CurrentRotation, Source.CurrentPosition) * InSourceTransform;
	const FTransform TargetPelvis = CurrentPelvisTransform * InComponentTransform;
	const FVector TargetFloor = TargetPelvis.GetLocation() * FVector(1,1,0);
	const FVector TargetCrotch = TargetPelvis.GetLocation() + FVector(0,0, -Settings.TargetCrotchOffset);
	const FVector SourceFloor = SourcePelvis.GetLocation() * FVector(1,1,0);
	const FVector SourceCrotch = SourcePelvis.GetLocation() + FVector(0,0, -Settings.SourceCrotchOffset);
	const FLinearColor CircleColor = InEditorState.bIsRootSelected ? InEditorState.MainColor : InEditorState.MainColor * InEditorState.NonSelected;
	const float Thickness = static_cast<float>(Settings.DebugDrawThickness * InComponentScale);
	const float Size = static_cast<float>(Settings.DebugDrawSize * InComponentScale);

	InPDI->SetHitProxy(new HIKRetargetEditorRootProxy());

	// floor coordinate system
	DrawCoordinateSystem(
		InPDI,
		TargetFloor,
		TargetPelvis.GetRotation().Rotator(),
		Size * 2.0f,
		SDPG_World,
		Thickness * 0.25f);
	
	// draw pelvis locations
	InPDI->DrawPoint(SourcePelvis.GetLocation(), FLinearColor::Black, Size, SDPG_Foreground);
	InPDI->DrawPoint(TargetPelvis.GetLocation(), FLinearColor::Black, Size, SDPG_Foreground);

	// draw crotch locations
	InPDI->DrawPoint(SourceCrotch, FLinearColor::Green, Size, SDPG_Foreground);
	InPDI->DrawPoint(TargetCrotch, FLinearColor::Green, Size, SDPG_Foreground);

	// draw pelvis-to-crotch lines
	DrawDashedLine(InPDI,SourcePelvis.GetLocation(),SourceCrotch, FLinearColor::Black, 1.0f,SDPG_Foreground);
	DrawDashedLine(InPDI,TargetPelvis.GetLocation(),TargetCrotch, FLinearColor::Black, 1.0f,SDPG_Foreground);
	
	// draw crotch-to-floor lines
	DrawDashedLine(InPDI,SourceCrotch,SourceFloor, CircleColor, 1.0f,SDPG_Foreground);
	DrawDashedLine(InPDI,TargetCrotch,TargetFloor, CircleColor, 1.0f,SDPG_Foreground);
	
	InPDI->SetHitProxy(nullptr);
}
#endif

FIKRetargetPelvisMotionOpSettings UIKRetargetPelvisMotionController::GetSettings()
{
	return GetPelvisOpSettings();
}

void UIKRetargetPelvisMotionController::SetSettings(FIKRetargetPelvisMotionOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

void UIKRetargetPelvisMotionController::SetSourcePelvisBone(const FName InSourcePelvisBone)
{
	GetPelvisOpSettings().SourcePelvisBone.BoneName = InSourcePelvisBone;
}

FName UIKRetargetPelvisMotionController::GetSourcePelvisBone()
{
	return GetPelvisOpSettings().SourcePelvisBone.BoneName;
}

void UIKRetargetPelvisMotionController::SetTargetPelvisBone(const FName InTargetPelvisBone)
{
	GetPelvisOpSettings().TargetPelvisBone.BoneName = InTargetPelvisBone;
}

FName UIKRetargetPelvisMotionController::GetTargetPelvisBone()
{
	return GetPelvisOpSettings().TargetPelvisBone.BoneName;
}

FIKRetargetPelvisMotionOpSettings& UIKRetargetPelvisMotionController::GetPelvisOpSettings() const
{
	return *reinterpret_cast<FIKRetargetPelvisMotionOpSettings*>(OpSettingsToControl);
}

#undef LOCTEXT_NAMESPACE
