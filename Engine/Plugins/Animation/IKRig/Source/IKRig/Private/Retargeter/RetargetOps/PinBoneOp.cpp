// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/PinBoneOp.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Engine/SkeletalMesh.h"

#include "IKRigObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PinBoneOp)

#define LOCTEXT_NAMESPACE "PinBoneOp"


void FPinBoneData::CachePinData(ERetargetSourceOrTarget InSkeletonToCopyFrom, const FIKRetargetProcessor& Processor)
{
	LastUsedSourceScale = Processor.GetPoseScale(ERetargetSourceOrTarget::Source);

    // get skeletons we are copying from/to
    const FRetargetSkeleton& SkeletonToCopyFrom = Processor.GetSkeleton(InSkeletonToCopyFrom);
    const FRetargetSkeleton& SkeletonToCopyTo = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);

	// get retarget pose of source and target
	const TArray<FTransform>& FromRetargetPose = SkeletonToCopyFrom.RetargetPoses.GetGlobalRetargetPose();
	const TArray<FTransform>& ToRetargetPose = SkeletonToCopyTo.RetargetPoses.GetGlobalRetargetPose();

	// get relevant transforms from the retarget poses
	FTransform BoneToCopyFromParentRefPoseGlobal = SkeletonToCopyFrom.GetParentTransform(BoneToCopyFrom.BoneIndex, FromRetargetPose);
	FTransform BoneToCopyToParentRefPoseGlobal = SkeletonToCopyTo.GetParentTransform(BoneToCopyTo.BoneIndex, ToRetargetPose);
	FTransform BoneToCopyFromRefPoseGlobal = FromRetargetPose[BoneToCopyFrom.BoneIndex];
	FTransform BoneToCopyToRefPoseGlobal = ToRetargetPose[BoneToCopyTo.BoneIndex];

	// cache offset from BoneToCopyFrom to BoneToCopyTo in ref pose
    OffsetFromBoneToCopyFromInRefPose = BoneToCopyFromRefPoseGlobal.GetRelativeTransform(BoneToCopyToRefPoseGlobal);
	// cache local transform of BoneToCopyFrom
    LocalRefPoseBoneToCopyFrom = BoneToCopyFromParentRefPoseGlobal.GetRelativeTransform(BoneToCopyFromRefPoseGlobal);
	// cache local transform of BoneToCopyTo in ref pose
    LocalRefPoseBoneToCopyTo = BoneToCopyToParentRefPoseGlobal.GetRelativeTransform(BoneToCopyToRefPoseGlobal);
}

void FPinBoneData::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FIKRigObjectVersion::GUID) < FIKRigObjectVersion::PinBoneTypeAndOffsetsUpgraded)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			BoneToCopyTo.BoneName = BoneToPin_DEPRECATED;
			BoneToCopyFrom.BoneName = BoneToPinTo_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

const UClass* FIKRetargetPinBoneOpSettings::GetControllerType() const
{
	return UIKRetargetPinBoneController::StaticClass();
}

void FIKRetargetPinBoneOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies everything except the bones we are operating on (those require reinit)
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetPinBoneOpSettings, BonesToPin)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetPinBoneOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

#if WITH_EDITOR
USkeleton* FIKRetargetPinBoneOpSettings::GetSkeleton(const FName InPropertyName)
{
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FPinBoneData, BoneToCopyTo))
	{
		// bone to copy TO is ALWAYS target
		return const_cast<USkeleton*>(TargetSkeletonAsset);
	}
	
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FPinBoneData, BoneToCopyFrom))
	{
		// bone to copy FROM may be either source or target
		const USkeleton* Skeleton = SkeletonToCopyFrom == ERetargetSourceOrTarget::Target ? TargetSkeletonAsset : SourceSkeletonAsset;
		return const_cast<USkeleton*>(Skeleton);
	}
	
	ensureMsgf(false, TEXT("PinBoneOp unable to get skeleton for UI widget."));
	return nullptr;
}
#endif

void FIKRetargetPinBoneOpSettings::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	FIKRetargetOpSettingsBase::PostLoad(InVersion);

	if (InVersion < FIKRigObjectVersion::PinBoneTypeAndOffsetsUpgraded)
	{
		for (FPinBoneData& BoneToPin : BonesToPin)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			BoneToPin.BoneToCopyTo.BoneName = BoneToPin.BoneToPin_DEPRECATED;
			BoneToPin.BoneToCopyFrom.BoneName = BoneToPin.BoneToPinTo_DEPRECATED;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
}

bool FIKRetargetPinBoneOpSettings::Serialize(FArchive& Ar)
{
	return SerializeOpWithVersion(Ar, *this);
}

bool FIKRetargetPinBoneOp::Initialize(
	const FIKRetargetProcessor& Processor,
	const FRetargetSkeleton& SourceSkeleton,
	const FTargetSkeleton& TargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& Log)
{
	bFoundAllBonesToPin = true;
	
	for (FPinBoneData& PinData : Settings.BonesToPin)
	{
		PinData.BoneToCopyTo.BoneIndex = TargetSkeleton.FindBoneIndexByName(PinData.BoneToCopyTo.BoneName);
		
		if (Settings.SkeletonToCopyFrom == ERetargetSourceOrTarget::Source)
		{
			PinData.BoneToCopyFrom.BoneIndex = SourceSkeleton.FindBoneIndexByName(PinData.BoneToCopyFrom.BoneName);
		}
		else
		{
			PinData.BoneToCopyFrom.BoneIndex = TargetSkeleton.FindBoneIndexByName(PinData.BoneToCopyFrom.BoneName);
		}

		const bool bFoundBoneToPin = PinData.BoneToCopyTo.BoneIndex != INDEX_NONE;
		const bool bFoundBoneToPinTo = PinData.BoneToCopyFrom.BoneIndex != INDEX_NONE;
		if (!bFoundBoneToPin)
		{
			bFoundAllBonesToPin = false;
			Log.LogWarning(FText::Format(
				LOCTEXT("MissingSourceBone", "Pin Bone retarget op refers to non-existant bone to pin, {0}."),
				FText::FromName(PinData.BoneToCopyTo.BoneName)));
		}

		if (!bFoundBoneToPinTo)
		{
			bFoundAllBonesToPin = false;
			Log.LogWarning(FText::Format(
				LOCTEXT("MissingTargetBone", "Pin Bone retarget op refers to non-existant bone to pin to, {0}."),
				FText::FromName(PinData.BoneToCopyFrom.BoneName)));
		}

		// force reinitialization
		PinData.LastUsedSourceScale.Factor = -1.0f;
	}
	
	// always treat this op as "initialized", individual pins will only execute if their prerequisites are met
	bIsInitialized = true;
	return true;
}

void FIKRetargetPinBoneOp::Run(
	FIKRetargetProcessor& Processor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	const TArray<FTransform>& PoseToCopyFrom = Settings.SkeletonToCopyFrom == ERetargetSourceOrTarget::Source ? InSourceGlobalPose : OutTargetGlobalPose;
	
	for (FPinBoneData& PinData : Settings.BonesToPin)
	{
		bool bValidBoneToCopyFrom = PoseToCopyFrom.IsValidIndex(PinData.BoneToCopyFrom.BoneIndex);
		bool bValidBoneToCopyTo = OutTargetGlobalPose.IsValidIndex(PinData.BoneToCopyTo.BoneIndex);
		if (!bValidBoneToCopyFrom || !bValidBoneToCopyTo)
		{
			continue; // disabled or not successfully initialized
		}

		// recache the offsets if the source is scaled differently
		if (PinData.LastUsedSourceScale != Processor.GetPoseScale(ERetargetSourceOrTarget::Source))
		{
			PinData.CachePinData(Settings.SkeletonToCopyFrom, Processor);
		}

		// calculate new transform for bone to pin
		const FTransform NewTransform = GetNewBoneTransform(PinData, Processor, InSourceGlobalPose, OutTargetGlobalPose);
		// apply static local and global offsets
		FTransform Result = Settings.LocalOffset * (NewTransform * Settings.GlobalOffset);

		// filter channels
		const FTransform& BoneToCopyToTransform = OutTargetGlobalPose[PinData.BoneToCopyTo.BoneIndex];
		if (!Settings.bCopyTranslation)
		{
			Result.SetTranslation(BoneToCopyToTransform.GetLocation());
		}
		if (!Settings.bCopyRotation)
		{
			Result.SetRotation(BoneToCopyToTransform.GetRotation());
		}
		if (!Settings.bCopyScale)
		{
			Result.SetScale3D(BoneToCopyToTransform.GetScale3D());
		}

		// apply to pose
		if (Settings.bPropagateToChildren)
		{
			// assign result and update children
			const FRetargetSkeleton& TargetSkeleton = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);
			TargetSkeleton.SetGlobalTransformAndUpdateChildren(PinData.BoneToCopyTo.BoneIndex, Result,OutTargetGlobalPose);
		}
		else
		{
			// assign result directly
			OutTargetGlobalPose[PinData.BoneToCopyTo.BoneIndex] = Result;
		}
	}
}

FIKRetargetOpSettingsBase* FIKRetargetPinBoneOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetPinBoneOp::GetSettingsType() const
{
	return FIKRetargetPinBoneOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetPinBoneOp::GetType() const
{
	return FIKRetargetPinBoneOp::StaticStruct();
}

#if WITH_EDITOR
FText FIKRetargetPinBoneOp::GetWarningMessage() const
{
	if (IsInitialized() && IsEnabled())
	{
		if (bFoundAllBonesToPin)
		{
			return FText::Format(LOCTEXT("ReadyToRun", "Running on {0} bone(s)."), FText::AsNumber(Settings.BonesToPin.Num()));
		}
		else
		{
			return FText::Format(LOCTEXT("MissingBones", "Running, but missing {0} bones. See log."), FText::AsNumber(Settings.BonesToPin.Num()));
		}
	}
	
	return FIKRetargetOpBase::GetWarningMessage();
}
#endif

FTransform FIKRetargetPinBoneOp::GetNewBoneTransform(
	const FPinBoneData& PinData,
	const FIKRetargetProcessor& Processor,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose) const
{
	const TArray<FTransform>& PoseToCopyFrom = Settings.SkeletonToCopyFrom == ERetargetSourceOrTarget::Source ? InSourceGlobalPose : OutTargetGlobalPose;
	const FRetargetSkeleton& CopyFrom = Processor.GetSkeleton(Settings.SkeletonToCopyFrom);
	const FRetargetSkeleton& CopyTo = Processor.GetSkeleton(ERetargetSourceOrTarget::Target);

	// get the current translational offset of BoneToCopyFrom relative to it's parent
	auto GetBoneToCopyFromCurrentVector = [PoseToCopyFrom, CopyFrom, PinData]()
	{
		const FTransform& BoneToCopyFromCurrent = PoseToCopyFrom[PinData.BoneToCopyFrom.BoneIndex];
		const FTransform& ParentOfBoneToCopyFromCurrent = CopyFrom.GetParentTransform(PinData.BoneToCopyFrom.BoneIndex, PoseToCopyFrom);
		return BoneToCopyFromCurrent.GetTranslation() - ParentOfBoneToCopyFromCurrent.GetTranslation();
	};
	
	// the resulting offset to return
	FTransform Result = FTransform::Identity;

	// generate translational offset
	switch (Settings.TranslationMode)
	{
	case EPinBoneTranslationMode::CopyGlobalPosition:
		{
			Result.SetTranslation(PoseToCopyFrom[PinData.BoneToCopyFrom.BoneIndex].GetTranslation());
			break;
		}
		
	case EPinBoneTranslationMode::CopyGlobalPositionAndMaintainOffset:
		{
			const FTransform BoneToCopyFromTransform = PoseToCopyFrom[PinData.BoneToCopyFrom.BoneIndex];
			Result.SetTranslation((PinData.OffsetFromBoneToCopyFromInRefPose * BoneToCopyFromTransform).GetTranslation());
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPosition:
		{
			const FVector BoneToCopyFromCurrentVector = GetBoneToCopyFromCurrentVector();
			const FTransform& ParentOfBoneToCopyToCurrent = CopyTo.GetParentTransform(PinData.BoneToCopyTo.BoneIndex, OutTargetGlobalPose);
			Result.SetTranslation(ParentOfBoneToCopyToCurrent.GetTranslation() + BoneToCopyFromCurrentVector);
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPositionRelativeOffset:
		{
			const double RestPoseLengthDifference = PinData.LocalRefPoseBoneToCopyTo.GetTranslation().Size() - PinData.LocalRefPoseBoneToCopyFrom.GetTranslation().Size();
			const FVector BoneToCopyFromTranslationCurrent = GetBoneToCopyFromCurrentVector();
			double CurrentLength;
			FVector BoneToCopyFromTranslationNorm;
			BoneToCopyFromTranslationCurrent.ToDirectionAndLength(BoneToCopyFromTranslationNorm, CurrentLength);
			const FVector BoneToCopyFromRelativeOffset = BoneToCopyFromTranslationNorm * (CurrentLength + RestPoseLengthDifference);
		
			const FTransform& ParentOfBoneToCopyToCurrent = CopyTo.GetParentTransform(PinData.BoneToCopyTo.BoneIndex, OutTargetGlobalPose);
			Result.SetTranslation(ParentOfBoneToCopyToCurrent.GetTranslation() + BoneToCopyFromRelativeOffset);
			break;
		}
		
	case EPinBoneTranslationMode::CopyLocalPositionRelativeScaled:
		{
			const FVector BoneToCopyFromTranslationCurrent = GetBoneToCopyFromCurrentVector();
			double LengthOfBoneToCopyFromCurrent;
			FVector BoneToCopyFromTranslationNorm;
			BoneToCopyFromTranslationCurrent.ToDirectionAndLength(BoneToCopyFromTranslationNorm, LengthOfBoneToCopyFromCurrent);
			
			const double LengthOfBoneToCopyFromInRefPose = PinData.LocalRefPoseBoneToCopyFrom.GetTranslation().Size();
			const double BoneLengthScaleFactor = LengthOfBoneToCopyFromCurrent / LengthOfBoneToCopyFromInRefPose;
			
			const double LengthOfBoneToCopyToInRefPose = PinData.LocalRefPoseBoneToCopyTo.GetTranslation().Size();
			const FVector BoneToCopyFromRelativeOffset = BoneToCopyFromTranslationNorm * (LengthOfBoneToCopyToInRefPose * BoneLengthScaleFactor);
		
			const FTransform& ParentOfBoneToCopyToCurrent = CopyTo.GetParentTransform(PinData.BoneToCopyTo.BoneIndex, OutTargetGlobalPose);
			Result.SetTranslation(ParentOfBoneToCopyToCurrent.GetTranslation() + BoneToCopyFromRelativeOffset);
			
			break;
		}
		
	default:
		ensureAlwaysMsgf(false, TEXT("Missing translation offset mode."));
		break;
	}

	// generate rotational offset
	switch (Settings.RotationMode)
	{
	case EPinBoneRotationMode::CopyGlobalRotation:
		{
			Result.SetRotation(PoseToCopyFrom[PinData.BoneToCopyFrom.BoneIndex].GetRotation());
			break;
		}
	case EPinBoneRotationMode::MaintainOffsetFromBoneToCopyFrom:
		{
			// get global retarget pose of from/to bones
			const TArray<FTransform>& FromRetargetPose = CopyFrom.RetargetPoses.GetGlobalRetargetPose();
			const TArray<FTransform>& ToRetargetPose = CopyTo.RetargetPoses.GetGlobalRetargetPose();
			const FQuat& SourceRefRot = FromRetargetPose[PinData.BoneToCopyFrom.BoneIndex].GetRotation();
			const FQuat& TargetRefRot = ToRetargetPose[PinData.BoneToCopyTo.BoneIndex].GetRotation();

			// get rotation delta of from bone
			const FQuat& SourceCurrentRot = PoseToCopyFrom[PinData.BoneToCopyFrom.BoneIndex].GetRotation();
			const FQuat SourceDelta = SourceCurrentRot * SourceRefRot.Inverse();

			// apply delta to the "to" retarget pose
			Result.SetRotation(SourceDelta * TargetRefRot);
			
			break;
		}
		
	default:
		ensureAlwaysMsgf(false, TEXT("Missing rotation offset mode."));
		break;
	}

	return Result;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UPinBoneOp::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FIKRigObjectVersion::GUID) < FIKRigObjectVersion::PinBoneTypeAndOffsetsUpgraded)
		{
			// load the boolean "bMaintainOffset" and convert it into the equivalent translation and rotation offset modes
			if (bMaintainOffset_DEPRECATED)
			{
				TranslationMode = EPinBoneTranslationMode::CopyGlobalPositionAndMaintainOffset;
				RotationMode = EPinBoneRotationMode::MaintainOffsetFromBoneToCopyFrom;
			}
			else
			{
				TranslationMode = EPinBoneTranslationMode::CopyGlobalPosition;
				RotationMode = EPinBoneRotationMode::CopyGlobalRotation;
			}

			// load the "PinType" enum and convert it into the equivalent trans/rot/scale toggles
			switch (PinType_DEPRECATED)
			{
			case EPinBoneType::FullTransform:
				{
					bCopyTranslation = true;
					bCopyRotation = true;
					bCopyScale = true;
					break;
				}
			case EPinBoneType::TranslateOnly:
				{
					bCopyTranslation = true;
					bCopyRotation = false;
					bCopyScale = false;
					break;
				}
			case EPinBoneType::RotateOnly:
				{
					bCopyTranslation = false;
					bCopyRotation = true;
					bCopyScale = false;
					break;
				}
			case EPinBoneType::ScaleOnly:
				{
					bCopyTranslation = false;
					bCopyRotation = false;
					bCopyScale = true;
					break;
				}
			default:
				break;
			}
		}
	}
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FIKRetargetPinBoneOpSettings UIKRetargetPinBoneController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetPinBoneOpSettings*>(OpSettingsToControl);
}

void UIKRetargetPinBoneController::SetSettings(FIKRetargetPinBoneOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

void UIKRetargetPinBoneController::ClearAllBonePairs()
{
	FIKRetargetPinBoneOpSettings* Settings = reinterpret_cast<FIKRetargetPinBoneOpSettings*>(OpSettingsToControl);
	Settings->BonesToPin.Reset();
}

void UIKRetargetPinBoneController::SetBonePair(const FName InBoneToCopyFrom, const FName InBoneToCopyTo)
{
	FIKRetargetPinBoneOpSettings* Settings = reinterpret_cast<FIKRetargetPinBoneOpSettings*>(OpSettingsToControl);

	// update existing pair with new bone to copy from (if there is one)
	for (FPinBoneData& BonePair : Settings->BonesToPin)
	{
		if (BonePair.BoneToCopyTo == InBoneToCopyTo)
		{
			BonePair.BoneToCopyFrom = InBoneToCopyFrom;
			return;
		}
	}

	// needs new bone pair
	FPinBoneData NewBonePair;
	NewBonePair.BoneToCopyFrom.BoneName = InBoneToCopyFrom;
	NewBonePair.BoneToCopyTo.BoneName = InBoneToCopyTo;
	Settings->BonesToPin.Add(NewBonePair);
}

TMap<FName, FName> UIKRetargetPinBoneController::GetAllBonePairs()
{
	TMap<FName, FName> AllBonePairs;
	
	FIKRetargetPinBoneOpSettings* Settings = reinterpret_cast<FIKRetargetPinBoneOpSettings*>(OpSettingsToControl);
	for (FPinBoneData& BonePair : Settings->BonesToPin)
	{
		AllBonePairs.Add(BonePair.BoneToCopyTo.BoneName, BonePair.BoneToCopyFrom.BoneName);
	}
	
	return MoveTemp(AllBonePairs);
}

#undef LOCTEXT_NAMESPACE
