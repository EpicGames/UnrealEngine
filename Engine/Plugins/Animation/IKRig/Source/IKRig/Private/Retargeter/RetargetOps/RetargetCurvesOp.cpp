// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/RetargetCurvesOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RetargetCurvesOp)

// TODO: For future curve manpulation base struct implementation.