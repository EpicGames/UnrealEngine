// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/RetargetOpUtils.h"

#include "VectorUtil.h"

double FOneEuroScalarFilter::Update(
	double InValue,
	double InDeltaTime,
	const FOneEuroFilterSettings& InSettings)
{
	auto LowPass = [](const double InPrev, const double InCurrent, const double InAlpha) -> double
		{
			return InAlpha * InCurrent + (1.0 - InAlpha) * InPrev;
		};
		
	if (bIsFirstRun)
	{
		PrevValue = PrevRawValue = InValue;
		PrevValueDerivative = 0.0;
		bIsFirstRun = false;
	}

	const double ClampedDeltaTime = ClampDeltaTime(InDeltaTime);

	// compute the derivative of the signal
	const double Derivative = (InValue - PrevRawValue) / ClampedDeltaTime;
	PrevRawValue = InValue;
	const double VelCutoffFreq = ClampFreqToFramerate(InSettings.VelocityCutoffFrequency, ClampedDeltaTime);
	const double LowPassDerivative = LowPass(PrevValueDerivative, Derivative, AlphaFromHz(ClampedDeltaTime, VelCutoffFreq));
	PrevValueDerivative = LowPassDerivative;

	// compute the cutoff frequency based on signal speed
	const double CutoffFreq = ClampFreqToFramerate(InSettings.CutoffFrequency, ClampedDeltaTime);
	const double Cutoff = CutoffFreq + InSettings.Responsiveness * FMath::Abs(LowPassDerivative);

	// filter the signal
	const double Result = LowPass(PrevValue, InValue, AlphaFromHz(ClampedDeltaTime, Cutoff));
	PrevValue = Result;

	return Result;
}

void FOneEuroScalarFilter::Reset(const double InValue)
{
	PrevValue = PrevRawValue = InValue;
	PrevValueDerivative = 0.0;
	bIsFirstRun = false;
}

double FOneEuroScalarFilter::AlphaFromHz(const double InDeltaTime, const double InFreqCutoffHz)
{
	const double OmegaDt = 2.0 * PI * InFreqCutoffHz * InDeltaTime;
	return OmegaDt / (1.0 + OmegaDt);
}

double FOneEuroScalarFilter::ClampDeltaTime(const double InDeltaTime)
{
	constexpr double MAX_DELTATIME = 1.0 / 15.0;
	constexpr double MIN_DELTATIME = 1.0 / 240.0;
	return FMath::Clamp(InDeltaTime, MIN_DELTATIME, MAX_DELTATIME);
}

double FOneEuroScalarFilter::ClampFreqToFramerate(const double InHz, const double InDeltaTime)
{
	// calculate the theoretical maximum frequency we can sample (Speed Limit)
	// Nyquist = SampleRate / 2 = (1 / DeltaTime) / 2
	const double NyquistFreq = 0.5 / InDeltaTime;
    
	// clamp to AT MOST the Nyquist frequency
	return (InHz > NyquistFreq) ? NyquistFreq : InHz;
}

FVector FOneEuroVectorFilter::Update(const FVector& InValue, const double InDeltaTime, const FOneEuroFilterSettings& InSettings)
{
	const double NewX = X.Update(InValue.X, InDeltaTime, InSettings);
	const double NewY = Y.Update(InValue.Y, InDeltaTime, InSettings);
	const double NewZ = Z.Update(InValue.Z, InDeltaTime, InSettings);
	return FVector(NewX,NewY,NewZ);
}

void FOneEuroVectorFilter::Reset(const FVector& StartingValue)
{
	X.Reset(StartingValue.X);
	Y.Reset(StartingValue.Y);
	Z.Reset(StartingValue.Z);
}

void FOneEuroVectorFilter::Reset()
{
	X.Reset();
	Y.Reset();
	Z.Reset();
}

FQuat FOneEuroQuatFilter::Update(
	const FQuat& InTargetRotation,
	double InDeltaTime,
	const FOneEuroFilterSettings& InSettings)
{
	if (bIsFirstRun)
	{
		PrevFilteredRot = PrevRawTarget = InTargetRotation;
		PrevHalfAngVel = FVector::ZeroVector;
		bIsFirstRun = false;
		return InTargetRotation;
	}
	
	// NOTE: because we are filtering rotations, and desire to use linear math,
	// all calculations are done with the exponential map of the delta quaternion

	// sanity check delta time
	const double ClampedDeltaTime = FOneEuroScalarFilter::ClampDeltaTime(InDeltaTime);

	// calc delta rotation from prev input to current input
	FQuat DeltaRot = InTargetRotation * PrevRawTarget.Inverse();
	DeltaRot = DeltaRot.W < 0 ? -DeltaRot : DeltaRot; // negate if W negative

	// store prev raw target for next frame
	PrevRawTarget = InTargetRotation;

	// convert into tangent space
	// NOTE: this produces a 3d vector amenable to filtering, with |Phi| = angle/2
	const FQuat DeltaLog = DeltaRot.Log();
	const FVector Phi(DeltaLog.X, DeltaLog.Y, DeltaLog.Z);
	
	// compute angular velocity
	const FVector HalfAngVel = Phi / ClampedDeltaTime; // rad/s
	// low-pass filter the velocity with fixed cutoff frequency
	const double VelCutoffFreq = FOneEuroScalarFilter::ClampFreqToFramerate(InSettings.VelocityCutoffFrequency, ClampedDeltaTime);
	const double VelAlpha = FOneEuroScalarFilter::AlphaFromHz(ClampedDeltaTime, VelCutoffFreq);
	PrevHalfAngVel = FMath::Lerp(PrevHalfAngVel, HalfAngVel, VelAlpha);

	// adaptive cutoff for the signal based on filtered speed
	const double Speed = 2.0 * PrevHalfAngVel.Size(); // rad/s
	const double ClampedFreqCutoff = FOneEuroScalarFilter::ClampFreqToFramerate(InSettings.CutoffFrequency, ClampedDeltaTime);
	const double FreqCutoff = ClampedFreqCutoff + (InSettings.Responsiveness * Speed); // Hz
	// take only a fraction of the filtered velocity this frame
	const double Alpha = FOneEuroScalarFilter::AlphaFromHz(ClampedDeltaTime, FreqCutoff);

	// Convert to tangent space (this is the total displacement needed)
	FQuat ErrorQ = InTargetRotation * PrevFilteredRot.Inverse();
	ErrorQ = ErrorQ.W < 0 ? -ErrorQ : ErrorQ;
	FVector ErrorPhi(ErrorQ.Log().X, ErrorQ.Log().Y, ErrorQ.Log().Z);
	const FVector PhiStep = ErrorPhi * Alpha;
	
	// exponentiate back into SO(3)
	const FQuat StepQ = FQuat(PhiStep.X, PhiStep.Y, PhiStep.Z, 0).Exp();
	// integrate the angular velocity to get new rotation
	const FQuat NewRotation = (StepQ * PrevFilteredRot).GetNormalized();
	// store prev filtered rotation for next frame
	PrevFilteredRot = NewRotation;
	return NewRotation;
}
