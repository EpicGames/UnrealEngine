// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/RetargetPoseOp.h"

#include "Retargeter/IKRetargetProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RetargetPoseOp)

const UClass* FIKRetargetPoseOpSettings::GetControllerType() const
{
	return UIKRetargetPoseController::StaticClass();
}

void FIKRetargetPoseOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy all properties
	static TArray<FName> PropertiesToIgnore = {};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetPoseOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

bool FIKRetargetPoseOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& Log)
{
	bIsInitialized = true;
	return true;
}

void FIKRetargetPoseOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	// NOTE: this op is read-from by the processor and does no work of its own
}

FIKRetargetOpSettingsBase* FIKRetargetPoseOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetPoseOp::GetSettingsType() const
{
	return FIKRetargetPoseOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetPoseOp::GetType() const
{
	return FIKRetargetPoseOp::StaticStruct();
}

FIKRetargetPoseOpSettings UIKRetargetPoseController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetPoseOpSettings*>(OpSettingsToControl);
}

void UIKRetargetPoseController::SetSettings(FIKRetargetPoseOpSettings InSettings)
{
	reinterpret_cast<FIKRetargetPoseOpSettings*>(OpSettingsToControl)->CopySettingsAtRuntime(&InSettings);
}

