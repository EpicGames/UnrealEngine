// Copyright Epic Games, Inc. All Rights Reserved.
#include "Retargeter/RetargetOps/RootMotionGeneratorOp.h"

#include "IKRigObjectVersion.h"
#include "Retargeter/IKRetargeter.h"
#include "Engine/SkeletalMesh.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"
#include "StructUtils/InstancedStruct.h"
#include "Animation/BoneReference.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RootMotionGeneratorOp)

#define LOCTEXT_NAMESPACE "RootMotionGeneratorOp"

#if WITH_EDITOR
IMPLEMENT_HIT_PROXY(HIKRetargetEditorRootProxy, HHitProxy);
#endif

const UClass* FIKRetargetRootMotionOpSettings::GetControllerType() const
{
	return UIKRetargetRootMotionController::StaticClass();
}

void FIKRetargetRootMotionOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies everything except the bones we are operating on (those require reinit)
	const TArray<FName> PropertiesToIgnore = {
		GET_MEMBER_NAME_CHECKED(FIKRetargetRootMotionOpSettings, SourceRoot),
		GET_MEMBER_NAME_CHECKED(FIKRetargetRootMotionOpSettings, TargetRoot),
		GET_MEMBER_NAME_CHECKED(FIKRetargetRootMotionOpSettings, TargetPelvis)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetRootMotionOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

#if WITH_EDITOR
USkeleton* FIKRetargetRootMotionOpSettings::GetSkeleton(const FName InPropertyName)
{
	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FIKRetargetRootMotionOpSettings, SourceRoot))
	{
		return const_cast<USkeleton*>(SourceSkeletonAsset);
	}

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FIKRetargetRootMotionOpSettings, TargetRoot))
	{
		return const_cast<USkeleton*>(TargetSkeletonAsset);
	}

	if (InPropertyName == GET_MEMBER_NAME_CHECKED(FIKRetargetRootMotionOpSettings, TargetPelvis))
	{
		return const_cast<USkeleton*>(TargetSkeletonAsset);
	}
	
	ensureMsgf(false, TEXT("Root motion op unable to get skeleton for UI widget."));
	return nullptr;
}
#endif

void FIKRetargetRootMotionOpSettings::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	if (InVersion < FIKRigObjectVersion::ModularRetargeterOps)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SourceRoot.BoneName = SourceRootBone_DEPRECATED;
		TargetRoot.BoneName = TargetRootBone_DEPRECATED;
		TargetPelvis.BoneName = TargetPelvisBone_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

bool FIKRetargetRootMotionOpSettings::Serialize(FArchive& Ar)
{
	return SerializeOpWithVersion(Ar, *this);
}

bool FIKRetargetRootMotionOp::Initialize(
	const FIKRetargetProcessor& Processor,
	const FRetargetSkeleton& SourceSkeleton,
	const FTargetSkeleton& TargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& Log)
{
	bIsInitialized = false;
	Reset();
	
	SourceRootIndex = SourceSkeleton.BoneNames.Find(Settings.SourceRoot.BoneName);
	bool bHasAllPrerequisites = true;
	if (SourceRootIndex == INDEX_NONE)
	{
		Log.LogWarning(FText::Format(LOCTEXT("MissingSourceRootBone", "Root Motion Remap Op, missing source root bone {0}."),
			FText::FromName(Settings.SourceRoot.BoneName)));
		bHasAllPrerequisites = false;
	}
	
	TargetRootIndex = TargetSkeleton.BoneNames.Find(Settings.TargetRoot.BoneName);
	if (TargetRootIndex == INDEX_NONE)
	{
		Log.LogWarning(FText::Format(LOCTEXT("MissingTargetRootBone", "Root Motion Remap Op, missing target root bone {0}."),
			FText::FromName(Settings.TargetRoot.BoneName)));
		bHasAllPrerequisites = false;
	}

	TargetPelvisIndex = TargetSkeleton.BoneNames.Find(Settings.TargetPelvis.BoneName);
	if (TargetPelvisIndex == INDEX_NONE)
	{
		Log.LogWarning(FText::Format(LOCTEXT("MissingPelvisBone", "Root Motion Remap Op, missing target pelvis bone {0}."),
			FText::FromName(Settings.TargetPelvis.BoneName)));
		bHasAllPrerequisites = false;
	}

	// can't cache everything unless all prerequisites are met
	if (!bHasAllPrerequisites)
	{
		return false;
	}

	// get retarget pose of source and target
	const TArray<FTransform>& TargetRetargetPose = TargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	const TArray<FTransform>& SourceRetargetPose = SourceSkeleton.RetargetPoses.GetGlobalRetargetPose();

	TargetPelvisInRefPose = TargetRetargetPose[TargetPelvisIndex];
	SourceRootInRefPose = SourceRetargetPose[SourceRootIndex];
	TargetRootInRefPose = TargetRetargetPose[TargetRootIndex];
	TargetPelvisRelativeToTargetRootRefPose = TargetRootInRefPose.GetRelativeTransform(TargetPelvisInRefPose);

	bIsInitialized = true;
	return true;
}

void FIKRetargetRootMotionOp::Run(
	FIKRetargetProcessor& Processor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	// generate a new transform for the target root bone
	FTransform NewRootTransform;
	
	// either generate motion from target pelvis, or copy from source root
	if (Settings.RootMotionSource == ERootMotionSource::GenerateFromTargetPelvis)
	{
		GenerateRootMotionFromTargetPelvis(NewRootTransform, InSourceGlobalPose, OutTargetGlobalPose);
	}
	else
	{
		CopyRootMotionFromSourceRoot(NewRootTransform, Processor, InSourceGlobalPose);
	}

	// optionally propagate delta to all non-retargeted children
	if (Settings.bPropagateToNonRetargetedChildren)
	{
		const FTransform Delta = OutTargetGlobalPose[TargetRootIndex].Inverse() * NewRootTransform;
		for (const int32 BoneIndex : NonRetargetedChildrenOfRoot)
		{
			OutTargetGlobalPose[BoneIndex] = OutTargetGlobalPose[BoneIndex] * Delta;
		}
	}

	// apply to the target root bone
	OutTargetGlobalPose[TargetRootIndex] = NewRootTransform * Settings.GlobalOffset;
}

void FIKRetargetRootMotionOp::PostInitialize(
	const FIKRetargetProcessor& Processor,
	const FRetargetSkeleton& SourceSkeleton,
	const FTargetSkeleton& TargetSkeleton,
	FIKRigLogger& Log)
{
	// generate list of non-retargeted child indices
	const int32 NumTargetBones = TargetSkeleton.BoneNames.Num();
	NonRetargetedChildrenOfRoot.Reset();
	for (int32 BoneIndex=1; BoneIndex<NumTargetBones; ++BoneIndex)
	{
		bool bHasNoRetargetedParent = true;
		
		int32 ParentIndex = BoneIndex;
		while(ParentIndex != INDEX_NONE)
		{
			if (TargetSkeleton.GetIsBoneRetargeted(ParentIndex))
			{
				bHasNoRetargetedParent = false;
				break;
			}
			ParentIndex = TargetSkeleton.ParentIndices[ParentIndex];
		}

		if (bHasNoRetargetedParent)
		{
			NonRetargetedChildrenOfRoot.Add(BoneIndex);
		}
	}
}

void FIKRetargetRootMotionOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
#if WITH_EDITOR

	// first try and get the root bone from the characterization data on the IK Rig assets
	if (const UIKRigDefinition* SourceIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source))
	{
		Settings.SourceRoot = SourceIKRig->GetRoot();
	}
	if (const UIKRigDefinition* TargetIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target))
	{
		Settings.TargetRoot = TargetIKRig->GetRoot();
		// get the target pelvis bone too
		Settings.TargetPelvis = TargetIKRig->GetPelvis();
	}

	// if source root bone was not registered, fallback to the root of the SOURCE skeleton
	if (Settings.SourceRoot.BoneName == NAME_None)
	{
		if (const USkeletalMesh* Mesh = InRetargetAsset->GetPreviewMesh(ERetargetSourceOrTarget::Source))
		{
			Settings.SourceRoot = Mesh->GetRefSkeleton().GetBoneName(0);
		}
	}

	// if target root bone was not registered, fallback to the root of the TARGET skeleton
	if (Settings.TargetRoot.BoneName == NAME_None)
	{
		if (const USkeletalMesh* Mesh = InRetargetAsset->GetPreviewMesh(ERetargetSourceOrTarget::Target))
		{
			Settings.TargetRoot = Mesh->GetRefSkeleton().GetBoneName(0);
		}
	}
	
#endif
}

FIKRetargetOpSettingsBase* FIKRetargetRootMotionOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetRootMotionOp::GetSettingsType() const
{
	return FIKRetargetRootMotionOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetRootMotionOp::GetType() const
{
	return FIKRetargetRootMotionOp::StaticStruct();
}

void FIKRetargetRootMotionOp::Reset()
{
	SourceRootIndex = INDEX_NONE;
	TargetRootIndex = INDEX_NONE;
	TargetPelvisIndex = INDEX_NONE;
	NonRetargetedChildrenOfRoot.Reset();
}

void FIKRetargetRootMotionOp::GenerateRootMotionFromTargetPelvis(
	FTransform& OutRootTransform,
	const TArray<FTransform>& InSourceGlobalPose,
	const TArray<FTransform>& InTargetGlobalPose) const
{
	// In this case, we are generating root motion "from scratch"
	// using the target Pelvis bone as the source of the motion
	
	if (Settings.bMaintainOffsetFromPelvis)
	{
		// set root to the relative offset from the pelvis (recorded from ref pose)
		OutRootTransform = TargetPelvisRelativeToTargetRootRefPose * InTargetGlobalPose[TargetPelvisIndex];
	}
	else
	{
		// snap root to the pelvis directly
		OutRootTransform = InTargetGlobalPose[TargetPelvisIndex];
	}

	// optionally remove all rotation (use static ref pose orientation)
	if (!Settings.bRotateWithPelvis)
	{
		OutRootTransform.SetRotation(TargetRootInRefPose.GetRotation());
	}

	// adjust height of root
	if (Settings.RootHeightSource == ERootMotionHeightSource::SnapToGround)
	{
		// snap the root to the ground plane
		FVector RootTranslation = OutRootTransform.GetTranslation();
		RootTranslation.Z = 0.f;
		OutRootTransform.SetTranslation(RootTranslation);
	}else if (Settings.RootHeightSource == ERootMotionHeightSource::CopyHeightFromSource)
	{
		// snap the root to the height from the source
		FVector RootTranslation = OutRootTransform.GetTranslation();
		RootTranslation.Z = InSourceGlobalPose[SourceRootIndex].GetTranslation().Z;
		OutRootTransform.SetTranslation(RootTranslation);
	}
}

void FIKRetargetRootMotionOp::CopyRootMotionFromSourceRoot(
	FTransform& OutRootTransform,
	const FIKRetargetProcessor& Processor,
	const TArray<FTransform>& InSourceGlobalPose) const
{
	// In this case, we are copying root motion from the source root
	// But we also scale it based on the relative height of the source/target Pelvis

	// rotation is the original target root rotation in ref pose plus the current rotation delta of the source
	const FQuat SourceRootRotationDelta = InSourceGlobalPose[SourceRootIndex].GetRotation() * SourceRootInRefPose.GetRotation().Inverse();
	OutRootTransform.SetRotation(SourceRootRotationDelta * TargetRootInRefPose.GetRotation());

	// scale root translation by same scale factor applied to Pelvis (and modified by settings)
	FVector NewRootLocation = InSourceGlobalPose[SourceRootIndex].GetLocation();
	FVector RootTranslationDelta = NewRootLocation - SourceRootInRefPose.GetTranslation();
	const FIKRetargetPelvisMotionOp* PelvisMotionOp = Processor.GetFirstRetargetOpOfType<FIKRetargetPelvisMotionOp>();
	if (PelvisMotionOp)
	{
		RootTranslationDelta *= PelvisMotionOp->GetGlobalScaleVector();
	}
	
	NewRootLocation = SourceRootInRefPose.GetTranslation() + RootTranslationDelta;

	// optionally snap the root to the ground plane
	if (Settings.RootHeightSource == ERootMotionHeightSource::SnapToGround)
	{
		NewRootLocation.Z = 0.f;
	}

	// apply the modified translation
	OutRootTransform.SetTranslation(NewRootLocation);
}

#if WITH_EDITOR

FText FIKRetargetRootMotionOp::GetWarningMessage() const
{
	if (SourceRootIndex == INDEX_NONE || TargetRootIndex == INDEX_NONE)
	{
		return FText::Format(LOCTEXT("MissingARootBone", "Missing a root bone."), FText::FromName(Settings.TargetRoot.BoneName));
	}

	if (Settings.RootMotionSource == ERootMotionSource::GenerateFromTargetPelvis && TargetPelvisIndex == INDEX_NONE)
	{
		return FText::Format(LOCTEXT("MissingTargetPelvis", "Missing target pelvis bone."), FText::FromName(Settings.TargetRoot.BoneName));
	}

	return FIKRetargetOpBase::GetWarningMessage();
}
#endif

void URootMotionGeneratorOp::ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct)
{
	OutInstancedStruct.InitializeAs(FIKRetargetRootMotionOp::StaticStruct());
	FIKRetargetRootMotionOp& NewOp = OutInstancedStruct.GetMutable<FIKRetargetRootMotionOp>();
	NewOp.SetEnabled(bIsEnabled);
	NewOp.Settings.SourceRootBone_DEPRECATED = SourceRootBone;
	NewOp.Settings.TargetRootBone_DEPRECATED = TargetRootBone;
	NewOp.Settings.TargetPelvisBone_DEPRECATED = TargetPelvisBone;
	NewOp.Settings.RootMotionSource = RootMotionSource;
	NewOp.Settings.RootHeightSource = RootHeightSource;
	NewOp.Settings.bPropagateToNonRetargetedChildren = bPropagateToNonRetargetedChildren;
	NewOp.Settings.bMaintainOffsetFromPelvis = bMaintainOffsetFromPelvis;
	NewOp.Settings.bRotateWithPelvis = bRotateWithPelvis;
	NewOp.Settings.GlobalOffset = GlobalOffset;
};

FIKRetargetRootMotionOpSettings UIKRetargetRootMotionController::GetSettings()
{
	return *GetSettingsPtr();
}

void UIKRetargetRootMotionController::SetSettings(FIKRetargetRootMotionOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

void UIKRetargetRootMotionController::SetSourceRootBone(const FName InSourceRootBone)
{
	GetSettingsPtr()->SourceRoot.BoneName = InSourceRootBone;
}

FName UIKRetargetRootMotionController::GetSourceRootBone()
{
	return GetSettingsPtr()->SourceRoot.BoneName;
}

void UIKRetargetRootMotionController::SetTargetRootBone(const FName InTargetRootBone)
{
	GetSettingsPtr()->TargetRoot.BoneName = InTargetRootBone;
}

FName UIKRetargetRootMotionController::GetTargetRootBone()
{
	return GetSettingsPtr()->TargetRoot.BoneName;
}

void UIKRetargetRootMotionController::SetTargetPelvisBone(const FName InTargetPelvisBone)
{
	GetSettingsPtr()->TargetPelvis.BoneName = InTargetPelvisBone;
}

FName UIKRetargetRootMotionController::GetTargetPelvisBone()
{
	return GetSettingsPtr()->TargetPelvis.BoneName;
}

FIKRetargetRootMotionOpSettings* UIKRetargetRootMotionController::GetSettingsPtr() const
{
	return reinterpret_cast<FIKRetargetRootMotionOpSettings*>(OpSettingsToControl);
}

#undef LOCTEXT_NAMESPACE
