// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/RunIKRigOp.h"

#include "IKRigDebugRendering.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Engine/SkeletalMesh.h"
#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/RetargetOps/BlendToSourceOp.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RunIKRigOp)

#define LOCTEXT_NAMESPACE "RunIKRigSolversOp"

struct FIKRigGoal;

bool FIKRetargetRunIKRigChainSettings::operator==(const FIKRetargetRunIKRigChainSettings& Other) const
{
	return bEnableIK == Other.bEnableIK
		&& FMath::IsNearlyEqualByULP(ChainPositionAlpha, Other.ChainPositionAlpha)
		&& bOverrideGoalAlpha == Other.bOverrideGoalAlpha
		&& FMath::IsNearlyEqualByULP(FinalPositionAlpha, Other.FinalPositionAlpha)
		&& FMath::IsNearlyEqualByULP(FinalRotationAlpha, Other.FinalRotationAlpha);
}

const UClass* FIKRetargetRunIKRigOpSettings::GetControllerType() const
{
	return UIKRetargetRunIKRigController::StaticClass();
}

void FIKRetargetRunIKRigOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies everything except "IKRigAsset"
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetRunIKRigOpSettings, IKRigAsset)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetRunIKRigOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

FRunIKRigChain::FRunIKRigChain(
	const FResolvedBoneChain& InSourceBoneChain,
	const FResolvedBoneChain& InTargetBoneChain,
	const FIKRetargetRunIKRigChainSettings& InSettings,
	FIKRigLogger& InLog)
{
	bIsReady = false;
	
	if (InSourceBoneChain.BoneIndices.Num() < 2)
	{
		InLog.LogWarning(FText::Format(
			LOCTEXT("SourceChainLessThanTwo","Run IK Rig Op: found source bone chain, {0} with less than 2 joints. Cannot retarget goal."),
			FText::FromName(InSourceBoneChain.ChainName)));
		return;
	}

	// store pointers to required data
	SourceBoneChain = &InSourceBoneChain;
	TargetBoneChain = &InTargetBoneChain;
	Settings = &InSettings;
	
	// cache initial length of SOURCE chain
	const FTransform& InitialSourceStartGlobal = InSourceBoneChain.RefPoseGlobalTransforms[0];
	const FTransform& InitialSourceEndGlobal = InSourceBoneChain.RefPoseGlobalTransforms.Last();
	const float InitialStartLength = static_cast<float>((InitialSourceStartGlobal.GetLocation() - InitialSourceEndGlobal.GetLocation()).Size());
	if (InitialStartLength <= UE_KINDA_SMALL_NUMBER)
	{
		InLog.LogWarning(FText::Format(
			LOCTEXT("SourceZeroLengthIK", "Run IK Rig Op: found source bone chain {0} with IK, but it is zero length. Cannot retarget goal."),
			FText::FromName(InSourceBoneChain.ChainName)));
		return;
	}
	InvInitialLengthSource = 1.0f / InitialStartLength;

	// cache initial length of TARGET chain
	const FTransform& InitialTargetStartGlobal = InTargetBoneChain.RefPoseGlobalTransforms[0];
	const FTransform& InitialTargetEndGlobal = InTargetBoneChain.RefPoseGlobalTransforms.Last();
	InitialTargetChainLength = static_cast<float>((InitialTargetStartGlobal.GetTranslation() - InitialTargetEndGlobal.GetTranslation()).Size());
	if (InitialTargetChainLength <= UE_KINDA_SMALL_NUMBER)
	{
		InLog.LogWarning(FText::Format(
			LOCTEXT("TargetZeroLengthIK", "Run IK Rig Op: found target bone chain {0} with IK, but it is zero length. Cannot retarget goal."),
			FText::FromName(InTargetBoneChain.ChainName)));
		return;
	}

	bIsReady = true;
}

void FRunIKRigChain::Run(
	FIKRigGoalContainer& InGoalContainer,
	const TArray<FTransform>& InSourceGlobalPose,
	const TArray<FTransform>& InTargetGlobalPose,
	const FIKRetargetRunIKRigOpSettings& InOpSettings,
	const FIKRetargetPelvisMotionOp* InPelvisMotionOp) const
{
	if (!bIsReady)
	{
		return;
	}
	
	FIKRigGoal* Goal = InGoalContainer.FindGoalByName(TargetBoneChain->IKGoalName);
	if (!ensure(Goal))
	{
		return;
	}

	if (Settings->ChainPositionAlpha > UE_KINDA_SMALL_NUMBER)
	{
		// get relevant bone transforms
		const int32 SourceStartBoneIndex = SourceBoneChain->BoneIndices[0];
		const int32 SourceEndBoneIndex = SourceBoneChain->BoneIndices.Last();
		const FTransform& CurrentSourceStartGlobal = InSourceGlobalPose[SourceStartBoneIndex];
		const FTransform& CurrentSourceEndGlobal = InSourceGlobalPose[SourceEndBoneIndex];

		// get the current normalized direction / length of the IK limb (how extended it is as percentage of original length)
		const FVector CurrentSourceChainVector = CurrentSourceEndGlobal.GetLocation() - CurrentSourceStartGlobal.GetLocation();
		double CurrentSourceChainLength;
		FVector CurrentSourceChainDirection;
		CurrentSourceChainVector.ToDirectionAndLength(CurrentSourceChainDirection, CurrentSourceChainLength);
		const double NormalizedLimbLength = CurrentSourceChainLength * InvInitialLengthSource;
		const FVector CurrentSourceEndDirectionNormalized = CurrentSourceChainDirection * NormalizedLimbLength;
	
		// set position to length-scaled direction from source limb
		const FVector PelvisTranslationDelta = InPelvisMotionOp ? InPelvisMotionOp->GetPelvisTranslationOffset() : FVector::ZeroVector;
		const FVector AffectIKWeights = InPelvisMotionOp ? InPelvisMotionOp->GetAffectIKWeightAsVector() : FVector::ZeroVector;
		const FVector InvAffectIKWeights = FVector::OneVector - AffectIKWeights;
		const FVector InvRootModification = PelvisTranslationDelta * InvAffectIKWeights;
		const int32 TargetStartBoneIndex = TargetBoneChain->BoneIndices[0];
		const FVector CurrentTargetStartLocation = InTargetGlobalPose[TargetStartBoneIndex].GetTranslation() - InvRootModification;
		FVector GoalPosition = CurrentTargetStartLocation + (CurrentSourceEndDirectionNormalized * InitialTargetChainLength);
		
		// alpha and apply to the goal
		Goal->Position = FMath::Lerp(Goal->Position, GoalPosition, Settings->ChainPositionAlpha);
	}

	if (Settings->ChainRotationAlpha > UE_KINDA_SMALL_NUMBER)
	{
		// calc delta rotation of source relative to source retarget pose
		const int32 SourceEndBoneIndex = SourceBoneChain->BoneIndices.Last();
		const FTransform& CurrentSourceEndGlobal = InSourceGlobalPose[SourceEndBoneIndex];
		const FTransform& InitialSourceEndGlobal = SourceBoneChain->RefPoseGlobalTransforms.Last();
		const FTransform& InitialTargetEndGlobal = TargetBoneChain->RefPoseGlobalTransforms.Last();
		const FQuat CurrentSourceEndRotation = CurrentSourceEndGlobal.GetRotation();
		const FQuat SourceDeltaRotation = (CurrentSourceEndRotation * InitialSourceEndGlobal.GetRotation().Inverse()).GetNormalized();

		// apply delta rotation to the target
		const FQuat NewGoalRotation = SourceDeltaRotation * InitialTargetEndGlobal.GetRotation();
		
		// alpha blend rotation and apply to goal
		Goal->Rotation = FQuat::FastLerp(Goal->Rotation.Quaternion(), NewGoalRotation, Settings->ChainRotationAlpha).GetNormalized().Rotator();
	}
}

void FRunIKRigChain::SetGoalAlpha(FIKRigGoalContainer& InGoalContainer)
{
	if (!bIsReady)
	{
		return;
	}

	if (!Settings->bOverrideGoalAlpha)
	{
		return;
	}
	
	FIKRigGoal* Goal = InGoalContainer.FindGoalByName(TargetBoneChain->IKGoalName);
	if (!ensure(Goal))
	{
		return;
	}

	// NOTE: actual blending happens in the IK Rig processor
	Goal->PositionAlpha = Settings->FinalPositionAlpha;
	Goal->RotationAlpha = Settings->FinalRotationAlpha;
}

bool FIKRetargetRunIKRigOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;

	if (!Settings.IKRigAsset)
	{
		InLog.LogWarning( LOCTEXT("NoIKRigFound", "Run IK Rig Op: No IK Rig asset was specified."));
		return false;
	}

	// initialize the IK Rig
	IKRigProcessor.Initialize(Settings.IKRigAsset, InTargetSkeleton.SkeletalMesh, InProcessor.GetIKRigGoalContainer());

	// warn if the IK Rig couldn't initialize
	if (!IKRigProcessor.IsInitialized())
	{
		// couldn't initialize the IK Rig, we don't disable the retargeter in this case, just warn the user
		InLog.LogWarning( FText::Format(
			LOCTEXT("CouldNotInitializeIKRig", "Run IK Rig Op: unable to initialize the IK Rig, {0} for the Skeletal Mesh {1}. See previous warnings."),
			FText::FromString(Settings.IKRigAsset->GetName()), FText::FromString(InTargetSkeleton.SkeletalMesh->GetName())));
	}

	// cache goal bone indices
	GoalBoneIndicesMap.Reset();
	for (UIKRigEffectorGoal* Goal : Settings.IKRigAsset->GetGoalArray())
	{
		const int32 BoneIndex = InTargetSkeleton.FindBoneIndexByName(Goal->BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			GoalBoneIndicesMap.Add(Goal->BoneName, BoneIndex);
		}
	}

	// warn user if IK goal is not on the END bone of the target chain.
	// NOTE: this will still work but could produce bad results so we throw a warning instead of an error
	const TArray<UIKRigEffectorGoal*>& AllGoals = Settings.IKRigAsset->GetGoalArray();
	const TMap<const UIKRigDefinition*, TArray<FResolvedBoneChain>>& AllTargetBoneChains = InProcessor.GetBoneChains().GetAllResolvedTargetBoneChains();
	if (const TArray<FResolvedBoneChain>* ChainsInRig = AllTargetBoneChains.Find(Settings.IKRigAsset))
	{
		for (const FResolvedBoneChain& Chain : *ChainsInRig)
		{
			if (Chain.IKGoalName == NAME_None)
			{
				continue;
			}
			
			for (UIKRigEffectorGoal* Goal : AllGoals)
			{
				if (Goal->GoalName != Chain.IKGoalName)
				{
					continue;
				}

				if (Goal->BoneName != Chain.EndBone)
				{
					InLog.LogWarning( FText::Format(
						LOCTEXT("TargetIKNotOnEndBone", "IK chain, '{0}' has an IK goal that is not on the End Bone of the chain."),
						FText::FromName(Chain.ChainName)));
					break;
				}
			}
		}
	}

	// go through all chains to retarget and load them
	Chains.Reset();
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (FIKRetargetRunIKRigChainSettings& ChainSettings : Settings.Chains)
	{
		FName TargetChainName = ChainSettings.TargetChainName;
		const FResolvedBoneChain* SourceBoneChain = nullptr;
		const FResolvedBoneChain* TargetBoneChain = nullptr;
		if (!FIKRetargetRunIKRigOp::ValidateIKChain(
			TargetChainName,
			GoalContainer,
			BoneChains,
			this,
			this,
			InLog,
			SourceBoneChain /*out*/,
			TargetBoneChain /*out*/))
		{
			continue;
		}

		Chains.Emplace(*SourceBoneChain, *TargetBoneChain, ChainSettings, InLog);
	}
	
	bIsInitialized = IKRigProcessor.IsInitialized();
	return bIsInitialized;
}

void FIKRetargetRunIKRigOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	if (InProcessor.IsIKForcedOff() || !InProcessor.IsInitialized())
	{
		return; // skip this op when IK is off
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		// live preview source asset settings in the retarget editor
		// NOTE: this copies solver settings and goal.PositionAlpha and goal.RotationAlpha
		IKRigProcessor.CopyAllSettingsFromAsset(Settings.IKRigAsset);
	}
#endif

	FIKRigGoalContainer& GoalsFromOpStack = InProcessor.GetIKRigGoalContainer();
	
	// set main alpha on the goals
	for (FRunIKRigChain& Chain : Chains)
	{
		Chain.SetGoalAlpha(GoalsFromOpStack);
	}

	// copy the goal container from the retargeter to the IK Rig processor
	IKRigProcessor.ApplyGoalsFromOtherContainer(GoalsFromOpStack);
	
#if WITH_EDITOR
	// store initial goals transforms
	// (must be before the IK solve changes the bone transforms)
	SaveInitialGoalTransformsIntoDebugData(InProcessor, OutTargetGlobalPose);
#endif
	
	// copy input pose to start IK solve from
	IKRigProcessor.SetInputPoseGlobal(OutTargetGlobalPose);
	// run IK solve
	IKRigProcessor.Solve();
	// copy results of solve
	IKRigProcessor.GetOutputPoseGlobal(OutTargetGlobalPose);

#if WITH_EDITOR
	// store current goals transforms after the IK solve
	// (must be AFTER the IK solve because the IK Rig processor resolves the final goal transforms)
	SaveCurrentGoalTransformsIntoDebugData();
#endif
}

void FIKRetargetRunIKRigOp::InitializeBeforeChildren(
	FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	FIKRigLogger& Log)
{
	// reset the goal container for this IK Rig
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	ResetGoalContainer(InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose(), GoalContainer);
}

void FIKRetargetRunIKRigOp::RunBeforeChildren(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	// reset the goal container for this IK Rig
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	ResetGoalContainer(OutTargetGlobalPose, GoalContainer);
	
	// do basic chain-based goal retargeting
	const FIKRetargetPelvisMotionOp* PelvisMotionOp = InProcessor.GetFirstRetargetOpOfType<FIKRetargetPelvisMotionOp>();
	for (const FRunIKRigChain& Chain : Chains)
	{
		Chain.Run(GoalContainer, InSourceGlobalPose, OutTargetGlobalPose, Settings, PelvisMotionOp);
	}
}

void FIKRetargetRunIKRigOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	// load the target IK Rig asset to execute
	Settings.IKRigAsset = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target);

	// initialize the chain mapping
	ChainMapping.ReinitializeWithIKRigs(InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source), Settings.IKRigAsset);
	static bool bForceRemap = true;
	ChainMapping.AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);

	// regenerate set of chain settings
	RegenerateChainSettings();
}

void FIKRetargetRunIKRigOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	const bool bAssignedTarget = SourceOrTarget == ERetargetSourceOrTarget::Target;
	Settings.IKRigAsset = bAssignedTarget ? InIKRig : ChainMapping.GetIKRig(ERetargetSourceOrTarget::Target);

	// re-initialize the chain mapping
	ChainMapping.ReinitializeWithIKRigs(ChainMapping.GetIKRig(ERetargetSourceOrTarget::Source), Settings.IKRigAsset);
	static bool bForceRemap = false; // don't force mapping, keeps existing mappings
	ChainMapping.AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);

	// regenerate set of chain settings based on new IK Rig
	RegenerateChainSettings();
}

FIKRetargetOpSettingsBase* FIKRetargetRunIKRigOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetRunIKRigOp::GetSettingsType() const
{
	return FIKRetargetRunIKRigOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetRunIKRigOp::GetType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

const UIKRigDefinition* FIKRetargetRunIKRigOp::GetCustomTargetIKRig() const
{
	return Settings.IKRigAsset;
}

FRetargetChainMapping* FIKRetargetRunIKRigOp::GetChainMapping()
{
	return &ChainMapping;
}

void FIKRetargetRunIKRigOp::OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent)
{
	ChainMapping.ReinitializeWithIKRigs(ChainMapping.GetIKRig(ERetargetSourceOrTarget::Source), Settings.IKRigAsset);

	// regenerate set of chain settings based on new IK Rig
	RegenerateChainSettings();
}

bool FIKRetargetRunIKRigOp::ValidateIKChain(
	const FName& InChainToValidate,
	const FIKRigGoalContainer& InGoalContainer,
	const FRetargeterBoneChains& InBoneChains,
	const FIKRetargetOpBase* InOp,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog,
	const FResolvedBoneChain*& OutSourceBoneChain,
	const FResolvedBoneChain*& OutTargetBoneChain)
{
	OutSourceBoneChain = nullptr;
	OutTargetBoneChain = nullptr;
	
	// requires a parent Run IK Rig op
	if (!ensure(InParentOp))
	{
		return false;
	}

	// validate that an IK rig has been assigned
	const FIKRetargetRunIKRigOp* ParentOp = reinterpret_cast<const FIKRetargetRunIKRigOp*>(InParentOp);
	if (ParentOp->Settings.IKRigAsset == nullptr)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingIKRig", "{0}, is missing an IK rig. No chains can be retargeted."), FText::FromName(InOp->GetName())));
		return false;
	}
	
	OutTargetBoneChain = InBoneChains.GetResolvedBoneChainByName(
		InChainToValidate,
		ERetargetSourceOrTarget::Target,
		ParentOp->Settings.IKRigAsset);

	// validate that the chain even exists
	if (OutTargetBoneChain == nullptr)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("IKChainOpMissingChain", "IK chain data is out of sync with IK Rig. Missing target chain, '{0}."),
		FText::FromName(InChainToValidate)));
		return false;
	}

	// validate that the chain has IK applied to it
	if (OutTargetBoneChain->IKGoalName == NAME_None)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("IKChainOpChainHasNoIK", "An IK chain was found with no IK goal assigned to it, '{0}."),
		FText::FromName(InChainToValidate)));
		return false;
	}
	
	// which source chain was this target chain mapped to?
	const FName SourceChainName = ParentOp->ChainMapping.GetChainMappedTo(InChainToValidate, ERetargetSourceOrTarget::Target);
	OutSourceBoneChain = InBoneChains.GetResolvedBoneChainByName(SourceChainName, ERetargetSourceOrTarget::Source);
	if (OutSourceBoneChain == nullptr)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("IKChainOpChainNotMapped", "Found IK chain that was not mapped to a source chain, '{0}."),
		FText::FromName(InChainToValidate)));
		return false;
	}

	// does the goal actually exist?
	const FIKRigGoal* Goal = InGoalContainer.FindGoalByName(OutTargetBoneChain->IKGoalName);
	if (!Goal)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("IKChainWithMissingGoal", "Found a retarget chain, '{0}' referring to a goal that doesn't exist, '{1}'."),
		FText::FromName(InChainToValidate),
		FText::FromName(OutTargetBoneChain->IKGoalName)));
		return false;
	}

	// chain is valid!
	return true;
};

#if WITH_EDITOR

FCriticalSection FIKRetargetRunIKRigOp::DebugDataMutex;

void FIKRetargetRunIKRigOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	// lock because main thread is doing the drawing while data is modified on worker
	FScopeLock ScopeLock(&DebugDataMutex);
	
	// draw IK goals on each IK chain
	if (!(Settings.bDrawGoals || Settings.bDrawGoalBoneLocations))
	{
		return;
	}

	// spin through all the IK goals debug data
	for (const FRunIKRigOpGoalDebugData& GoalData : GoalDebugData)
	{
		bool bIsSelected = InEditorState.SelectedGoals.Contains(GoalData.GoalName);

		FTransform Initial = GoalData.InitialGoal * InComponentTransform;
		FTransform Current = GoalData.CurrentGoal * InComponentTransform;

		if (Settings.bDrawGoals)
		{
			FLinearColor GoalColor = bIsSelected ? InEditorState.GoalColor : InEditorState.GoalColor * InEditorState.NonSelected;
			
			IKRigDebugRendering::DrawWireCube(
			InPDI,
			Current,
			GoalColor,
			static_cast<float>(Settings.GoalDrawSize * InComponentScale),
			static_cast<float>(Settings.GoalDrawThickness * InComponentScale));
		}
		
		if (Settings.bDrawGoalBoneLocations)
		{
			IKRigDebugRendering::DrawWireCube(
			InPDI,
			Initial,
			InEditorState.Muted,
			static_cast<float>(Settings.GoalDrawSize * InComponentScale * 0.5),
			static_cast<float>(Settings.GoalDrawThickness * InComponentScale));

			if (Settings.bDrawGoals)
			{
				DrawDashedLine(
					InPDI,
					Initial.GetLocation(),
					Current.GetLocation(),
					InEditorState.Muted,
					1.0f,
					SDPG_Foreground);
			}
		}
	}
}

void FIKRetargetRunIKRigOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.Chains, InChainName);
}

bool FIKRetargetRunIKRigOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.Chains, InChainName);
}

uint8* FIKRetargetRunIKRigOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.Chains, InChainName);
}

void FIKRetargetRunIKRigOp::SaveInitialGoalTransformsIntoDebugData(
	const FIKRetargetProcessor& InProcessor,
	const TArray<FTransform>& InTargetGlobalPose)
{
	// lock because main thread is doing the drawing while this is on worker
	FScopeLock ScopeLock(&DebugDataMutex);
	
	GoalDebugData.Reset();
	const FRetargetSkeleton& TargetSkeleton = InProcessor.GetSkeleton(ERetargetSourceOrTarget::Target);
	const TArray<FIKRigGoal>& GoalArray = IKRigProcessor.GetGoalContainer().GetGoalArray();
	for (int32 GoalIndex=0; GoalIndex<GoalArray.Num(); ++GoalIndex)
	{
		const FIKRigGoal& Goal = GoalArray[GoalIndex];

		FRunIKRigOpGoalDebugData NewGoalData;
		NewGoalData.GoalName = Goal.Name;
		
		int32 BoneIndex = TargetSkeleton.FindBoneIndexByName(Goal.BoneName);
		if (!ensure(InTargetGlobalPose.IsValidIndex(BoneIndex)))
		{
			NewGoalData.InitialGoal = FTransform(Goal.Rotation, Goal.Position);
		}
		else
		{
			NewGoalData.InitialGoal = InTargetGlobalPose[BoneIndex];
		}
		
		GoalDebugData.Add(NewGoalData);
	}
}

void FIKRetargetRunIKRigOp::SaveCurrentGoalTransformsIntoDebugData()
{
	// lock because main thread is doing the drawing while this is on worker
	FScopeLock ScopeLock(&DebugDataMutex);
	
	const TArray<FIKRigGoal>& GoalArray = IKRigProcessor.GetGoalContainer().GetGoalArray();
	for (int32 GoalIndex=0; GoalIndex<GoalArray.Num(); ++GoalIndex)
	{
		const FIKRigGoal& Goal = GoalArray[GoalIndex];
		GoalDebugData[GoalIndex].CurrentGoal = FTransform(Goal.FinalBlendedRotation, Goal.FinalBlendedPosition);
	}
}
#endif

TArray<FName> FIKRetargetRunIKRigOp::GetRequiredTargetChains() const
{
	if (!Settings.IKRigAsset)
	{
		return TArray<FName>{};
	}
	
	// find the target chains that children ops should deal with
	TArray<FName> RequiredTargetChains;
	const TArray<FBoneChain>& TargetChains = Settings.IKRigAsset->GetRetargetChains();
	for (const FBoneChain& TargetChain : TargetChains)
	{
		if (TargetChain.IKGoalName == NAME_None)
		{
			continue; // skip non-IK chains
		}

		const FName SourceChain = ChainMapping.GetChainMappedTo(TargetChain.ChainName, ERetargetSourceOrTarget::Target);
		if (SourceChain == NAME_None)
		{
			continue; // skip unmapped chains
		}
		
		RequiredTargetChains.Add(TargetChain.ChainName);
	}
	
	return MoveTemp(RequiredTargetChains);
}

void FIKRetargetRunIKRigOp::ResetGoalContainer(
	const TArray<FTransform>& InTargetGlobalPose,
	FIKRigGoalContainer& InOutGoalContainer)
{
	// if no IK Rig assigned, leave container empty
	if (!Settings.IKRigAsset)
	{
		InOutGoalContainer.Empty();
		return;
	}

	// add all the goals from the IK Rig (empties and refills)
	InOutGoalContainer.FillWithGoalArray(Settings.IKRigAsset->GetGoalArray());

	// set all goals to the input pose in component space
	for (FIKRigGoal& Goal : InOutGoalContainer.GetGoalArray())
	{
		const int32* BoneIndexPtr = GoalBoneIndicesMap.Find(Goal.BoneName);
		if (BoneIndexPtr && InTargetGlobalPose.IsValidIndex(*BoneIndexPtr))
		{
			Goal.PositionSpace = EIKRigGoalSpace::Component;
			Goal.RotationSpace = EIKRigGoalSpace::Component;
			Goal.Position = InTargetGlobalPose[*BoneIndexPtr].GetLocation();
			Goal.Rotation = InTargetGlobalPose[*BoneIndexPtr].GetRotation().Rotator();
		}
		else
		{
			Goal.PositionSpace = EIKRigGoalSpace::Additive;
			Goal.RotationSpace = EIKRigGoalSpace::Additive;
			Goal.Position = FVector::ZeroVector;
			Goal.Rotation = FRotator::ZeroRotator;
		}
	}

	// optionally enable/disable IK goals completely (removes them from the IK Rig solve)
	for (const FRunIKRigChain& Chain : Chains)
	{
		if (!Chain.TargetBoneChain)
		{
			continue;
		}

		FIKRigGoal* Goal = InOutGoalContainer.FindGoalByName(Chain.TargetBoneChain->IKGoalName);
		Goal->bEnabled = Chain.Settings->bEnableIK;
	}
}

void FIKRetargetRunIKRigOp::RegenerateChainSettings()
{
	// regenerate chain settings
	constexpr bool bSkipUnmappedChains = false;
	constexpr bool bSkipNonIKChains = true;
	IKRetargetOpUtils::SynchronizeChainSettingsWithIKRig(Settings.Chains, this, bSkipUnmappedChains, bSkipNonIKChains);
}

FIKRetargetRunIKRigOpSettings UIKRetargetRunIKRigController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetRunIKRigOpSettings*>(OpSettingsToControl);
}

void UIKRetargetRunIKRigController::SetSettings(FIKRetargetRunIKRigOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
