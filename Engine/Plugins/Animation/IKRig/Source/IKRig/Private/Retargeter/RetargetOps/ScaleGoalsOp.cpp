// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/ScaleGoalsOp.h"

#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"

bool FIKRetargetScaleGoalsChainSettings::operator==(const FIKRetargetScaleGoalsChainSettings& Other) const
{
	return FMath::IsNearlyEqualByULP(ScaleVertical,    Other.ScaleVertical)
		&& FMath::IsNearlyEqualByULP(ScaleHorizontal, Other.ScaleHorizontal)
		&& FMath::IsNearlyEqualByULP(ScaleAlongChain, Other.ScaleAlongChain);
}

const UClass* FIKRetargetScaleGoalsOpSettings::GetControllerType() const
{
	return UIKRetargetScaleGoalsController::StaticClass();
}

void FIKRetargetScaleGoalsOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the chains
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetScaleGoalsOpSettings, Chains)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetScaleGoalsOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);

	// copy chains
	const FIKRetargetScaleGoalsOpSettings* NewSettings = reinterpret_cast<const FIKRetargetScaleGoalsOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->Chains, Chains);
}

void FIKChainToScale::Run(
	FIKRigGoalContainer& InGoalContainer,
	const TArray<FTransform>& InTargetGlobalPose,
	const FIKRetargetScaleGoalsOpSettings& InOpSettings) const
{
	FIKRigGoal* Goal = InGoalContainer.FindGoalByName(TargetBoneChain->IKGoalName);
	if (!ensure(Goal))
	{
		return;
	}

	const bool bScalingGoal = !FMath::IsNearlyEqual(Settings->ScaleVertical, 1.0f) || !FMath::IsNearlyEqual(Settings->ScaleHorizontal, 1.0f);
	const bool bExtendingGoal = !FMath::IsNearlyEqual(Settings->ScaleAlongChain, 1.0f);
	if (!bScalingGoal && !bExtendingGoal)
	{
		return; // nothing to do
	}

	// modify position of goal
	FVector GoalPosition = Goal->Position;
	
	// apply horizontal and vertical scaling
	if (bScalingGoal)
	{
		GoalPosition *= FVector(Settings->ScaleHorizontal, Settings->ScaleHorizontal, Settings->ScaleVertical);
	}
	
	// apply extension
	if (bExtendingGoal)
	{
		const int32 TargetStartBoneIndex = TargetBoneChain->BoneIndices[0];
		const FVector CurrentTargetStartLocation = InTargetGlobalPose[TargetStartBoneIndex].GetTranslation();
		GoalPosition = CurrentTargetStartLocation + (GoalPosition - CurrentTargetStartLocation) * Settings->ScaleAlongChain;	
	}

	// alpha and apply
	GoalPosition = FMath::Lerp(Goal->Position, GoalPosition, InOpSettings.Alpha);
	Goal->Position = GoalPosition;
}

bool FIKRetargetScaleGoalsOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;
	
	// go through all chains to retarget and load them
	ChainsToScale.Reset();
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (FIKRetargetScaleGoalsChainSettings& ChainSettings : Settings.Chains)
	{
		FName TargetChainName = ChainSettings.TargetChainName;
		const FResolvedBoneChain* SourceBoneChain = nullptr;
		const FResolvedBoneChain* TargetBoneChain = nullptr;
		if (!FIKRetargetRunIKRigOp::ValidateIKChain(
			TargetChainName,
			GoalContainer,
			BoneChains,
			this,
			InParentOp,
			InLog,
			SourceBoneChain /*out*/,
			TargetBoneChain /*out*/))
		{
			continue;
		}

		ChainsToScale.Emplace(*TargetBoneChain, ChainSettings);
	}

	bIsInitialized = true;
	return true;
}

void FIKRetargetScaleGoalsOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	if (Settings.Alpha <= UE_KINDA_SMALL_NUMBER)
	{
		return; // fast skip whole op
	}
	
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (const FIKChainToScale& Chain : ChainsToScale)
	{
		Chain.Run(GoalContainer, OutTargetGlobalPose, Settings);
	}
}

FIKRetargetOpSettingsBase* FIKRetargetScaleGoalsOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetScaleGoalsOp::GetSettingsType() const
{
	return FIKRetargetScaleGoalsOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetScaleGoalsOp::GetType() const
{
	return FIKRetargetScaleGoalsOp::StaticStruct();
}

const UScriptStruct* FIKRetargetScaleGoalsOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

void FIKRetargetScaleGoalsOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetScaleGoalsOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetScaleGoalsOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	IKRetargetOpUtils::OnRetargetChainRenamed(Settings.Chains, InOldChainName, InNewChainName);
}

void FIKRetargetScaleGoalsOp::OnParentReinitPropertyEdited(
	const FIKRetargetOpBase& InParentOp,
	const FPropertyChangedEvent* InPropertyChangedEvent)
{
	RegenerateChainSettings(&InParentOp);	
}

void FIKRetargetScaleGoalsOp::RegenerateChainSettings(const FIKRetargetOpBase* InParentOp)
{
	// regenerate chain settings
	constexpr bool bSkipUnmappedChains = false;
	constexpr bool bSkipNonIKChains = true;
	IKRetargetOpUtils::SynchronizeChainSettingsWithIKRig(Settings.Chains, InParentOp, bSkipUnmappedChains, bSkipNonIKChains);
}

FIKRetargetScaleGoalsOpSettings UIKRetargetScaleGoalsController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetScaleGoalsOpSettings*>(OpSettingsToControl);
}

void UIKRetargetScaleGoalsController::SetSettings(FIKRetargetScaleGoalsOpSettings InSettings)
{
	reinterpret_cast<FIKRetargetScaleGoalsOpSettings*>(OpSettingsToControl)->CopySettingsAtRuntime(&InSettings);
}

#if WITH_EDITOR
void FIKRetargetScaleGoalsOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
}

void FIKRetargetScaleGoalsOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.Chains, InChainName);
}

bool FIKRetargetScaleGoalsOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.Chains, InChainName);
}

uint8* FIKRetargetScaleGoalsOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.Chains, InChainName);
}
#endif
