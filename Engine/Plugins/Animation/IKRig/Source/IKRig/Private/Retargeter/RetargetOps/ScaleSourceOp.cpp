// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/ScaleSourceOp.h"

#include "Retargeter/IKRetargetProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScaleSourceOp)

bool FRetargetPoseScaleWithPivot::ScalePose(TArray<FTransform>& InOutGlobalPose) const
{
	if (FMath::IsNearlyEqual(Factor, 1.0))
	{
		return false;
	}
	
	// scale the input pose
	for (FTransform& SourceBoneTransform : InOutGlobalPose)
	{
		const FVector ScaleBoneOffset = (SourceBoneTransform.GetTranslation() - Pivot) * Factor;
		SourceBoneTransform.SetTranslation(Pivot + ScaleBoneOffset);
	}

	return true;
}

const UClass* FIKRetargetScaleSourceOpSettings::GetControllerType() const
{
	return UIKRetargetScaleSourceController::StaticClass();
}

void FIKRetargetScaleSourceOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy ALL properties
	static TArray<FName> PropertiesToIgnore;
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetScaleSourceOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

#if WITH_EDITORONLY_DATA
USkeleton* FIKRetargetScaleSourceOpSettings::GetSkeleton(const FName InPropertyName)
{
	return const_cast<USkeleton*>(SourceSkeletonAsset);
}
#endif

bool FIKRetargetScaleSourceOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& Log)
{
	bIsInitialized = true;
	return true;
}

FIKRetargetOpSettingsBase* FIKRetargetScaleSourceOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetScaleSourceOp::GetSettingsType() const
{
	return FIKRetargetScaleSourceOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetScaleSourceOp::GetType() const
{
	return FIKRetargetScaleSourceOp::StaticStruct();
}

int32 FIKRetargetScaleSourceOp::GetScalePivotBoneIndex(const FRetargetSkeleton& InSourceSkeleton) const
{
	if (CachedScalePivotBoneName != Settings.ScalePivotBone.BoneName)
	{
		CachedScalePivotBoneIndex = InSourceSkeleton.FindBoneIndexByName(Settings.ScalePivotBone.BoneName);
		CachedScalePivotBoneName = Settings.ScalePivotBone.BoneName;
	}

	return CachedScalePivotBoneIndex;
}

FIKRetargetScaleSourceOpSettings UIKRetargetScaleSourceController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetScaleSourceOpSettings*>(OpSettingsToControl);
}

void UIKRetargetScaleSourceController::SetSettings(FIKRetargetScaleSourceOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}
