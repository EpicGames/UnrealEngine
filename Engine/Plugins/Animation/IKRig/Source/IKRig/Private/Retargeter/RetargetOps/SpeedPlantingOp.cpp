// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/SpeedPlantingOp.h"

#include "Retargeter/IKRetargetProcessor.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimNodeBase.h"
#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"

#if WITH_EDITOR
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SpeedPlantingOp)

#define LOCTEXT_NAMESPACE "SpeedPlantingOp"

bool FRetargetSpeedPlantingSettings::operator==(const FRetargetSpeedPlantingSettings& Other) const
{
	return SpeedCurveName == Other.SpeedCurveName && FMath::IsNearlyEqualByULP(Alpha,Other.Alpha);
}

const UClass* FIKRetargetSpeedPlantingOpSettings::GetControllerType() const
{
	return UIKRetargetSpeedPlantingController::StaticClass();
}

void FIKRetargetSpeedPlantingOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copies everything except "ChainsToSpeedPlant"
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetSpeedPlantingOpSettings, ChainsToSpeedPlant)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetSpeedPlantingOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);

	// copy chains
	const FIKRetargetSpeedPlantingOpSettings* NewSettings = reinterpret_cast<const FIKRetargetSpeedPlantingOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->ChainsToSpeedPlant, ChainsToSpeedPlant);
}

void FIKRetargetSpeedPlantingOpSettings::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	SpringStrength = FMath::Sqrt(Stiffness) / (2.0 * PI);
}

void FSpeedPlantingGoalState::Update(
	FIKRigGoal* IKRigGoal,
	const double InDeltaTime,
	const FIKRetargetSpeedPlantingOpSettings& InSettings)
{
	if (CurrentSpeedValue < 0.0f)
    {
       return; 
    }

    if (bWasReset)
    {
        CurrentPosition = TargetPosition = PlantLocation = LastTargetPosition = IKRigGoal->Position;
        AccelerationTimer = 0.0f;
        bIsTrackingTarget = true;
        bWasReset = false;
    }

    // update planting State
    const bool bWasPlanted = bIsPlanted;
    bIsPlanted = CurrentSpeedValue < InSettings.SpeedThreshold;

    if (bIsPlanted && !bWasPlanted)
    {
    	// planted this tick
    	PlantLocation = IKRigGoal->Position;
    }

	if (bWasPlanted && !bIsPlanted)
    {
		// unplanted this tick
		bIsTrackingTarget = false;
    }

	// target is either the planted location or the live animation
	TargetPosition = bIsPlanted ? PlantLocation : IKRigGoal->Position;

    // update CurrentPosition
    if (InSettings.BlendMethod == ESpeedPlantBlendMethod::Spring)
    {
    	//
    	// spring method
    	//
    	
        CurrentPosition = UKismetMathLibrary::VectorSpringInterp(
            CurrentPosition,
            TargetPosition,
            PositionSpringState,
            (InSettings.SpringStrength * 2.0 * PI) * (InSettings.SpringStrength * 2.0 * PI),
            1.0, 
            InDeltaTime,
            1.0, 
            0.0);
    }
	else if (InSettings.BlendMethod == ESpeedPlantBlendMethod::MoveTowards)
	{
		//
		// move towards method
		//

		// calculate Target Velocity (how fast the animation is moving)
		FVector TargetVelocity = (IKRigGoal->Position - LastTargetPosition) / FMath::Max(0.0001f, InDeltaTime);
		LastTargetPosition = IKRigGoal->Position;

		if (bIsTrackingTarget)
		{
			CurrentPosition = TargetPosition;
		}
		else
		{
			// calc distance to target
			const FVector ToTarget = TargetPosition - CurrentPosition;
			const float DistanceToTarget = ToTarget.Size();
			const FVector DirectionToTarget = ToTarget.GetSafeNormal();

			// determine how much "extra" speed we can use to close the gap
			const float MaxBoostSpeed = FMath::Max(0.0f, InSettings.MaxBlendSpeed - (float)TargetVelocity.Size());

			// factor in the arrival speed gain: as DistanceToTarget -> 0, DistanceLimitedBoost -> 0
			const float DistanceLimitedBoost = DistanceToTarget * InSettings.ArrivalSpeedGain;

			// current Boost is the lesser of our acceleration ramp or our braking limit
			// time-based ramp (0-1) to handle the initial lift-off acceleration
			AccelerationTimer += InDeltaTime;
			float TimeAlpha = FMath::Clamp(AccelerationTimer / FMath::Max(0.001f, InSettings.TimeToMaxSpeed), 0.0f, 1.0f);
			TimeAlpha = TimeAlpha * TimeAlpha; // quadratic falloff
			const float CurrentBoost = FMath::Min(TimeAlpha * MaxBoostSpeed, DistanceLimitedBoost);

			// final velocity is the animation's speed + our corrective boost (which fades off as we approach target)
			FVector DesiredVelocity = TargetVelocity + (DirectionToTarget * CurrentBoost);

			// step towards target
			const FVector MoveStep = DesiredVelocity * InDeltaTime;
			if (MoveStep.Size() >= DistanceToTarget || DistanceToTarget < 0.1f)
			{
				// if the step is larger than the distance, we have arrived and should begin tracking animation 1:1
				CurrentPosition = TargetPosition;
				bIsTrackingTarget = true;
				AccelerationTimer = 0.0f;
			}
			else
			{
				CurrentPosition += MoveStep;
			}
		}
	}

	const float Alpha = Settings->Alpha * InSettings.Alpha;
	IKRigGoal->Position = FMath::Lerp(IKRigGoal->Position, CurrentPosition, Alpha);
}

void FSpeedPlantingGoalState::Reset()
{
	bIsPlanted = false;
	bWasReset = true;
	PositionSpringState.Reset();
}

bool FIKRetargetSpeedPlantingOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;
	
	// this op requires a parent to supply an IK Rig
	if (!ensure(InParentOp))
	{
		return false;
	}

	// validate that an IK rig has been assigned
	const FIKRetargetRunIKRigOp* ParentOp = reinterpret_cast<const FIKRetargetRunIKRigOp*>(InParentOp);
	if (ParentOp->Settings.IKRigAsset == nullptr)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingIKRig", "{0}, is missing an IK rig. No chains can be retargeted. "), FText::FromName(GetName())));
		return false;
	}

	GoalsToPlant.Reset();

	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (const FRetargetSpeedPlantingSettings& SettingsForChain : Settings.ChainsToSpeedPlant)
	{
		FName TargetChainName = SettingsForChain.TargetChainName;
		const FResolvedBoneChain* SourceBoneChain = nullptr;
		const FResolvedBoneChain* TargetBoneChain = nullptr;
		if (!FIKRetargetRunIKRigOp::ValidateIKChain(
			TargetChainName,
			GoalContainer,
			BoneChains,
			this,
			InParentOp,
			InLog,
			SourceBoneChain /*out*/,
			TargetBoneChain /*out*/))
		{
			continue;
		}
		
		GoalsToPlant.Emplace(TargetBoneChain->IKGoalName, &SettingsForChain);
	}

	bIsInitialized = !GoalsToPlant.IsEmpty();
	return bIsInitialized;
}

void FIKRetargetSpeedPlantingOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	if (InProcessor.IsIKForcedOff())
	{
		return; // skip this op when IK is off
	}
	
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	
	for (FSpeedPlantingGoalState& GoalToPlant : GoalsToPlant)
	{
		FIKRigGoal* IKRigGoal = GoalContainer.FindGoalByName(GoalToPlant.GoalName);
		if (!IKRigGoal)
		{
			continue; // goal excluded, just ignore it
		}

		GoalToPlant.Update(IKRigGoal, InDeltaTime, Settings);
	}
}

void FIKRetargetSpeedPlantingOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetSpeedPlantingOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

FIKRetargetOpSettingsBase* FIKRetargetSpeedPlantingOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetSpeedPlantingOp::GetSettingsType() const
{
	return FIKRetargetSpeedPlantingOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetSpeedPlantingOp::GetType() const
{
	return FIKRetargetSpeedPlantingOp::StaticStruct();
}

const UScriptStruct* FIKRetargetSpeedPlantingOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

void FIKRetargetSpeedPlantingOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	for (FRetargetSpeedPlantingSettings& ChainSettings : Settings.ChainsToSpeedPlant)
	{
		if (ChainSettings.TargetChainName == InOldChainName)
		{
			ChainSettings.TargetChainName = InNewChainName;
		}
	}
}

void FIKRetargetSpeedPlantingOp::OnParentReinitPropertyEdited(
	const FIKRetargetOpBase& InParentOp,
	const FPropertyChangedEvent* InPropertyChangedEvent)
{
	RegenerateChainSettings(&InParentOp);
}

void FIKRetargetSpeedPlantingOp::OnPlaybackReset()
{
	for (FSpeedPlantingGoalState& GoalToPlant : GoalsToPlant)
	{
		GoalToPlant.Reset();
	}
}

void FIKRetargetSpeedPlantingOp::AnimGraphPreUpdateMainThread(
	USkeletalMeshComponent& SourceMeshComponent,
	USkeletalMeshComponent& TargetMeshComponent)
{
	if (!bIsInitialized)
	{
		return;
	}

	const UAnimInstance* SourceAnimInstance = SourceMeshComponent.GetAnimInstance();
	if (!SourceAnimInstance)
	{
		return;
	}

	// update speed values for each planted chain
	// NOTE: these are values from curves running on the SOURCE skeletal mesh
	// they will be overriden by any values coming from the target in AnimGraphEvaluateAnyThread
	const TMap<FName, float>& AnimCurveList = SourceAnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve);
	for (FSpeedPlantingGoalState& GoalToPlant : GoalsToPlant)
	{
		const float* SourceValue = AnimCurveList.Find(GoalToPlant.Settings->SpeedCurveName);
		GoalToPlant.CurrentSpeedValue = SourceValue ? *SourceValue : GoalToPlant.CurrentSpeedValue;
	}
}

void FIKRetargetSpeedPlantingOp::AnimGraphEvaluateAnyThread(FPoseContext& Output)
{
	if (!bIsInitialized)
	{
		return;
	}

	// update speed values for each planted chain
	// NOTE: these are values from curves running on the TARGET skeletal mesh
	// they will override any values coming from the source via AnimGraphPreUpdateMainThread()
	for (FSpeedPlantingGoalState& GoalToPlant : GoalsToPlant)
	{
		bool bFoundCurve = false;
		GoalToPlant.CurrentSpeedValue = Output.Curve.Get(GoalToPlant.Settings->SpeedCurveName, bFoundCurve, static_cast<float>(GoalToPlant.CurrentSpeedValue));
	}
}

TArray<FName> FIKRetargetSpeedPlantingOp::GetRequiredSpeedCurves() const
{
	TArray<FName> OutSpeedCurveNames;
	for (const FRetargetSpeedPlantingSettings& ChainToPlant : Settings.ChainsToSpeedPlant)
	{
		const FName SpeedCurveName = ChainToPlant.SpeedCurveName;
		if (SpeedCurveName != NAME_None)
		{
			OutSpeedCurveNames.Add(SpeedCurveName);
		}
	}
	return MoveTemp(OutSpeedCurveNames);
}



#if WITH_EDITOR

void FIKRetargetSpeedPlantingOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	for (const FSpeedPlantingGoalState& GoalToPlant : GoalsToPlant)
	{
		FLinearColor Color = GoalToPlant.bIsPlanted ? FLinearColor::Red : FLinearColor::Green;
		if (Settings.BlendMethod == ESpeedPlantBlendMethod::MoveTowards && !GoalToPlant.bIsTrackingTarget)
		{
			Color = FLinearColor::Yellow;
		}
		
		const FVector WorldPosition = InComponentTransform.TransformPosition(GoalToPlant.CurrentPosition);
		DrawWireSphere(
				InPDI,
				WorldPosition,
				Color,
				Settings.DebugDrawSize,
				12,
				SDPG_MAX,
				0.0f,
				0.001f,
				false);
	}
}

FText FIKRetargetSpeedPlantingOp::GetWarningMessage() const
{
	// warn about missing curves
	if (bIsInitialized)
	{
		int32 NumMissingCurves = 0;
		for (const FSpeedPlantingGoalState& GoalToPlant : GoalsToPlant)
		{
			if (!GoalToPlant.bFoundCurveInSourceComponent && !GoalToPlant.bFoundCurveInTargetComponent)
			{
				NumMissingCurves++;
			}
		}

		if (NumMissingCurves > 0)
		{
			return FText::Format(LOCTEXT("MissingSpeedCurves", "Running, but missing {0} speed curves."), FText::AsNumber(NumMissingCurves));
		}
	}

	return FIKRetargetOpBase::GetWarningMessage();
}

void FIKRetargetSpeedPlantingOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.ChainsToSpeedPlant, InChainName);
}

bool FIKRetargetSpeedPlantingOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.ChainsToSpeedPlant, InChainName);
}

uint8* FIKRetargetSpeedPlantingOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.ChainsToSpeedPlant, InChainName);
}
#endif

void FIKRetargetSpeedPlantingOp::RegenerateChainSettings(const FIKRetargetOpBase* InParentOp)
{
	constexpr bool bSkipUnmappedChains = true;
	constexpr bool bSkipNonIKChains = true;
	IKRetargetOpUtils::SynchronizeChainSettingsWithIKRig(Settings.ChainsToSpeedPlant, InParentOp, bSkipUnmappedChains, bSkipNonIKChains);
}

FIKRetargetSpeedPlantingOpSettings UIKRetargetSpeedPlantingController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetSpeedPlantingOpSettings*>(OpSettingsToControl);
}

void UIKRetargetSpeedPlantingController::SetSettings(FIKRetargetSpeedPlantingOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
