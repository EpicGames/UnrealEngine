// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/StretchChainOp.h"

#include "Engine/SkeletalMesh.h"
#include "Retargeter/IKRetargetOpUtils.h"
#include "Retargeter/IKRetargetProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StretchChainOp)

#define LOCTEXT_NAMESPACE "StretchChainOp"

FChainStretcher::FChainStretcher(
	const FRetargetStretchChainSettings* InSettings,
	const FResolvedBoneChain& InSourceBoneChain,
	const FResolvedBoneChain& InTargetBoneChain)
{
	Settings = InSettings;
	SourceBoneChain = &InSourceBoneChain;
	TargetBoneChain = &InTargetBoneChain;

	// lazy initialized to avoid wasting memory
	bIsInitialized = false;
}

void FChainStretcher::Initialize(const FRetargetSkeleton& InTargetSkeleton)
{
	// cache indices of all children of all bones in chain (recursively)
	AllChildrenOfChain.Reset();
	InTargetSkeleton.GetChildrenIndicesRecursive(TargetBoneChain->BoneIndices[0],AllChildrenOfChain);
	// filter out indices of the chain itself
	const TArray<int32>& IndicesToRemove = TargetBoneChain->BoneIndices;
	AllChildrenOfChain.RemoveAll([&IndicesToRemove](const int32& Element)
	{
		return IndicesToRemove.Contains(Element);
	});

	bIsInitialized = true;
}

void FChainStretcher::StretchChain(
	const TArray<FTransform> &InSourceGlobalPose,
	const FRetargetSkeleton& InTargetSkeleton,
	TArray<FTransform> &OutTargetGlobalPose)
{
	if (!(Settings && Settings->bEnabled))
	{
		return;
	}

	if (!bIsInitialized)
	{
		Initialize(InTargetSkeleton);
	}
	
	const bool bMatchingSourceLength = !FMath::IsNearlyZero(Settings->MatchSourceLength);
	const bool bScalingChainLength = !FMath::IsNearlyEqual(Settings->ScaleChainLength, 1.0f);
	if (!bMatchingSourceLength && !bScalingChainLength)
	{
		// no operations to perform
		return;
	}

	// store input local spaces of all children to propagate to
	TArray<FTransform> AllChildrenInputLocalTransforms = InTargetSkeleton.GetLocalTransformsOfMultipleBones(AllChildrenOfChain, OutTargetGlobalPose);

	// get the lengths of the chains as they currently are at this point in the op stack
	const TArray<FTransform> InputTargetChainGlobal = TargetBoneChain->GetChainTransformsFromPose(OutTargetGlobalPose);
	const TArray<FTransform> InputSourceChainGlobal = SourceBoneChain->GetChainTransformsFromPose(InSourceGlobalPose);
	const double CurrentTargetChainLength = FResolvedBoneChain::GetChainLength(InputTargetChainGlobal);
	const double CurrentSourceChainLength = FResolvedBoneChain::GetChainLength(InputSourceChainGlobal);
	if (FMath::IsNearlyZero(CurrentTargetChainLength))
	{
		return; // cannot scale a collapsed chain
	}

	// determine amount to scale the chain to match the source
	double TargetLengthToMatch = FMath::Lerp(CurrentTargetChainLength, CurrentSourceChainLength, Settings->MatchSourceLength);
	const double MatchLengthMultiplier = TargetLengthToMatch / CurrentTargetChainLength;

	// scale the length of each bone in the chain
	for (int32 ChainIndex = 1; ChainIndex < TargetBoneChain->BoneIndices.Num(); ChainIndex++)
	{
		const FVector InputBoneVectorGlobal = InputTargetChainGlobal[ChainIndex].GetLocation() - InputTargetChainGlobal[ChainIndex-1].GetLocation();
		const FVector NewBoneVector = InputBoneVectorGlobal * (MatchLengthMultiplier * Settings->ScaleChainLength);
		const int32 BoneIndex = TargetBoneChain->BoneIndices[ChainIndex];
		const int32 ParentBoneIndex = TargetBoneChain->BoneIndices[ChainIndex-1];
		const FVector NewLocation = OutTargetGlobalPose[ParentBoneIndex].GetLocation() + NewBoneVector;
		OutTargetGlobalPose[BoneIndex].SetLocation(NewLocation);
	}

	// now propagate changes to children of the chain
	InTargetSkeleton.UpdateGlobalTransformsOfMultipleBones(AllChildrenOfChain, AllChildrenInputLocalTransforms,OutTargetGlobalPose);
}

bool FRetargetStretchChainSettings::operator==(const FRetargetStretchChainSettings& Other) const
{
	return bEnabled == Other.bEnabled
		&& FMath::IsNearlyEqualByULP(MatchSourceLength,Other.MatchSourceLength)
		&& FMath::IsNearlyEqualByULP(ScaleChainLength, Other.ScaleChainLength);
}

const UClass* FIKRetargetStretchChainOpSettings::GetControllerType() const
{
	return UIKRetargetStretchChainController::StaticClass();
}

void FIKRetargetStretchChainOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy chains
	const FIKRetargetStretchChainOpSettings* NewSettings = reinterpret_cast<const FIKRetargetStretchChainOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->ChainsToStretch, ChainsToStretch);
}

bool FIKRetargetStretchChainOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;

	ChainStretchers.Reset();

	// spin through all the mapped retarget bone chains and load them
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	for (const FRetargetStretchChainSettings& ChainSettings : Settings.ChainsToStretch)
	{
		const FResolvedBoneChain* TargetBoneChain = BoneChains.GetResolvedBoneChainByName(
			ChainSettings.TargetChainName,
			ERetargetSourceOrTarget::Target,
			Settings.IKRigAsset);
		if (TargetBoneChain == nullptr)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("ChainStretchOpMissingChain", "Chain Stretch Op: chain data is out of sync with IK Rig. Missing target chain, '{0}."),
			FText::FromName(ChainSettings.TargetChainName)));
			continue;
		}

		// which source chain was this target chain mapped to?
		const FName SourceChainName = ChainMapping.GetChainMappedTo(ChainSettings.TargetChainName, ERetargetSourceOrTarget::Target);
		const FResolvedBoneChain* SourceBoneChain = BoneChains.GetResolvedBoneChainByName(SourceChainName, ERetargetSourceOrTarget::Source);
		if (SourceBoneChain == nullptr)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("ChainStretchOpMissingSource", "Chain Stretch Op: missing source chain. Cannot stretch a target chain without a source chain, '{0}."),
			FText::FromName(ChainSettings.TargetChainName)));
			continue;
		}
		
		// store a chain stretcher
		ChainStretchers.Emplace(&ChainSettings, *SourceBoneChain, *TargetBoneChain);
	}

	bIsInitialized = true;
	return true;
}

void FIKRetargetStretchChainOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	const FTargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
	for (FChainStretcher& ChainStretcher : ChainStretchers)
	{
		ChainStretcher.StretchChain(InSourceGlobalPose, TargetSkeleton, OutTargetGlobalPose);
	}
}

void FIKRetargetStretchChainOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	// on initial setup, use the default source/target IK rigs
	const UIKRigDefinition* SourceIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source);
	const UIKRigDefinition* TargetIKRig = InRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target);
	ApplyIKRigs(SourceIKRig, TargetIKRig);

	// auto map
	static bool bForceRemap = true;
	ChainMapping.AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);
}

void FIKRetargetStretchChainOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	// get the newly assigned IK Rig
	const bool bAssignedSource = SourceOrTarget == ERetargetSourceOrTarget::Source;
	const UIKRigDefinition* SourceIKRig = bAssignedSource ? InIKRig : ChainMapping.GetIKRig(ERetargetSourceOrTarget::Source);
	const bool bAssignedTarget = SourceOrTarget == ERetargetSourceOrTarget::Target;
	const UIKRigDefinition* TargetIKRig = bAssignedTarget ? InIKRig : ChainMapping.GetIKRig(ERetargetSourceOrTarget::Target);
	ApplyIKRigs(SourceIKRig, TargetIKRig);

	// auto map, but don't force it (leave existing mappings alone)
	static bool bForceRemap = false;
	ChainMapping.AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);
}

FIKRetargetOpSettingsBase* FIKRetargetStretchChainOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetStretchChainOp::GetSettingsType() const
{
	return FIKRetargetStretchChainOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetStretchChainOp::GetType() const
{
	return FIKRetargetStretchChainOp::StaticStruct();
}

const UIKRigDefinition* FIKRetargetStretchChainOp::GetCustomTargetIKRig() const
{
	return Settings.IKRigAsset;
}

FRetargetChainMapping* FIKRetargetStretchChainOp::GetChainMapping()
{
	return &ChainMapping;
}

void FIKRetargetStretchChainOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	for (FRetargetStretchChainSettings& ChainSettings : Settings.ChainsToStretch)
	{
		if (ChainSettings.TargetChainName == InOldChainName)
		{
			ChainSettings.TargetChainName = InNewChainName;
		}
	}
}

void FIKRetargetStretchChainOp::OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent)
{
	const UIKRigDefinition* SourceIKRig = ChainMapping.GetIKRig(ERetargetSourceOrTarget::Source);
	const UIKRigDefinition* TargetIKRig = Settings.IKRigAsset;
	ApplyIKRigs(SourceIKRig, TargetIKRig);
}

#if WITH_EDITOR
FText FIKRetargetStretchChainOp::GetWarningMessage() const
{
	if (!bIsInitialized && Settings.ChainsToStretch.IsEmpty())
	{
		return LOCTEXT("NoChainsAssigned", "Not initialized. No chains assigned.");
	}
	
	return FIKRetargetOpBase::GetWarningMessage();
}

void FIKRetargetStretchChainOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.ChainsToStretch, InChainName);
}

bool FIKRetargetStretchChainOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.ChainsToStretch, InChainName);
}

uint8* FIKRetargetStretchChainOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.ChainsToStretch, InChainName);
}
#endif

void FIKRetargetStretchChainOp::ApplyIKRigs(const UIKRigDefinition* InSourceIKRig, const UIKRigDefinition* InTargetIKRig)
{
	// store IK Rig
	Settings.IKRigAsset = InTargetIKRig;
	
	// update chain mapping
	ChainMapping.ReinitializeWithIKRigs(InSourceIKRig, InTargetIKRig);

	// update settings only if we have a valid mapping
	if (!ChainMapping.IsReady())
	{
		// don't remove settings, instead we want to preserve existing settings at least until the next valid rig is loaded
		return;
	}
	
	// get the required target chains
	const TArray<FBoneChain>& AllTargetChains = Settings.IKRigAsset->GetRetargetChains();
	TArray<FName> RequiredTargetChains;
	for (const FBoneChain& ChainToRetarget : AllTargetChains)
	{
		RequiredTargetChains.Add(ChainToRetarget.ChainName);
	}
	
	// remove chains that are not required
	Settings.ChainsToStretch.RemoveAll([&RequiredTargetChains](const FRetargetStretchChainSettings& InChainSettings)
	{
		return !RequiredTargetChains.Contains(InChainSettings.TargetChainName);
	});

	// add any required chains not already present
	for (const FName RequiredTargetChain : RequiredTargetChains)
	{
		bool bFound = false;
		for (const FRetargetStretchChainSettings& ChainToStretch : Settings.ChainsToStretch)
		{
			if (ChainToStretch.TargetChainName == RequiredTargetChain)
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			Settings.ChainsToStretch.Emplace(RequiredTargetChain);
		}
	}
}

FIKRetargetStretchChainOpSettings UIKRetargetStretchChainController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetStretchChainOpSettings*>(OpSettingsToControl);
}

void UIKRetargetStretchChainController::SetSettings(FIKRetargetStretchChainOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
