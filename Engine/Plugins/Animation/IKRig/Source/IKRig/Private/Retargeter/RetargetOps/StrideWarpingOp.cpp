// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/StrideWarpingOp.h"

#include "IKRigObjectVersion.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "Rig/Solvers/PointsToRotation.h"

#if WITH_EDITOR
#include "PrimitiveDrawingUtils.h"
#include "Engine/EngineTypes.h"
#endif

#include "Retargeter/IKRetargetOpUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StrideWarpingOp)

#define LOCTEXT_NAMESPACE "StrideWarpingOp"


bool FRetargetStrideWarpChainSettings::operator==(const FRetargetStrideWarpChainSettings& Other) const
{
	return EnableStrideWarping == Other.EnableStrideWarping;
}

const UClass* FIKRetargetStrideWarpingOpSettings::GetControllerType() const
{
	return UIKRetargetStrideWarpingController::StaticClass();
}

void FIKRetargetStrideWarpingOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the chains
	const TArray<FName> PropertiesToIgnore = {GET_MEMBER_NAME_CHECKED(FIKRetargetStrideWarpingOpSettings, ChainSettings)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetStrideWarpingOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);

	// copy chains
	const FIKRetargetStrideWarpingOpSettings* NewSettings = reinterpret_cast<const FIKRetargetStrideWarpingOpSettings*>(InSettingsToCopyFrom);
	IKRetargetOpUtils::CopyChainSettingsAtRuntime(NewSettings->ChainSettings, ChainSettings);
}

FVector FIKRetargetStrideWarpingOpSettings::GetAxisVector(const EBasicAxis& Axis)
{
	switch (Axis)
	{
	case EBasicAxis::X:
		return FVector::XAxisVector;
	case EBasicAxis::Y:
		return FVector::YAxisVector;
	case EBasicAxis::Z:
		return FVector::ZAxisVector;
	case EBasicAxis::NegX:
		return -FVector::XAxisVector;
	case EBasicAxis::NegY:
		return -FVector::YAxisVector;
	case EBasicAxis::NegZ:
		return -FVector::ZAxisVector;
	default:
		checkNoEntry();
		return FVector::ZeroVector;
	}
}

bool FIKRetargetStrideWarpingOpSettings::operator==(const FIKRetargetStrideWarpingOpSettings& Other) const
{
	return DirectionSource == Other.DirectionSource
		&& ForwardDirection == Other.ForwardDirection
		&& DirectionChain == Other.DirectionChain
		&& FMath::IsNearlyEqualByULP(WarpForwards, Other.WarpForwards)
		&& FMath::IsNearlyEqualByULP(SidewaysOffset, Other.SidewaysOffset)
		&& FMath::IsNearlyEqualByULP(WarpSplay, Other.WarpSplay);
}

void FIKRetargetStrideWarpingOpSettings::PostLoad(const FIKRigObjectVersion::Type InVersion)
{
	FIKRetargetOpSettingsBase::PostLoad(InVersion);
	
	if (InVersion < FIKRigObjectVersion::AddedDebugDrawTogglePerOp)
	{
		bDebugDraw = bEnableDebugDraw_DEPRECATED;
	}
}

bool FIKRetargetStrideWarpingOpSettings::Serialize(FArchive& Ar)
{
	return SerializeOpWithVersion(Ar, *this);
}

bool FIKRetargetStrideWarpingOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = false;
	GoalsToWarp.Reset();

	// this op requires a parent to supply an IK Rig
	if (!ensure(InParentOp))
	{
		return false;
	}

	// validate that an IK rig has been assigned
	const FIKRetargetRunIKRigOp* ParentOp = reinterpret_cast<const FIKRetargetRunIKRigOp*>(InParentOp);
	if (ParentOp->Settings.IKRigAsset == nullptr)
	{
		InLog.LogWarning( FText::Format(
		LOCTEXT("MissingIKRig", "{0}, is missing an IK rig. No chains can be retargeted. "), FText::FromName(GetName())));
		return false;
	}

	// store the target IK rig for querying bone chains
	TargetIKRig = ParentOp->Settings.IKRigAsset;
	
	const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
	const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	const TArray<FTransform>& TargetGlobalRetargetPose = InTargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	for (const FRetargetStrideWarpChainSettings& ChainSettings : Settings.ChainSettings)
	{
		if (!ChainSettings.EnableStrideWarping)
		{
			continue;
		}
		
		const FResolvedBoneChain* TargetBoneChain = BoneChains.GetResolvedBoneChainByName(
			ChainSettings.TargetChainName,
			ERetargetSourceOrTarget::Target,
			TargetIKRig);
		if (TargetBoneChain == nullptr)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("StrideWarpingMissingChain", "Stride Warping Op: chain data is out of sync with IK Rig. Missing target chain, '{0}."),
			FText::FromName(ChainSettings.TargetChainName)));
			continue;
		}

		if (TargetBoneChain->IKGoalName == NAME_None)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("StrideWarpingChainWithNoGoal", "Stride Warping Op: specified chain does not have an IK goal. Cannot stride warp, '{0}."),
			FText::FromName(ChainSettings.TargetChainName)));
			continue;
		}

		const FIKRigGoal* Goal = GoalContainer.FindGoalByName(TargetBoneChain->IKGoalName);
		if (Goal == nullptr)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("StrideWarpingRigWithNoGoal", "Stride Warping Op: target chain references a goal that is not present in the IK Rig, '{0}."),
			FText::FromName(TargetBoneChain->IKGoalName)));
			continue;
		}

		const int32 GoalBoneIndex = InTargetSkeleton.FindBoneIndexByName(Goal->BoneName);
		if (GoalBoneIndex == INDEX_NONE)
		{
			InLog.LogWarning( FText::Format(
			LOCTEXT("StrideWarpingMissingBone", "Stride Warping Op: IK goal, '{0}' references missing bone, '{1}'."),
			FText::FromName(TargetBoneChain->IKGoalName), FText::FromName(Goal->BoneName)));
			continue;
		}
		
		// store goal to warp
		GoalsToWarp.Emplace(Goal->Name, TargetGlobalRetargetPose[GoalBoneIndex], ChainSettings);
	}

	bIsInitialized = !GoalsToWarp.IsEmpty();
	return bIsInitialized;
}

void FIKRetargetStrideWarpingOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	if (InProcessor.IsIKForcedOff())
	{
		return; // skip this op when IK is off
	}
	
	auto ConvertForwardVectorsToRotation = [](FVector& ForwardOrig, FVector& ForwardCurrent) -> FQuat
	{
		// project on floor (we only want Yaw) and normalize
		ForwardOrig.Z = 0.0f;
		ForwardCurrent.Z = 0.0f;
		ForwardOrig.Normalize();
		ForwardCurrent.Normalize();
		// rotate from orig facing direction to current
		return FQuat::FindBetweenNormals(ForwardOrig, ForwardCurrent);
	};

	// we need to determine the character's forward and side directions for warping
	FVector InitialBodyPosition;
	FVector CurrentBodyPosition;
	FQuat CurrentRotation;
	
	// get sample points to use for "best fit" global body rotation
	switch(Settings.DirectionSource)
	{
	case EWarpingDirectionSource::Goals:
		{
			TArray<FVector> InitialPoints;
			TArray<FVector> CurrentPoints;
			
			// use goals to determine the body's rotation
			const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
			for (const FStrideWarpGoalData& GoalToWarp : GoalsToWarp)
			{
				const FIKRigGoal* Goal = GoalContainer.FindGoalByName(GoalToWarp.IKRigGoalName);
				if (!Goal)
				{
					continue;
				}
				FVector A = GoalToWarp.GlobalRefPoseOfGoalBone.GetLocation();
				FVector B = Goal->Position;

				// flatten into 2D for more robust yaw construction (what really matters)
				A.Z = 0.0f;
				B.Z = 0.0f;
		
				InitialPoints.Add(A);
				CurrentPoints.Add(B);
			}

			// calculate "best fit" global body rotation based on deformation of sample points
			CurrentRotation = GetRotationFromDeformedPoints(
				InitialPoints,
				CurrentPoints,
				InitialBodyPosition,
				CurrentBodyPosition);
			
			break;
		}
	case EWarpingDirectionSource::Chain:
		{
			// use chain to determine the body's rotation
			const FRetargeterBoneChains& BoneChains = InProcessor.GetBoneChains();
			const FResolvedBoneChain* Chain = BoneChains.GetResolvedBoneChainByName(Settings.DirectionChain, ERetargetSourceOrTarget::Target, TargetIKRig);
			if (!Chain)
			{
				return;
			}

			const TArray<FTransform>& InitialChainTransforms = Chain->RefPoseGlobalTransforms;
			const TArray<FTransform> CurrentChainTransforms = Chain->GetChainTransformsFromPose(OutTargetGlobalPose);
			if (!ensure(!InitialChainTransforms.IsEmpty()))
			{
				return;
			}

			// calculate initial and current centroids
			InitialBodyPosition = FVector::ZeroVector;
			CurrentBodyPosition = FVector::ZeroVector;
			for (int32 TransformIndex=0; TransformIndex<InitialChainTransforms.Num(); ++TransformIndex)
			{
				InitialBodyPosition += InitialChainTransforms[TransformIndex].GetTranslation();
				CurrentBodyPosition += CurrentChainTransforms[TransformIndex].GetTranslation();
			}
			const float InvNum = 1.0f / static_cast<float>(InitialChainTransforms.Num());
			InitialBodyPosition *= InvNum;
			CurrentBodyPosition *= InvNum;

			// get the forward vectors of the chain
			FVector ForwardOrig;
			FVector ForwardCurrent;
			
			// get forward vectors: orig and current
			if (InitialChainTransforms.Num() == 1)
			{
				// in case of single bone chain, we rotate the global forward vector with this bone and then project onto floor
				const FQuat DeltaRotation = InitialChainTransforms[0].GetRotation() * CurrentChainTransforms[0].GetRotation().Inverse();
				ForwardOrig = Settings.GetAxisVector(Settings.ForwardDirection);
				ForwardCurrent = DeltaRotation.RotateVector(ForwardOrig);
			}
			else
			{
				// in case of multi-bone chain, we use the vector from the start to the end of the chain
				ForwardOrig = InitialChainTransforms.Last().GetTranslation() - InitialChainTransforms[0].GetTranslation();
				ForwardCurrent = CurrentChainTransforms.Last().GetTranslation() - CurrentChainTransforms[0].GetTranslation();
			}

			// get rotation from fwd vectors
			CurrentRotation = ConvertForwardVectorsToRotation(ForwardOrig, ForwardCurrent);
			break;
	}
	case EWarpingDirectionSource::RootBone:
		{
			const FTargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
			const TArray<FTransform>& TargetRetargetPose = TargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
			// use the root bone
			InitialBodyPosition = TargetRetargetPose[0].GetTranslation();
			CurrentBodyPosition = TargetSkeleton.OutputGlobalPose[0].GetTranslation();

			// get the forward vectors of the root bone
			const FQuat DeltaRotation = TargetRetargetPose[0].GetRotation() * TargetSkeleton.OutputGlobalPose[0].GetRotation().Inverse();
			FVector ForwardOrig = Settings.GetAxisVector(Settings.ForwardDirection);
			FVector ForwardCurrent = DeltaRotation.RotateVector(ForwardOrig);

			// get rotation from fwd vectors
			CurrentRotation = ConvertForwardVectorsToRotation(ForwardOrig, ForwardCurrent);
			break;
		}
	default:
		{
			checkNoEntry();
			return;
		}
	}
	
	FTransform CurrentBodyTransform = FTransform(CurrentRotation, CurrentBodyPosition);
	FTransform InitialBodyTransform = FTransform(FQuat::Identity, InitialBodyPosition);
	FVector Fwd = CurrentBodyTransform.TransformVector(Settings.GetAxisVector(Settings.ForwardDirection));
	FVector Side = FVector::CrossProduct(Fwd, FVector::ZAxisVector);
	FVector SideOrig = FVector::CrossProduct(Settings.GetAxisVector(Settings.ForwardDirection), FVector::ZAxisVector);

	// warp goal positions...
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	for (FStrideWarpGoalData& GoalToWarp : GoalsToWarp)
	{
		// get the goal we wish to apply warping to
		FIKRigGoal* Goal = GoalContainer.FindGoalByName(GoalToWarp.IKRigGoalName);
		if (!Goal)
		{
			continue;
		}

		// get initial goal position
		const FVector& InitialPosition = GoalToWarp.GlobalRefPoseOfGoalBone.GetLocation();
		// accumulate a warped position to apply to the goal
		FVector WarpedPosition = Goal->Position;
		
		// forward warping
		const FVector InitialGoalInOrigSpace = InitialBodyTransform.InverseTransformPosition(InitialPosition);
		const FVector IntialGoalInCurrentSpace = CurrentBodyTransform.TransformPosition(InitialGoalInOrigSpace);
		const FPlane FwdPlane(IntialGoalInCurrentSpace, Fwd);
		const FVector GoalProjOnFwdPlane = FwdPlane.PointPlaneProject(Goal->Position, FwdPlane);
		WarpedPosition = GoalProjOnFwdPlane + ((WarpedPosition - GoalProjOnFwdPlane) * Settings.WarpForwards);

		// sideways offset
		// first determine which side the goal is on originally
		float GoalSideMultiplier = static_cast<float>(FVector::DotProduct(InitialPosition.GetSafeNormal(), SideOrig));
		// push goal by offset in the newly calculated sideways direction
		WarpedPosition += Side * Settings.SidewaysOffset * GoalSideMultiplier;

		// splay warping
		FVector SplayOrigin = CurrentBodyPosition;
		SplayOrigin.Z = WarpedPosition.Z;
		WarpedPosition = SplayOrigin + (WarpedPosition - SplayOrigin) * Settings.WarpSplay;

		// alpha blend the goal location towards the warped position
		const float Alpha = GoalToWarp.Settings->Alpha * Settings.Alpha;
		Goal->Position = FMath::Lerp(Goal->Position, WarpedPosition, Alpha);

		// goals are additive by default, this one is now in component space
		Goal->PositionSpace = EIKRigGoalSpace::Component;
	}

#if WITH_EDITOR
	FScopeLock ScopeLock(&DebugDataMutex);
	DebugStrideWarpingFrame = CurrentBodyTransform;
#endif
}

void FIKRetargetStrideWarpingOp::OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

void FIKRetargetStrideWarpingOp::OnAssignIKRig(
	const ERetargetSourceOrTarget SourceOrTarget,
	const UIKRigDefinition* InIKRig,
	const FIKRetargetOpBase* InParentOp)
{
	RegenerateChainSettings(InParentOp);
}

FIKRetargetOpSettingsBase* FIKRetargetStrideWarpingOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetStrideWarpingOp::GetSettingsType() const
{
	return FIKRetargetStrideWarpingOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetStrideWarpingOp::GetType() const
{
	return FIKRetargetStrideWarpingOp::StaticStruct();
}

const UScriptStruct* FIKRetargetStrideWarpingOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

void FIKRetargetStrideWarpingOp::OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName)
{
	for (FRetargetStrideWarpChainSettings& ChainSettings : Settings.ChainSettings)
	{
		if (ChainSettings.TargetChainName == InOldChainName)
		{
			ChainSettings.TargetChainName = InNewChainName;
		}
	}
}

void FIKRetargetStrideWarpingOp::OnParentReinitPropertyEdited(
	const FIKRetargetOpBase& InParentOp,
	const FPropertyChangedEvent* InPropertyChangedEvent)
{
	RegenerateChainSettings(&InParentOp);
}

#if WITH_EDITOR

FCriticalSection FIKRetargetStrideWarpingOp::DebugDataMutex;

void FIKRetargetStrideWarpingOp::DebugDraw(
	FPrimitiveDrawInterface* InPDI,
	const FTransform& InSourceTransform,
	const FTransform& InComponentTransform,
	const double InComponentScale,
	const FIKRetargetDebugDrawState& InEditorState) const
{
	FScopeLock ScopeLock(&DebugDataMutex);
	
	FTransform WarpingFrame = DebugStrideWarpingFrame * InComponentTransform;
	DrawCoordinateSystem(
		InPDI,
		WarpingFrame.GetLocation(),
		WarpingFrame.GetRotation().Rotator(),
		static_cast<float>(Settings.DebugDrawSize * InComponentScale),
		SDPG_World,
		static_cast<float>(Settings.DebugDrawThickness * InComponentScale));
}

void FIKRetargetStrideWarpingOp::ResetChainSettingsToDefault(const FName& InChainName)
{
	IKRetargetOpUtils::ResetChainSettingsToDefault(Settings.ChainSettings, InChainName);
}

bool FIKRetargetStrideWarpingOp::AreChainSettingsAtDefault(const FName& InChainName)
{
	return IKRetargetOpUtils::AreChainSettingsAtDefault(Settings.ChainSettings, InChainName);
}

uint8* FIKRetargetStrideWarpingOp::GetChainSettingsMemory(const FName InChainName)
{
	return IKRetargetOpUtils::GetChainSettingsMemory(Settings.ChainSettings, InChainName);
}
#endif

void FIKRetargetStrideWarpingOp::RegenerateChainSettings(const FIKRetargetOpBase* InParentOp)
{
	const FIKRetargetRunIKRigOp* ParentOp = reinterpret_cast<const FIKRetargetRunIKRigOp*>(InParentOp);
	if (!ensure(ParentOp))
	{
		return;
	}
	
	// find the target chains that require goal retargeting
	const TArray<FName> RequiredTargetChains = ParentOp->GetRequiredTargetChains();
	if (RequiredTargetChains.IsEmpty())
	{
		// NOTE: if there's no chains, we don't clear the settings
		// this allows users to clear and reassign a different rig and potentially retain/restore compatible settings
		return;
	}

	// remove chains that are not required
	Settings.ChainSettings.RemoveAll([&RequiredTargetChains](const FRetargetStrideWarpChainSettings& InChainSettings)
	{
		return !RequiredTargetChains.Contains(InChainSettings.TargetChainName);
	});
	
	// add any required chains not already present
	for (FName RequiredTargetChain : RequiredTargetChains)
	{
		bool bFound = false;
		for (const FRetargetStrideWarpChainSettings& ChainToStrideWarp : Settings.ChainSettings)
		{
			if (ChainToStrideWarp.TargetChainName == RequiredTargetChain)
			{
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			Settings.ChainSettings.Emplace(RequiredTargetChain);
		}
	}
}

FIKRetargetStrideWarpingOpSettings UIKRetargetStrideWarpingController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetStrideWarpingOpSettings*>(OpSettingsToControl);
}

void UIKRetargetStrideWarpingController::SetSettings(FIKRetargetStrideWarpingOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
