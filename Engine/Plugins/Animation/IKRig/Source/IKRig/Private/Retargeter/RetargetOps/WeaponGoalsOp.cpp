// Copyright Epic Games, Inc. All Rights Reserved.

#include "Retargeter/RetargetOps/WeaponGoalsOp.h"

#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"

#define LOCTEXT_NAMESPACE "WeaponGoalsOp"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WeaponGoalsOp)

struct FIKRetargetWeaponGoalsOpSettings;


FIKRetargetWeaponGoalsOpSettings::FIKRetargetWeaponGoalsOpSettings()
{
	// construct with expected default names (based on UE skeleton convention)
	LeftWeaponBone.BoneName = FName("weapon_l");
	RightWeaponBone.BoneName = FName("weapon_r");
	LeftHandAttachBone.BoneName = FName("hand_attach_l");
	RightHandAttachBone.BoneName = FName("hand_attach_r");
	LeftHandBone.BoneName = FName("hand_l");
	RightHandBone.BoneName = FName("hand_r");
	IKHandLeftBone.BoneName = FName("ik_hand_l");
	IKHandRightBone.BoneName = FName("ik_hand_r");
	IKHandGun.BoneName = FName("ik_hand_gun");
}

const UClass* FIKRetargetWeaponGoalsOpSettings::GetControllerType() const
{
	return UIKRetargetWeaponGoalsOpController::StaticClass();
}

void FIKRetargetWeaponGoalsOpSettings::CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom)
{
	// copy everything except the bones (must be constant at runtime)
	const TArray<FName> PropertiesToIgnore = {
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, LeftWeaponBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, RightWeaponBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, LeftHandAttachBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, RightHandAttachBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, LeftHandBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, RightHandBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, IKHandLeftBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, IKHandRightBone),
		GET_MEMBER_NAME_CHECKED(FIKRetargetWeaponGoalsOpSettings, IKHandGun)};
	FIKRetargetOpBase::CopyStructProperties(
		FIKRetargetWeaponGoalsOpSettings::StaticStruct(),
		InSettingsToCopyFrom,
		this,
		PropertiesToIgnore);
}

bool FIKRetargetWeaponGoalOp::Initialize(
	const FIKRetargetProcessor& InProcessor,
	const FRetargetSkeleton& InSourceSkeleton,
	const FTargetSkeleton& InTargetSkeleton,
	const FIKRetargetOpBase* InParentOp,
	FIKRigLogger& InLog)
{
	bIsInitialized = true;
	
	// load target bones
	BoneIndices[(int32)ERetargetWeaponBone::LeftWeapon] = InTargetSkeleton.FindBoneIndexByName(Settings.LeftWeaponBone.BoneName);
	BoneIndices[(int32)ERetargetWeaponBone::RightWeapon] = InTargetSkeleton.FindBoneIndexByName(Settings.RightWeaponBone.BoneName);
	BoneIndices[(int32)ERetargetWeaponBone::LeftHandAttach] = InTargetSkeleton.FindBoneIndexByName(Settings.LeftHandAttachBone.BoneName);
	BoneIndices[(int32)ERetargetWeaponBone::RightHandAttach] = InTargetSkeleton.FindBoneIndexByName(Settings.RightHandAttachBone.BoneName);
	BoneIndices[(int32)ERetargetWeaponBone::LeftHand] = InTargetSkeleton.FindBoneIndexByName(Settings.LeftHandBone.BoneName);
	BoneIndices[(int32)ERetargetWeaponBone::RightHand] = InTargetSkeleton.FindBoneIndexByName(Settings.RightHandBone.BoneName);
	BoneIndices[(int32)ERetargetWeaponBone::IKHandLeft] = InTargetSkeleton.FindBoneIndexByName(Settings.IKHandLeftBone.BoneName);
	BoneIndices[(int32)ERetargetWeaponBone::IKHandRight] = InTargetSkeleton.FindBoneIndexByName(Settings.IKHandRightBone.BoneName);
	BoneIndices[(int32)ERetargetWeaponBone::HandGun] = InTargetSkeleton.FindBoneIndexByName(Settings.IKHandGun.BoneName);

	// warn if any bones were missing
	if (!Settings.bSuppressMissingBoneWarnings)
	{
		const UEnum* WeaponBoneEnum = StaticEnum<ERetargetWeaponBone>();
		check(WeaponBoneEnum);
		
		for (int32 Index = 0; Index < (int32)ERetargetWeaponBone::Count; ++Index)
		{
			if (BoneIndices[Index] == INDEX_NONE)
			{
				FText BoneEnumName = WeaponBoneEnum->GetDisplayNameTextByIndex(Index);
				FText ErrorMsg = FText::Format(LOCTEXT("WeaponGoalOpMissingBone", "Weapon Goals Op: Missing bone for '{0}'."),BoneEnumName);
				InLog.LogWarning(ErrorMsg);
			}
		}
	}

	// load and validate the goals associated with each hand
	const FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	const TArray<FIKRigGoal>& AllGoals = GoalContainer.GetGoalArray();
	LeftHandGoalName = NAME_None;
	RightHandGoalName = NAME_None;
	for (const FIKRigGoal& Goal : AllGoals)
	{
		if (Goal.BoneName == Settings.LeftHandBone.BoneName)
		{
			LeftHandGoalName = Goal.Name;
		}

		if (Goal.BoneName == Settings.RightHandBone.BoneName)
		{
			RightHandGoalName = Goal.Name;
		}
	}
	
	if (LeftHandGoalName == NAME_None)
	{
		FText ErrorMsg = FText::Format(LOCTEXT("WeaponGoalOpMissingLeftGoal", "Weapon Goals Op: No IK Goal found on Left Hand Bone, '{0}'."), FText::FromName(Settings.LeftHandBone.BoneName));
		InLog.LogWarning(ErrorMsg);
	}
	
	if (RightHandGoalName == NAME_None)
	{
		FText ErrorMsg = FText::Format(LOCTEXT("WeaponGoalOpMissingRightGoal", "Weapon Goals Op: No IK Goal found on Right Hand Bone, '{0}'."), FText::FromName(Settings.RightHandBone.BoneName));
		InLog.LogWarning(ErrorMsg);
	}
	
	return bIsInitialized;
}

void FIKRetargetWeaponGoalOp::Run(
	FIKRetargetProcessor& InProcessor,
	const double InDeltaTime,
	const TArray<FTransform>& InSourceGlobalPose,
	TArray<FTransform>& OutTargetGlobalPose)
{
	const FTargetSkeleton& TargetSkeleton = InProcessor.GetTargetSkeleton();
	
	// scale bones
	if (Settings.bEnableWeaponScale)
	{
		const FVector Scale(Settings.WeaponScale,Settings.WeaponScale,Settings.WeaponScale);
		TArray<int32, TFixedAllocator<5>> BonesToScale = {
			(int32)ERetargetWeaponBone::LeftWeapon,
			(int32)ERetargetWeaponBone::RightWeapon,
			(int32)ERetargetWeaponBone::LeftHandAttach,
			(int32)ERetargetWeaponBone::RightHandAttach,
			(int32)ERetargetWeaponBone::HandGun };
		const FVector FinalScale = FMath::Lerp(FVector::OneVector, Scale, Settings.Alpha);
		for (int32 Index : BonesToScale)
		{
			const int32 BoneIndex = BoneIndices[Index];
			if (BoneIndex != INDEX_NONE)
			{
				// scale the bone and propagate to children
				FTransform BoneTransform = OutTargetGlobalPose[BoneIndex];
				BoneTransform.SetScale3D(FinalScale);
				TargetSkeleton.SetGlobalTransformAndUpdateChildren(BoneIndex, BoneTransform, OutTargetGlobalPose);
			}
		}
	}

	// blend IK hand gun location between Left/Right hand
	if (Settings.bSnapIKHandGun)
	{
		const int32 LeftHandBoneIndex = BoneIndices[(int32)ERetargetWeaponBone::LeftHand];
		const int32 RightHandBoneIndex = BoneIndices[(int32)ERetargetWeaponBone::RightHand];
		const int32 HandGunBoneIndex = BoneIndices[(int32)ERetargetWeaponBone::HandGun];
		const bool bAllBonesFound = LeftHandBoneIndex != INDEX_NONE && RightHandBoneIndex != INDEX_NONE && HandGunBoneIndex != INDEX_NONE;
		if (bAllBonesFound)
		{
			const FTransform& LeftHand = OutTargetGlobalPose[LeftHandBoneIndex];
			const FTransform& RightHand = OutTargetGlobalPose[RightHandBoneIndex];
			const float Alpha = Settings.LeftRightHandIKAlpha * Settings.Alpha;
			const FVector BlendedPosition = FMath::Lerp(LeftHand.GetLocation(), RightHand.GetLocation(), Alpha);
			FTransform HandGunTransform = OutTargetGlobalPose[HandGunBoneIndex];
			HandGunTransform.SetTranslation(BlendedPosition);
			TargetSkeleton.SetGlobalTransformAndUpdateChildren(HandGunBoneIndex, HandGunTransform, OutTargetGlobalPose);
		}	
	}
	
	// blend the IK goals
	FIKRigGoalContainer& GoalContainer = InProcessor.GetIKRigGoalContainer();
	auto BlendGoalToIKHandBone = [&OutTargetGlobalPose](FIKRigGoal* InGoal, int32 InIKBoneIndex, float InAlpha)
		{
			const FTransform& IKBoneTransform = OutTargetGlobalPose[InIKBoneIndex];
			InGoal->Position = FMath::Lerp(InGoal->Position, IKBoneTransform.GetLocation(), InAlpha);
			const FQuat Rotation = FQuat::FastLerp(InGoal->Rotation.Quaternion(), IKBoneTransform.GetRotation(), InAlpha);
			InGoal->Rotation = Rotation.GetNormalized().Rotator();
		};
	if (FIKRigGoal* LeftHandGoal = GoalContainer.FindGoalByName(LeftHandGoalName))
	{
		const int32 IKHandLeftIndex = BoneIndices[(int32)ERetargetWeaponBone::IKHandLeft];
		if (IKHandLeftIndex != INDEX_NONE)
		{
			const float Alpha = Settings.LeftHandIKAlpha * Settings.Alpha;
			BlendGoalToIKHandBone(LeftHandGoal, IKHandLeftIndex, Alpha);
		}
	}
	if (FIKRigGoal* RightHandGoal = GoalContainer.FindGoalByName(RightHandGoalName))
	{
		const int32 IKHandRightIndex = BoneIndices[(int32)ERetargetWeaponBone::IKHandRight];
		if (IKHandRightIndex != INDEX_NONE)
		{
			const float Alpha = Settings.RightHandIKAlpha * Settings.Alpha;
			BlendGoalToIKHandBone(RightHandGoal, IKHandRightIndex, Alpha);
		}
	}
}

FIKRetargetOpSettingsBase* FIKRetargetWeaponGoalOp::GetSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRetargetWeaponGoalOp::GetSettingsType() const
{
	return FIKRetargetWeaponGoalsOpSettings::StaticStruct();
}

const UScriptStruct* FIKRetargetWeaponGoalOp::GetType() const
{
	return FIKRetargetWeaponGoalOp::StaticStruct();
}

const UScriptStruct* FIKRetargetWeaponGoalOp::GetParentOpType() const
{
	return FIKRetargetRunIKRigOp::StaticStruct();
}

FIKRetargetWeaponGoalsOpSettings UIKRetargetWeaponGoalsOpController::GetSettings()
{
	return *reinterpret_cast<FIKRetargetWeaponGoalsOpSettings*>(OpSettingsToControl);
}

void UIKRetargetWeaponGoalsOpController::SetSettings(FIKRetargetWeaponGoalsOpSettings InSettings)
{
	OpSettingsToControl->CopySettingsAtRuntime(&InSettings);
}

#undef LOCTEXT_NAMESPACE
