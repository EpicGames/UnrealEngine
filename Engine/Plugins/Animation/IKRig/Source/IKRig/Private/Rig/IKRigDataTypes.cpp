// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/IKRigDataTypes.h"
#include "Rig/IKRigDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigDataTypes)

void FIKRigGoalContainer::SetIKGoal(const FIKRigGoal& InGoal)
{
	FIKRigGoal* Goal = FindGoalWriteable(InGoal.Name);
	if (!Goal)
	{
		// container hasn't seen this goal before, create new one, copying the input goal
		Goals.Emplace(InGoal);
		// new goal added, rig needs initialized
		bRigNeedsInitialized = true;
		return;
	}

	// check if we need to reinitialize
	bRigNeedsInitialized = Goal->bEnabled != InGoal.bEnabled ? true : bRigNeedsInitialized;

	// copy settings to existing goal
	*Goal = InGoal;
}

void FIKRigGoalContainer::SetIKGoal(const UIKRigEffectorGoal* InEffectorGoal)
{
	if (!InEffectorGoal)
	{
		return;
	}
	
	FIKRigGoal* Goal = FindGoalWriteable(InEffectorGoal->GoalName);
	if (!Goal)
	{
		// container hasn't seen this goal before, create new one, copying the Effector goal
		Goals.Emplace(InEffectorGoal);
		// new goal added, rig needs initialized
		bRigNeedsInitialized = true;
		return;
	}
	
	// copy transform
	Goal->Position = InEffectorGoal->CurrentTransform.GetTranslation();
	Goal->Rotation = InEffectorGoal->CurrentTransform.Rotator();
    Goal->PositionAlpha = InEffectorGoal->PositionAlpha;
    Goal->RotationAlpha = InEffectorGoal->RotationAlpha;
	Goal->PositionSpace = EIKRigGoalSpace::Component;
	Goal->RotationSpace = EIKRigGoalSpace::Component;

	// goals in editor have "preview mode" which allows them to be additive relative to the goal initial position
#if WITH_EDITOR
	if (InEffectorGoal->PreviewMode == EIKRigGoalPreviewMode::Additive)
	{
		Goal->Position = InEffectorGoal->CurrentTransform.GetTranslation() - InEffectorGoal->InitialTransform.GetTranslation();
		const FQuat RelativeRotation = InEffectorGoal->CurrentTransform.GetRotation() * InEffectorGoal->InitialTransform.GetRotation().Inverse();
		Goal->Rotation = RelativeRotation.Rotator();
		Goal->PositionSpace = EIKRigGoalSpace::Additive;
		Goal->RotationSpace = EIKRigGoalSpace::Additive;
	}
#endif
}

const FIKRigGoal* FIKRigGoalContainer::FindGoalByName(const FName& GoalName) const
{
	return Goals.FindByPredicate([GoalName](const FIKRigGoal& Other){return Other.Name == GoalName;});
}

FIKRigGoal* FIKRigGoalContainer::FindGoalByName(const FName& GoalName)
{
	return Goals.FindByPredicate([GoalName](const FIKRigGoal& Other){return Other.Name == GoalName;});
}

const void FIKRigGoalContainer::FillWithGoalArray(const TArray<UIKRigEffectorGoal*>& InGoals)
{
	Empty();
	for (const UIKRigEffectorGoal* Goal : InGoals)
	{
		SetIKGoal(Goal);
	}
}

FIKRigGoal* FIKRigGoalContainer::FindGoalWriteable(const FName& GoalName) const
{
	return const_cast<FIKRigGoal*>(FindGoalByName(GoalName));
}

