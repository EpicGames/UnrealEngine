// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/IKRigDefinition.h"

#include "IKRigObjectVersion.h"
#include "Engine/SkeletalMesh.h"
#include "Rig/Solvers/IKRigSolverBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigDefinition)

#if WITH_EDITOR
#include "HAL/PlatformApplicationMisc.h"
#endif

#define LOCTEXT_NAMESPACE "IKRigDefinition"

#if WITH_EDITOR
const FName UIKRigDefinition::GetPreviewMeshPropertyName() { return GET_MEMBER_NAME_STRING_CHECKED(UIKRigDefinition, PreviewSkeletalMesh); };

TOptional<FTransform::FReal> UIKRigEffectorGoal::GetNumericValue(
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	EIKRigTransformType::Type TransformType) const
{
	switch(TransformType)
	{
		case EIKRigTransformType::Current:
		{
			return SAdvancedTransformInputBox<FTransform>::GetNumericValueFromTransform(
				CurrentTransform,
				Component,
				Representation,
				SubComponent);
		}
		case EIKRigTransformType::Reference:
		{
			return SAdvancedTransformInputBox<FTransform>::GetNumericValueFromTransform(
				InitialTransform,
				Component,
				Representation,
				SubComponent);
		}
		default:
		{
			break;
		}
	}
	return TOptional<FTransform::FReal>();
}

TTuple<FTransform, FTransform> UIKRigEffectorGoal::PrepareNumericValueChanged( ESlateTransformComponent::Type Component,
																			ESlateRotationRepresentation::Type Representation,
																			ESlateTransformSubComponent::Type SubComponent,
																			FTransform::FReal Value,
																			EIKRigTransformType::Type TransformType) const
{
	const FTransform& InTransform = TransformType == EIKRigTransformType::Current ? CurrentTransform : InitialTransform;
	FTransform OutTransform = InTransform;
	SAdvancedTransformInputBox<FTransform>::ApplyNumericValueChange(OutTransform, Value, Component, Representation, SubComponent);
	return MakeTuple(InTransform, OutTransform);
}

void UIKRigEffectorGoal::SetTransform(const FTransform& InTransform, EIKRigTransformType::Type InTransformType)
{
	// we assume that InTransform is not equal to the one it's being assigned to
	Modify();

	FTransform& TransformChanged = InTransformType == EIKRigTransformType::Current ? CurrentTransform : InitialTransform;
	TransformChanged = InTransform;
}

namespace
{
	
template<typename DataType>
void GetContentFromData(const DataType& InData, FString& Content)
{
	TBaseStructure<DataType>::Get()->ExportText(Content, &InData, &InData, nullptr, PPF_None, nullptr);
}

class FIKRigEffectorGoalErrorPipe : public FOutputDevice
{
public:
	int32 NumErrors;

	FIKRigEffectorGoalErrorPipe()
		: FOutputDevice()
		, NumErrors(0)
	{}

	virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
	{
		NumErrors++;
	}
};

template<typename DataType>
bool GetDataFromContent(const FString& Content, DataType& OutData)
{
	FIKRigEffectorGoalErrorPipe ErrorPipe;
	static UScriptStruct* DataStruct = TBaseStructure<DataType>::Get();
	DataStruct->ImportText(*Content, &OutData, nullptr, PPF_None, &ErrorPipe, DataStruct->GetName(), true);
	return (ErrorPipe.NumErrors == 0);
}
	
}

void UIKRigEffectorGoal::OnCopyToClipboard(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType) const
{
	const FTransform& Xfo = TransformType == EIKRigTransformType::Current ? CurrentTransform : InitialTransform;

	FString Content;
	switch(Component)
	{
	case ESlateTransformComponent::Location:
		{
			GetContentFromData(Xfo.GetLocation(), Content);
			break;
		}
	case ESlateTransformComponent::Rotation:
		{
			GetContentFromData(Xfo.Rotator(), Content);
			break;
		}
	case ESlateTransformComponent::Scale:
		{
			GetContentFromData(Xfo.GetScale3D(), Content);
			break;
		}
	case ESlateTransformComponent::Max:
	default:
		{
			GetContentFromData(Xfo, Content);
			break;
		}
	}

	if (!Content.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Content);
	}
}

void UIKRigEffectorGoal::OnPasteFromClipboard(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType)
{
	FString Content;
	FPlatformApplicationMisc::ClipboardPaste(Content);

	if (Content.IsEmpty())
	{
		return;
	}

	FTransform& Xfo = TransformType == EIKRigTransformType::Current ? CurrentTransform : InitialTransform;
	Modify();
	
	switch(Component)
	{
		case ESlateTransformComponent::Location:
		{
			FVector Data = Xfo.GetLocation();
			if (GetDataFromContent(Content, Data))
			{
				Xfo.SetLocation(Data);
			}
			break;
		}
		case ESlateTransformComponent::Rotation:
		{
			FRotator Data = Xfo.Rotator();
			if (GetDataFromContent(Content, Data))
			{
				Xfo.SetRotation(FQuat(Data));
			}
			break;
		}
		case ESlateTransformComponent::Scale:
		{
			FVector Data = Xfo.GetScale3D();
			if (GetDataFromContent(Content, Data))
			{
				Xfo.SetScale3D(Data);
			}
			break;
		}
		case ESlateTransformComponent::Max:
		default:
		{
			FTransform Data = Xfo;
			if (GetDataFromContent(Content, Data))
			{
				Xfo = Data;
			}
			break;
		}
	}
}

bool UIKRigEffectorGoal::TransformDiffersFromDefault(
	ESlateTransformComponent::Type Component,
	TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	if(PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UIKRigEffectorGoal, CurrentTransform))
	{
		switch(Component)
		{
			case ESlateTransformComponent::Location:
			{
				return !(CurrentTransform.GetLocation() - InitialTransform.GetLocation()).IsNearlyZero();
			}
			case ESlateTransformComponent::Rotation:
			{
				return !(CurrentTransform.Rotator() - InitialTransform.Rotator()).IsNearlyZero();
			}
			case ESlateTransformComponent::Scale:
			{
				return !(CurrentTransform.GetScale3D() - InitialTransform.GetScale3D()).IsNearlyZero();
			}
			default:
			{
				return false;
			}
		}
	}
	return false;
}

void UIKRigEffectorGoal::ResetTransformToDefault(
	ESlateTransformComponent::Type Component,
	TSharedPtr<IPropertyHandle> PropertyHandle)
{
	switch(Component)
	{
		case ESlateTransformComponent::Location:
		{
			CurrentTransform.SetLocation(InitialTransform.GetLocation());
			break;
		}
		case ESlateTransformComponent::Rotation:
		{
			CurrentTransform.SetRotation(InitialTransform.GetRotation());
			break;
		}
		case ESlateTransformComponent::Scale:
		{
			CurrentTransform.SetScale3D(InitialTransform.GetScale3D());
			break;
		}
		default:
		{
			break;
		}
	}
}

#endif

void FRetargetDefinition::AddBoneChain(
	const FName& ChainName,
	const FName& StartBone,
	const FName& EndBone,
	const FName& GoalName)
{
	if (FBoneChain* Chain = GetEditableBoneChainByName(ChainName))
	{
		Chain->StartBone = StartBone;
		Chain->EndBone = EndBone;
		Chain->IKGoalName = GoalName;
	}
	else
	{
		BoneChains.Emplace(ChainName, StartBone, EndBone, GoalName);
	}
}

FBoneChain* FRetargetDefinition::GetEditableBoneChainByName(FName ChainName)
{
	for (FBoneChain& Chain : BoneChains)
	{
		if (Chain.ChainName == ChainName)
		{
			return &Chain;
		}
	}
	
	return nullptr;
}

bool FRetargetDefinition::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);
	// doesn't actually serialize, just write the custom version for PostSerialize
	return false;
}

void FRetargetDefinition::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FIKRigObjectVersion::GUID) < FIKRigObjectVersion::AddedRootBoneToRetargetDefinition)
		{
			// we used to call "Pelvis" the "RetargetRoot"
			// but now we also want to characterize the actual skeleton "root" bone itself
			PelvisBone = RootBone;
		}
	}
}

void UIKRigDefinition::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
#if WITH_EDITORONLY_DATA
	Controller = nullptr;
#endif
}

void UIKRigDefinition::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);
}

void UIKRigDefinition::PostLoad()
{
	Super::PostLoad();
	
	// very early versions of the asset may not have been set as standalone
	SetFlags(RF_Standalone);

	// validate the input FInstancedStruct is a valid subclass of FIKRigSolverVase
	auto DerivesFromBaseSolverType = [](const FInstancedStruct& InConvertedSolver)
	{
		if (!InConvertedSolver.IsValid())
		{
			return false;
		}
		
		bool bIsDerivedFromBase = false;
		UStruct* CurrentSuperStruct = InConvertedSolver.GetScriptStruct()->GetSuperStruct();
		while (CurrentSuperStruct)
		{
			if (CurrentSuperStruct == FIKRigSolverBase::StaticStruct())
			{
				bIsDerivedFromBase = true;
				break;
			}
			CurrentSuperStruct = CurrentSuperStruct->GetSuperStruct();
		}
		
		return bIsDerivedFromBase;
	};
	
	// convert old UObject-based solver stack to new struct-based types
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (TObjectPtr<UIKRigSolver> Solver : Solvers_DEPRECATED)
	{
		if (Solver == nullptr)
		{
			continue;
		}
		
		FInstancedStruct ConvertedSolver;
		Solver->ConvertToInstancedStruct(ConvertedSolver);
		if (DerivesFromBaseSolverType(ConvertedSolver))
		{
			SolverStack.Add(ConvertedSolver);
		}
		else
		{
			UE_LOGF(LogTemp, Warning, "IK Rig: unable to load old UObject based solver type. Conversion failed for type:  %ls", *Solver->GetName());
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	// clean out null solvers
	SolverStack.RemoveAll([](const FInstancedStruct& InSolver)
	{
		return !InSolver.IsValid();
	});

	// clean out null goals
	Goals.RemoveAll([](const TObjectPtr<UIKRigEffectorGoal>& InGoal)
	{
		return !InGoal;
	});
}

const TArray<FInstancedStruct>& UIKRigDefinition::GetSolverStructs() const
{
	return SolverStack;
}

const FBoneChain* UIKRigDefinition::GetRetargetChainByName(FName ChainName) const
{
	for (const FBoneChain& Chain : RetargetDefinition.BoneChains)
	{
		if (Chain.ChainName == ChainName)
		{
			return &Chain;
		}
	}
	
	return nullptr;
}

TArray<FName> UIKRigDefinition::GetRetargetChainNames() const
{
	TArray<FName> OutChainNames;
	for (const FBoneChain& Chain : RetargetDefinition.BoneChains)
	{
		OutChainNames.Add(Chain.ChainName);
	}
	return MoveTemp(OutChainNames);
}

/** IInterface_PreviewMeshProvider interface */
void UIKRigDefinition::SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty)
{
	PreviewSkeletalMesh = PreviewMesh;
}

USkeletalMesh* UIKRigDefinition::GetPreviewMesh() const
{
	return PreviewSkeletalMesh.LoadSynchronous();
}
/** END IInterface_PreviewMeshProvider interface */

#undef LOCTEXT_NAMESPACE

