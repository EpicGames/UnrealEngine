// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/IKRigProcessor.h"
#include "Rig/IKRigDefinition.h"
#include "Rig/Solvers/IKRigSolverBase.h"

#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigProcessor)

#define LOCTEXT_NAMESPACE "IKRigProcessor"


void FIKRigProcessor::Initialize(
	const UIKRigDefinition* InRigAsset,
	const USkeletalMesh* InSkeletalMesh,
	const FIKRigGoalContainer& InOptionalGoals)
{
	bInitialized = false;
	
	// can't initialize without a rig definition
	if (!InRigAsset)
	{
		return;
	}

	// bail out if we've already tried initializing with this exact version of the rig asset
	if (bTriedToInitialize)
	{
		return; // don't keep spamming
	}

	// ok, lets try to initialize
	bTriedToInitialize = true;

	// copy skeleton data from the actual skeleton we want to run on
	Skeleton.SetInputSkeleton(InSkeletalMesh, InRigAsset->GetSkeleton().ExcludedBones);
	
	if (InRigAsset->GetSkeleton().BoneNames.IsEmpty())
	{
		Log.LogError(FText::Format(
			LOCTEXT("NoSkeleton", "Trying to initialize IK Rig, '{0}' that has no skeleton."),
			FText::FromString(InRigAsset->GetName())));
		return;
	}

	if (!FIKRigProcessor::IsIKRigCompatibleWithSkeleton(InRigAsset, InSkeletalMesh, &Log))
	{
		Log.LogError(FText::Format(
			LOCTEXT("SkeletonMissingRequiredBones", "Trying to initialize IKRig, '{0}' with a Skeleton that is missing required bones. See prior warnings."),
			FText::FromString(InRigAsset->GetName())));
		return;
	}
	
	// initialize goal container
	GoalContainer.Empty();
	if (InOptionalGoals.IsEmpty())
	{
		// use goals from IK Rig
		const TArray<UIKRigEffectorGoal*>& EffectorGoals = InRigAsset->GetGoalArray();
		GoalContainer.FillWithGoalArray(EffectorGoals);
	}
	else
	{
		// use the optional goals passed in
		GoalContainer = InOptionalGoals;
	}

	// get the set of ALL required goals so we can validate that each goal is used by at least 1 solver
	const TArray<FInstancedStruct>& AssetSolverStructs = InRigAsset->GetSolverStructs();
	TSet<FName> AllRequiredGoals;
	for (const FInstancedStruct& AssetSolverStruct : AssetSolverStructs)
	{
		const FIKRigSolverBase& AssetSolver = AssetSolverStruct.Get<FIKRigSolverBase>();
		AssetSolver.GetRequiredGoals(AllRequiredGoals);
	}

	// generate list of excluded goals
	// (these are goals that were not in the supplied GoalContainer but are used by at least 1 solver)
	TSet<FName> ExcludedGoals;
	for (FName RequiredGoalName : AllRequiredGoals)
	{
		const FIKRigGoal* RequiredGoal = GoalContainer.FindGoalByName(RequiredGoalName);
		if (!RequiredGoal)
		{
			ExcludedGoals.Add(RequiredGoalName);
		}
	}
	// exclude disabled goals
	if (!InOptionalGoals.IsEmpty())
	{
		for (const FIKRigGoal& InputGoal : InOptionalGoals.GetGoalArray())
		{
			if (!InputGoal.bEnabled)
			{
				ExcludedGoals.Add(InputGoal.Name);
			}
		}
	}
	
	// initialize goal bones from asset
	GoalBones.Reset();
	for (const FIKRigGoal& Goal : GoalContainer.GetGoalArray())
	{
		// default to using the bone name supplied by the goal
		FName BoneNameToUse = Goal.BoneName;
		
		// if the goal itself didn't come with a bone name, try to find it from the asset
		if (BoneNameToUse == NAME_None)
		{
			const TArray<UIKRigEffectorGoal*>& EffectorGoals = InRigAsset->GetGoalArray();
			for (const UIKRigEffectorGoal* EffectorGoal : EffectorGoals)
			{
				if (EffectorGoal->GoalName == Goal.Name)
				{
					BoneNameToUse = EffectorGoal->BoneName;
					break;
				}
			}
		}
		
		FGoalBone NewGoalBone;
		NewGoalBone.BoneName = BoneNameToUse;
		NewGoalBone.BoneIndex = Skeleton.GetBoneIndexFromName(Goal.BoneName);

		// validate that the skeleton we are trying to solve this goal on contains the bone the goal expects
		if (NewGoalBone.BoneIndex == INDEX_NONE)
		{
			Log.LogError(FText::Format(
				LOCTEXT("MissingGoalBone", "IK Rig, '{0}' has a Goal, '{1}' that references an unknown bone, '{2}'. Cannot evaluate."),
				FText::FromString(InRigAsset->GetName()), FText::FromName(Goal.Name), FText::FromName(Goal.BoneName) ));
			return;
		}

		// validate that there is not already a different goal, with the same name, that is using a different bone
		// (all goals with the same name must reference the same bone within a single IK Rig)
		if (const FGoalBone* Bone = GoalBones.Find(Goal.Name))
		{
			if (Bone->BoneName != NewGoalBone.BoneName)
			{
				Log.LogError(FText::Format(
				LOCTEXT("DuplicateGoal", "IK Rig, '{0}' has a Goal, '{1}' that references different bones in different solvers, '{2}' and '{3}'. Cannot evaluate."),
                FText::FromString(InRigAsset->GetName()),
                FText::FromName(Goal.Name),
                FText::FromName(Bone->BoneName), 
                FText::FromName(NewGoalBone.BoneName)));
				return;
			}
		}

		// warn if goal is not connected to any solvers
		if (!AllRequiredGoals.Contains(Goal.Name))
		{
			Log.LogWarning(FText::Format(
				LOCTEXT("DisconnectedGoal", "IK Rig, '{0}' has a Goal, '{1}' that is not connected to any solvers. It will have no effect."),
				FText::FromString(InRigAsset->GetName()), FText::FromName(Goal.Name)));
		}
		
		GoalBones.Add(Goal.Name, NewGoalBone);
	}

	// create copies of all the solvers in the IK rig
	Solvers.Reset(AssetSolverStructs.Num());
	for (const FInstancedStruct& AssetSolverStruct : AssetSolverStructs)
	{
		if (!AssetSolverStruct.IsValid())
		{
			// this can happen if asset references deleted IK Solver type which should only happen during development (if at all)
			Log.LogWarning(FText::Format(
				LOCTEXT("UnknownSolver", "IK Rig, '{0}' has null/unknown solver in it. Please remove it."),
				FText::FromString(InRigAsset->GetName())));
			continue;
		}

		// copy the solver memory into the processor
		const int32 NewSolverIndex = Solvers.Emplace(AssetSolverStruct);
		FIKRigSolverBase* NewSolver = Solvers[NewSolverIndex].GetMutablePtr<FIKRigSolverBase>();

		// remove excluded goals from the solver
		// this must be done BEFORE initializing the solver
		for (const FName ExcludedGoalName : ExcludedGoals)
		{
			NewSolver->OnGoalRemoved(ExcludedGoalName);
		}

		// initialize it and store in the processor
		NewSolver->Initialize(Skeleton);
	}

	// validate retarget chains
	const TArray<FBoneChain>& Chains = InRigAsset->GetRetargetChains();
	TArray<int32> OutBoneIndices;
	for (const FBoneChain& Chain : Chains)
	{
		if (!Skeleton.ValidateChainAndGetBones(Chain, OutBoneIndices))
		{
			Log.LogWarning(FText::Format(
				LOCTEXT("InvalidRetargetChain", "Invalid Retarget Chain: '{0}'. End bone is not a child of the start bone in Skeletal Mesh, '{1}'."),
				FText::FromString(Chain.ChainName.ToString()), FText::FromString(InSkeletalMesh->GetName())));
		}
	}

	Log.LogInfo(FText::Format(
				LOCTEXT("SuccessfulInit", "IK Rig, '{0}' ready to run on {1}."),
				FText::FromString(InRigAsset->GetName()), FText::FromString(InSkeletalMesh->GetName())));

	AssetInitializedWith = InRigAsset;
	GoalContainer.bRigNeedsInitialized = false;
	bTriedToInitialize = false;
	bInitialized = true;
}

bool FIKRigProcessor::IsIKRigCompatibleWithSkeleton(
	const UIKRigDefinition* InRigAsset,
	const FIKRigInputSkeleton& InputSkeleton,
	const FIKRigLogger* Log)
{
	// first we validate that all the required bones are in the input skeleton...
	
	TSet<FName> RequiredBones;
	const TArray<FInstancedStruct>& AssetSolverStructs = InRigAsset->GetSolverStructs();
	for (const FInstancedStruct& AssetSolverStruct : AssetSolverStructs)
	{
		const FIKRigSolverBase& AssetSolver = AssetSolverStruct.Get<FIKRigSolverBase>();
		AssetSolver.GetRequiredBones(RequiredBones);
	}

	const TArray<UIKRigEffectorGoal*>& Goals = InRigAsset->GetGoalArray();
	for (const UIKRigEffectorGoal* Goal : Goals)
	{
		RequiredBones.Add(Goal->BoneName);
	}

	// strip None out (in case of lost bones in solver)
	RequiredBones.Remove(NAME_None);

	bool bAllRequiredBonesFound = true;
	for (const FName& RequiredBone : RequiredBones)
	{
		if (!InputSkeleton.BoneNames.Contains(RequiredBone))
		{
			if (Log)
			{
				Log->LogError(FText::Format(
			LOCTEXT("MissingBone", "IK Rig, '{0}' is missing a required bone, '{1}' in the Skeletal Mesh."),
				FText::FromString(InRigAsset->GetName()),
				FText::FromName(RequiredBone)));
			}
			
			bAllRequiredBonesFound = false;
		}
	}

	if (!bAllRequiredBonesFound)
	{
		return false;
	}

	// now we validate that hierarchy matches for all required bones...
	bool bAllParentsValid = true;
	const FIKRigSkeleton& AssetSkeleton = InRigAsset->GetSkeleton();
	for (const FName& RequiredBone : RequiredBones)
	{
		const int32 InputBoneIndex = InputSkeleton.BoneNames.Find(RequiredBone);
		const int32 AssetBoneIndex = AssetSkeleton.BoneNames.Find(RequiredBone);

		// we shouldn't get this far otherwise due to early return above...
		check(InputBoneIndex != INDEX_NONE && AssetBoneIndex != INDEX_NONE)

		// validate that input skeleton hierarchy is as expected
		const int32 AssetParentIndex = AssetSkeleton.ParentIndices[AssetBoneIndex];
		if (AssetSkeleton.BoneNames.IsValidIndex(AssetParentIndex)) // root bone has no parent
		{
			const FName& AssetParentName = AssetSkeleton.BoneNames[AssetParentIndex];
			const int32 InputParentIndex = InputSkeleton.ParentIndices[InputBoneIndex];
			if (!InputSkeleton.BoneNames.IsValidIndex(InputParentIndex))
			{
				bAllParentsValid = false;

				if (Log)
				{
					Log->LogError(FText::Format(
					LOCTEXT("InvalidParent", "IK Rig is running on a skeleton with a required bone, '{0}', that expected to have a valid parent. The expected parent was, '{1}'."),
					FText::FromName(RequiredBone),
					FText::FromName(AssetParentName)));
				}
				
				continue;
			}
			
			const FName& InputParentName = InputSkeleton.BoneNames[InputParentIndex];
			if (AssetParentName != InputParentName)
			{
				if (Log)
				{
					// we only warn about this, because it may be nice not to have the exact same hierarchy
					Log->LogWarning(FText::Format(
					LOCTEXT("UnexpectedParent", "IK Rig is running on a skeleton with a required bone, '{0}', that has a different parent '{1}'. The expected parent was, '{2}'."),
					FText::FromName(RequiredBone),
					FText::FromName(InputParentName),
					FText::FromName(AssetParentName)));
				}
				
				continue;
			}
		}
	}

	return bAllParentsValid;
}

void FIKRigProcessor::SetInputPoseGlobal(const TArray<FTransform>& InGlobalBoneTransforms) 
{
	if (!ensure(bInitialized))
	{
		return;
	}
	check(InGlobalBoneTransforms.Num() == Skeleton.CurrentPoseGlobal.Num());
	Skeleton.CurrentPoseGlobal = InGlobalBoneTransforms;
}

void FIKRigProcessor::SetInputBoneGlobal(const int32& InBoneIndex, const FTransform& InGlobalBoneTransform)
{
	if (!ensure(Skeleton.CurrentPoseGlobal.IsValidIndex(InBoneIndex)))
	{
		return;
	}
	Skeleton.CurrentPoseGlobal[InBoneIndex] = InGlobalBoneTransform;
}

void FIKRigProcessor::SetInputPoseToRefPose()
{
	if (!ensure(bInitialized))
	{
		return;
	}
	
	Skeleton.CurrentPoseGlobal = Skeleton.RefPoseGlobal;
}

void FIKRigProcessor::ApplyGoalsFromOtherContainer(const FIKRigGoalContainer& InGoalContainer)
{
	for (const FIKRigGoal& InGoal : InGoalContainer.GetGoalArray())
	{
		SetIKGoal(InGoal);
	}
}

void FIKRigProcessor::SetIKGoal(const FIKRigGoal& InGoal)
{
	if (!ensure(bInitialized))
	{
		return;
	}

	GoalContainer.SetIKGoal(InGoal);
}

void FIKRigProcessor::SetIKGoal(const UIKRigEffectorGoal* InGoal)
{
	if (!ensure(bInitialized))
	{
		return;
	}
	
	GoalContainer.SetIKGoal(InGoal);
}

void FIKRigProcessor::Solve(const FTransform& ComponentToWorld)
{
	// trigger reinitialization if the goal container was modified such that it needs it
	if (GoalContainer.NeedsInitialized())
	{
		// save input pose
		TArray<FTransform> InputPoseGlobal = Skeleton.CurrentPoseGlobal;
		// make a copy of the goal container in it's current state to reinitialize with
		FIKRigGoalContainer CopiedGoalContainer = GoalContainer;
		// reinitialize
		Initialize(AssetInitializedWith, Skeleton.SkeletalMesh, CopiedGoalContainer);
		// restore input pose after reinit
		if (IsInitialized())
		{
			SetInputPoseGlobal(InputPoseGlobal);
		}
	}

	if (!bInitialized)
	{
		return;
	}

	// generate local space pose
	Skeleton.UpdateAllLocalTransformFromGlobal();
	
	// convert goals into component space and blend towards input pose by alpha
	ResolveFinalGoalTransforms(ComponentToWorld);

	// run all the solvers
	for (FInstancedStruct& SolverStruct : Solvers)
	{
		FIKRigSolverBase* Solver = SolverStruct.GetMutablePtr<FIKRigSolverBase>();
		if (Solver->IsEnabled())
		{
			Solver->Solve(Skeleton, GoalContainer);
		}
	}
}

void FIKRigProcessor::GetOutputPoseGlobal(TArray<FTransform>& OutputPoseGlobal) const
{
	// TODO, remove this copy
	OutputPoseGlobal = Skeleton.CurrentPoseGlobal;
}

void FIKRigProcessor::Reset()
{
	Solvers.Reset();
	GoalContainer.Empty();
	GoalBones.Reset();
	Skeleton.Reset();
	SetNeedsInitialized();
}

void FIKRigProcessor::SetNeedsInitialized()
{
	bInitialized = false;
	bTriedToInitialize = false;
}

const FIKRigSolverBase* FIKRigProcessor::GetSolver(const int32 InSolverIndex) const
{
	if (Solvers.IsValidIndex(InSolverIndex))
	{
		return Solvers[InSolverIndex].GetPtr<FIKRigSolverBase>();
	}
	
	return nullptr;
}

void FIKRigProcessor::CopyAllSettingsFromAsset(const UIKRigDefinition* SourceAsset)
{
	if (!IsValid(SourceAsset))
	{
		return;
	}
	
	if (!bInitialized)
	{
		return;
	}
	
	// copy goal settings
	const TArray<UIKRigEffectorGoal*>& AssetGoals =  SourceAsset->GetGoalArray();
	for (const UIKRigEffectorGoal* AssetGoal : AssetGoals)
	{
		SetIKGoal(AssetGoal);
	}

	// copy solver settings
	const TArray<FInstancedStruct>& AssetSolverStructs = SourceAsset->GetSolverStructs();
	check(Solvers.Num() == AssetSolverStructs.Num()); // if number of solvers has been changed, processor should have been reinitialized
	for (int32 SolverIndex=0; SolverIndex<Solvers.Num(); ++SolverIndex)
	{
		const FIKRigSolverBase& AssetSolver = AssetSolverStructs[SolverIndex].Get<FIKRigSolverBase>();
		FIKRigSolverBase& RunningSolver = Solvers[SolverIndex].GetMutable<FIKRigSolverBase>();
		RunningSolver.UpdateSettingsFromAsset(AssetSolver);
	}
}

const FIKRigGoalContainer& FIKRigProcessor::GetGoalContainer() const
{
	ensure(bInitialized);
	return GoalContainer;
}

FIKRigGoalContainer& FIKRigProcessor::GetGoalContainer()
{
	ensure(bInitialized);
	return GoalContainer;
}

const FGoalBone* FIKRigProcessor::GetGoalBone(const FName& GoalName) const
{
	return GoalBones.Find(GoalName);
}

FIKRigSkeleton& FIKRigProcessor::GetSkeletonWriteable()
{
	return Skeleton;
}

const FIKRigSkeleton& FIKRigProcessor::GetSkeleton() const
{
	return Skeleton;
}

void FIKRigProcessor::ResolveFinalGoalTransforms(const FTransform& WorldToComponent)
{
	for (FIKRigGoal& Goal : GoalContainer.Goals)
	{
		if (!GoalBones.Contains(Goal.Name))
		{
			// user is changing goals after initialization
			// not necessarily a bad thing, but new goal names won't work until re-init
			continue;
		}

		const FGoalBone& GoalBone = GoalBones[Goal.Name];
		const FTransform& InputPoseBoneTransform = Skeleton.CurrentPoseGlobal[GoalBone.BoneIndex];

		FVector ComponentSpaceGoalPosition = Goal.Position;
		FQuat ComponentSpaceGoalRotation = Goal.Rotation.Quaternion();

		// FIXME find a way to cache SourceBoneIndex to avoid calling Skeleton.GetBoneIndexFromName here
		// we may use FGoalBone::OptSourceIndex for this
		int SourceBoneIndex = INDEX_NONE;
		if (Goal.TransformSource == EIKRigGoalTransformSource::Bone && Goal.SourceBone.BoneName != NAME_None)
		{
			SourceBoneIndex = Skeleton.GetBoneIndexFromName(Goal.SourceBone.BoneName);
		}

		if (SourceBoneIndex != INDEX_NONE)
		{
			ComponentSpaceGoalPosition = Skeleton.CurrentPoseGlobal[SourceBoneIndex].GetLocation();
		}
		else
		{
			// put goal POSITION in Component Space
			switch (Goal.PositionSpace)
			{
			case EIKRigGoalSpace::Additive:
				// add position offset to bone position
				ComponentSpaceGoalPosition = Skeleton.CurrentPoseGlobal[GoalBone.BoneIndex].GetLocation() + Goal.Position;
				break;
			case EIKRigGoalSpace::Component:
				// was already supplied in Component Space
				break;
			case EIKRigGoalSpace::World:
				// convert from World Space to Component Space
				ComponentSpaceGoalPosition = WorldToComponent.TransformPosition(Goal.Position);
				break;
			default:
				checkNoEntry();
				break;
			}
		}
		
		// put goal ROTATION in Component Space
		if (SourceBoneIndex != INDEX_NONE)
		{
			ComponentSpaceGoalRotation = Skeleton.CurrentPoseGlobal[SourceBoneIndex].GetRotation();
		}
		else
		{
			switch (Goal.RotationSpace)
			{
			case EIKRigGoalSpace::Additive:
				// add rotation offset to bone rotation
				ComponentSpaceGoalRotation = Goal.Rotation.Quaternion() * Skeleton.CurrentPoseGlobal[GoalBone.BoneIndex].GetRotation();
				break;
			case EIKRigGoalSpace::Component:
				// was already supplied in Component Space
				break;
			case EIKRigGoalSpace::World:
				// convert from World Space to Component Space
				ComponentSpaceGoalRotation = WorldToComponent.TransformRotation(Goal.Rotation.Quaternion());
				break;
			default:
				checkNoEntry();
				break;
			}
		}

		// blend by alpha from the input pose, to the supplied goal transform
		// when Alpha is 0, the goal transform matches the bone transform at the input pose.
		// when Alpha is 1, the goal transform is left fully intact
		Goal.FinalBlendedPosition = FMath::Lerp(
            InputPoseBoneTransform.GetTranslation(),
            ComponentSpaceGoalPosition,
            Goal.PositionAlpha);
		
		Goal.FinalBlendedRotation = FQuat::Slerp(
            InputPoseBoneTransform.GetRotation(),
            ComponentSpaceGoalRotation,
            Goal.RotationAlpha);
	}
}

#undef LOCTEXT_NAMESPACE

