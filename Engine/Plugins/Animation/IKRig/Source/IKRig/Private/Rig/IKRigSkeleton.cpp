// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/IKRigSkeleton.h"

#include "Rig/IKRigDefinition.h"
#include "Algo/LevenshteinDistance.h"
#include "Engine/SkeletalMesh.h"
#include "Math/Bounds.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigSkeleton)

// This is the default end of branch index value, meaning we haven't cached it yet
#define FIKRIGSKELETON_INVALID_EO_BRANCH_INDEX -2

FIKRigInputSkeleton::FIKRigInputSkeleton(const USkeletalMesh* SkeletalMesh)
{
	Initialize(SkeletalMesh);
}

void FIKRigInputSkeleton::Initialize(const USkeletalMesh* InSkeletalMesh)
{
	Reset();

	if (!InSkeletalMesh)
	{
		return;
	}
	
	SkeletalMesh = InSkeletalMesh;
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const TArray<FMeshBoneInfo>& BoneInfo = RefSkeleton.GetRefBoneInfo();
	for (int32 BoneIndex=0; BoneIndex<BoneInfo.Num(); ++BoneIndex)
	{
		BoneNames.Add(BoneInfo[BoneIndex].Name);
		ParentIndices.Add(BoneInfo[BoneIndex].ParentIndex);
		LocalRefPose.Add(RefSkeleton.GetRefBonePose()[BoneIndex]);
	}
}

void FIKRigInputSkeleton::Reset()
{
	*this = FIKRigInputSkeleton();
}

void FIKRigSkeleton::SetInputSkeleton(const USkeletalMesh* InSkeletalMesh, const TArray<FName>& InExcludedBones)
{
	const FIKRigInputSkeleton InputSkeleton = FIKRigInputSkeleton(InSkeletalMesh);
	SetInputSkeleton(InputSkeleton, InExcludedBones);
}

void FIKRigSkeleton::SetInputSkeleton(const FIKRigInputSkeleton& InputSkeleton, const TArray<FName>& InExcludedBones)
{
	check(InputSkeleton.BoneNames.Num() == InputSkeleton.ParentIndices.Num() &&
		InputSkeleton.BoneNames.Num() == InputSkeleton.LocalRefPose.Num())
	
	// reset all containers
	Reset();

	// store skeletal mesh for reinit
	SkeletalMesh = InputSkeleton.SkeletalMesh;
	
	// we use the bone names and parent indices from the input skeleton
	BoneNames = InputSkeleton.BoneNames;
	ParentIndices = InputSkeleton.ParentIndices;
	
	// bones are excluded at the skeleton level, instead of per-solver
	ExcludedBones = InExcludedBones;
	
	// get a compacted local ref pose based on the stored names
	TArray<FTransform> CompactRefPoseLocal;
	// copy a compacted reference pose using only the bones in the input skeleton
	for (int32 BoneIndex=0; BoneIndex<InputSkeleton.BoneNames.Num(); ++BoneIndex)
	{
		// store ref pose of this bone
		CompactRefPoseLocal.Add(InputSkeleton.LocalRefPose[BoneIndex]);
	}
	
	// copy local ref pose to global
	ConvertLocalPoseToGlobal(ParentIndices, CompactRefPoseLocal, RefPoseGlobal);

	// initialize CURRENT GLOBAL pose to reference pose
	CurrentPoseGlobal = RefPoseGlobal;

	// initialize CURRENT LOCAL pose from global pose
	UpdateAllLocalTransformFromGlobal();

	// initialize branch caching
	CachedEndOfBranchIndices.Init(FIKRIGSKELETON_INVALID_EO_BRANCH_INDEX, ParentIndices.Num());
}

void FIKRigSkeleton::Reset()
{
	*this = FIKRigSkeleton();
}

int32 FIKRigSkeleton::GetBoneIndexFromName(const FName InName) const
{
	for (int32 Index=0; Index<BoneNames.Num(); ++Index)
	{
		if (InName == BoneNames[Index])
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

const FName& FIKRigSkeleton::GetBoneNameFromIndex(const int32 BoneIndex) const
{
	if (BoneNames.IsValidIndex(BoneIndex))
	{
		return BoneNames[BoneIndex];
	}

	static const FName InvalidName(NAME_None);  
	return InvalidName;
}

int32 FIKRigSkeleton::GetParentIndex(const int32 BoneIndex) const
{
	if (!BoneNames.IsValidIndex(BoneIndex))
	{
		return INDEX_NONE;
	}

	return ParentIndices[BoneIndex];
}

int32 FIKRigSkeleton::GetParentIndexThatIsNotExcluded(const int32 BoneIndex) const
{
	// find first parent that is NOT excluded
	int32 ParentIndex = GetParentIndex(BoneIndex);
	while(ParentIndex != INDEX_NONE)
	{
		if (!IsBoneExcluded(ParentIndex))
		{
			break;
		}
		ParentIndex = GetParentIndex(ParentIndex);
	}
	
	return ParentIndex;
}

int32 FIKRigSkeleton::GetChildIndices(const int32 ParentBoneIndex, TArray<int32>& Children) const
{
	Children.Reset();

	const int32 LastBranchIndex = GetCachedEndOfBranchIndex(ParentBoneIndex);
	if (LastBranchIndex != INDEX_NONE)
	{
		for (int32 BoneIndex = ParentBoneIndex + 1; BoneIndex <= LastBranchIndex; BoneIndex++)
		{
			if (GetParentIndex(BoneIndex) == ParentBoneIndex)
			{
				Children.Add(BoneIndex);
			}
		}
	}

	return Children.Num();
}

void FIKRigSkeleton::GetChildrenIndicesRecursive(const int32 BoneIndex, TArray<int32>& OutChildren) const
{
	const int32 LastBranchIndex = GetCachedEndOfBranchIndex(BoneIndex);
	if (LastBranchIndex == INDEX_NONE)
	{
		// no children (leaf bone)
		return;
	}
	
	for (int32 ChildBoneIndex = BoneIndex + 1; ChildBoneIndex <= LastBranchIndex; ChildBoneIndex++)
	{
		OutChildren.Add(ChildBoneIndex);
	}
}

int32 FIKRigSkeleton::GetCachedEndOfBranchIndex(const int32 InBoneIndex) const
{
	if (!CachedEndOfBranchIndices.IsValidIndex(InBoneIndex))
	{
		return INDEX_NONE;
	}

	// already cached
	if (CachedEndOfBranchIndices[InBoneIndex] != FIKRIGSKELETON_INVALID_EO_BRANCH_INDEX)
	{
		return CachedEndOfBranchIndices[InBoneIndex];
	}

	const int32 NumBones = BoneNames.Num();
	
	// if we're asking for the first or last bone, return the last bone  
	if (InBoneIndex == 0 || InBoneIndex + 1 >= NumBones)
	{
		CachedEndOfBranchIndices[InBoneIndex] = NumBones-1;
		return CachedEndOfBranchIndices[InBoneIndex];
	}

	CachedEndOfBranchIndices[InBoneIndex] = INDEX_NONE;
	
	// store start parent
	const int32 StartParentIndex = GetParentIndex(InBoneIndex);

	int32 BoneIndex = InBoneIndex + 1;
	int32 ParentIndex = GetParentIndex(BoneIndex);

	// if next child bone's parent is less than or equal to StartParentIndex,
	// we are leaving the branch so no need to go further
	int32 BoneIndexAtEndOfBranch = FIKRIGSKELETON_INVALID_EO_BRANCH_INDEX;
	while (ParentIndex > StartParentIndex)
	{
		BoneIndexAtEndOfBranch = BoneIndex;
		BoneIndex++;
		if (BoneIndex >= NumBones)
		{
			break;
		}
		ParentIndex = GetParentIndex(BoneIndex);
	}

	// set once (outside of while loop above) to avoid potential race condition
	CachedEndOfBranchIndices[InBoneIndex] = BoneIndexAtEndOfBranch;

	return CachedEndOfBranchIndices[InBoneIndex];
}

void FIKRigSkeleton::ConvertLocalPoseToGlobal(
	const TArray<int32>& InParentIndices,
	const TArray<FTransform>& InLocalPose,
	TArray<FTransform>& OutGlobalPose)
{
	check(InLocalPose.Num() == InParentIndices.Num());
	
	OutGlobalPose.Reset();
	for (int32 BoneIndex=0; BoneIndex<InLocalPose.Num(); ++BoneIndex)
	{
		const int32 ParentIndex = InParentIndices[BoneIndex];
		if (ParentIndex == INDEX_NONE)
		{
			OutGlobalPose.Add(InLocalPose[BoneIndex]);
			continue; // root bone is always in local space 
		}

		const FTransform& ChildLocalTransform  = InLocalPose[BoneIndex];
		const FTransform& ParentGlobalTransform  = OutGlobalPose[ParentIndex];
		OutGlobalPose.Add(ChildLocalTransform * ParentGlobalTransform);
	}
}

void FIKRigSkeleton::UpdateAllGlobalTransformFromLocal()
{
	// generate GLOBAL transforms based on LOCAL transforms
	CurrentPoseGlobal = CurrentPoseLocal;
	for (int32 BoneIndex=0; BoneIndex<CurrentPoseLocal.Num(); ++BoneIndex)
	{
		UpdateGlobalTransformFromLocal(BoneIndex);
	}
}

void FIKRigSkeleton::UpdateGlobalTransformFromLocal(const int32 BoneIndex)
{
	const int32 ParentIndex = ParentIndices[BoneIndex];
	if (ParentIndex == INDEX_NONE)
	{
		CurrentPoseGlobal[BoneIndex] = CurrentPoseLocal[BoneIndex]; // root bone is always in local space
		return; 
	}

	const FTransform& ChildLocalTransform  = CurrentPoseLocal[BoneIndex];
	const FTransform& ParentGlobalTransform  = CurrentPoseGlobal[ParentIndex];
	CurrentPoseGlobal[BoneIndex] = ChildLocalTransform * ParentGlobalTransform;
}

void FIKRigSkeleton::UpdateAllLocalTransformFromGlobal()
{
	// generate LOCAL transforms based on GLOBAL transforms
	CurrentPoseLocal = CurrentPoseGlobal;
	for (int32 BoneIndex=0; BoneIndex<CurrentPoseGlobal.Num(); ++BoneIndex)
	{
		UpdateLocalTransformFromGlobal(BoneIndex);
	}
}

void FIKRigSkeleton::UpdateLocalTransformFromGlobal(const int32 BoneIndex)
{
	const int32 ParentIndex = ParentIndices[BoneIndex];
	if (ParentIndex == INDEX_NONE)
	{
		CurrentPoseLocal[BoneIndex] = CurrentPoseGlobal[BoneIndex]; // root bone is always in local space
		return; 
	}

	const FTransform& ChildGlobal  = CurrentPoseGlobal[BoneIndex];
	const FTransform& ParentGlobal  = CurrentPoseGlobal[ParentIndex];
	CurrentPoseLocal[BoneIndex] = ChildGlobal.GetRelativeTransform(ParentGlobal);
	CurrentPoseLocal[BoneIndex].NormalizeRotation();
}

void FIKRigSkeleton::PropagateGlobalPoseBelowBone(const int32 StartBoneIndex)
{
	const int32 LastBranchIndex = GetCachedEndOfBranchIndex(StartBoneIndex);
	if (LastBranchIndex != INDEX_NONE)
	{
		for (int32 BoneIndex=StartBoneIndex+1; BoneIndex<=LastBranchIndex; ++BoneIndex)
		{
			UpdateGlobalTransformFromLocal(BoneIndex);
			UpdateLocalTransformFromGlobal(BoneIndex);
		}
	}
}

bool FIKRigSkeleton::IsBoneInDirectLineage(const FName& Child, const FName& PotentialParent) const
{
	const int32 ChildIndex = GetBoneIndexFromName(Child);
	if (ChildIndex == INDEX_NONE)
	{
		return false;
	}

	const int32 PotentialParentIndex = GetBoneIndexFromName(PotentialParent);
	if (PotentialParentIndex == INDEX_NONE)
	{
		return false;
	}

	int32 NextParentIndex = ChildIndex;
	while(NextParentIndex != INDEX_NONE)
	{
		if (NextParentIndex == PotentialParentIndex)
		{
			return true;
		}
		NextParentIndex = ParentIndices[NextParentIndex];
	}

	return false;
}

bool FIKRigSkeleton::IsBoneExcluded(const int32 BoneIndex) const
{
	if (ensure(BoneNames.IsValidIndex(BoneIndex)))
	{
		return ExcludedBones.Contains(BoneNames[BoneIndex]);	
	}
	
	return false;
}

void FIKRigSkeleton::NormalizeRotations(TArray<FTransform>& Transforms)
{
	for (FTransform& Transform : Transforms)
	{
		Transform.NormalizeRotation();
	}
}

void FIKRigSkeleton::GetChainsInList(const TArray<int32>& SelectedBones, TArray<FBoneChain>& OutChains) const
{	
	// no bones provided, return empty list
	if (SelectedBones.IsEmpty())
	{
		return;
	}

	// get parents of all the selected bones
	TSet<int32> SelectedParentIndices;
	for (const int32 SelectedBone : SelectedBones)
	{
		SelectedParentIndices.Add(GetParentIndex(SelectedBone));
	}
	
	// find all selected leaf nodes and make each one a chain
    for (const int32 SelectedBone : SelectedBones)
    {
    	// is this a leaf node in the selection?
    	if (SelectedParentIndices.Contains(SelectedBone))
    	{
    		continue;
    	}

    	// convert leaf nodes to chains
    	int32 ChainStart;
    	const int32 ChainEnd = SelectedBone;
    	int32 ParentIndex = SelectedBone;
    	while (true)
    	{
    		ChainStart = ParentIndex;
    		ParentIndex = GetParentIndex(ParentIndex);
    		if (ParentIndex == INDEX_NONE)
    		{
    			break;
    		}
    		if (!SelectedBones.Contains(ParentIndex))
    		{
    			break;
    		}
    	}

    	const FName ChainName = NAME_None;
    	const FName StartBoneName = GetBoneNameFromIndex(ChainStart);
    	const FName EndBoneName = GetBoneNameFromIndex(ChainEnd);
    	const FName GoalName = NAME_None;
    	OutChains.Add(FBoneChain(ChainName, StartBoneName, EndBoneName, GoalName));	
    }
}

bool FIKRigSkeleton::ValidateChainAndGetBones(const FBoneChain& BoneChain, TArray<int32>& OutBoneIndices) const
{
	OutBoneIndices.Reset();
	
	// validate start and end bones exist and are not the root
	const int32 StartIndex = GetBoneIndexFromName(BoneChain.StartBone.BoneName);
	const int32 EndIndex = GetBoneIndexFromName(BoneChain.EndBone.BoneName);
	const bool bFoundStartBone = StartIndex > INDEX_NONE;
	const bool bFoundEndBone = EndIndex > INDEX_NONE;

	// no need to build the chain if start/end indices are wrong 
	const bool bIsWellFormed = bFoundStartBone && bFoundEndBone && EndIndex >= StartIndex;
	if (!bIsWellFormed)
	{
		return false;
	}
	
	// init array with end bone 
	OutBoneIndices = {EndIndex};

	// if only one bone in the chain
	if (EndIndex == StartIndex)
	{
		return true;
	}

	// record all bones in chain while walking up the hierarchy (tip to root of chain)
	int32 ParentIndex = GetParentIndex(EndIndex);
	while (ParentIndex > INDEX_NONE && ParentIndex >= StartIndex)
	{
		OutBoneIndices.Add(ParentIndex);
		ParentIndex = GetParentIndex(ParentIndex);
	}

	// if we walked up till the start bone
	if (OutBoneIndices.Last() == StartIndex)
	{
		// reverse the indices (we want root to tip order)
		Algo::Reverse(OutBoneIndices);
		return true;
	}

	// oops, we walked all the way up without finding the start bone
	OutBoneIndices.Reset();
	return false;
}

bool FIKRigSkeleton::GetMirroredBoneIndices(
	const TArray<int32>& BoneIndices,
	TArray<int32>& OutMirroredIndices) const
{
	auto GetDistancesToAllBones = [this](const FVector& InPoint, TArray<float>& OutDistances)
	{
		OutDistances.SetNumUninitialized(RefPoseGlobal.Num());
		int32 Index = 0;
		for (const FTransform& BoneTransform : RefPoseGlobal)
		{
			OutDistances[Index] = static_cast<float>(FVector::Distance(InPoint, BoneTransform.GetLocation()));
			++Index;
		}
	};
	
	auto GetBonesWithinDistance = [this](const float MaxDistance, const TArray<float>& InDistances, TArray<int32>& OutBoneIndices)
	{
		OutBoneIndices.Reset();
		int32 Index = 0;
		for (const float& Distance : InDistances)
		{
			if (Distance < MaxDistance)
			{
				OutBoneIndices.Add(Index);
			}
			++Index;
		}
		return !OutBoneIndices.IsEmpty();
	};

	auto GetBoneWithClosestName = [this](const FName BaseName, const TArray<int32>& InBoneIndices) -> int32
	{
		check(!InBoneIndices.IsEmpty());
		int32 BoneWithClosestNameToBaseName = InBoneIndices[0];
		const FString BaseNameStr = BaseName.ToString().ToLower();
		float HighestScore = -1.f;
		for (const int32 BoneIndex : InBoneIndices)
		{
			FString CurrentNameStr = BoneNames[BoneIndex].ToString().ToLower();
			float WorstCase = static_cast<float>(BaseNameStr.Len() + CurrentNameStr.Len());
			WorstCase = WorstCase < 1.0f ? 1.0f : WorstCase;
			const float Score = 1.0f - (static_cast<float>(Algo::LevenshteinDistance(BaseNameStr, CurrentNameStr)) / WorstCase);
			if (Score > HighestScore)
			{
				HighestScore = Score;
				BoneWithClosestNameToBaseName = BoneIndex;
			}
		}
		
		return BoneWithClosestNameToBaseName;
	};

	auto GetBoundingSphere = [this]()->FSphere
	{
		TArray<FVector> Points;
		Points.AddUninitialized(RefPoseGlobal.Num());
		int32 Index=0;
		for (const FTransform& BoneTransform : RefPoseGlobal)
		{
			Points[Index] = BoneTransform.GetLocation();
			++Index;
		}
		return FSphere(&Points[0], Points.Num());
	};

	// generate bounding sphere of skeleton to establish reasonable search radius
	const FSphere BoundingSphere = GetBoundingSphere();
	if (BoundingSphere.W < KINDA_SMALL_NUMBER)
	{
		// skeleton is degenerate
		return false;
	}
		
	const float MinSearchRadius = static_cast<float>(BoundingSphere.W) * 0.01f;
	const float MaxSearchRadius = static_cast<float>(BoundingSphere.W);
	const float SearchStep = MinSearchRadius * 2.f;

	// spin through all the bones in the chain:
	// 1. iteratively expand a search radius to find bones near the mirrored location
	// 2. pick the bone near the mirrored location with the most similar name
	//
	// NOTE: we can't just do a "find closest bone" here because production skeletons often have lots of "helper" bones
	// that are located at or near the same position. So we have to use the name to help find the actual mirrored bone.
	OutMirroredIndices.Reset();
	for (const int32 BoneIndex : BoneIndices)
	{
		// get mirrored location to search around
		const FVector BoneLocation = RefPoseGlobal[BoneIndex].GetLocation();
		const FVector MirroredLocation = BoneLocation * FVector(-1.f,1.f,1.f);
		
		// get distance from the mirrored location to all other bones in the skeleton
		TArray<float> DistancesToAllBones;
		GetDistancesToAllBones(MirroredLocation, DistancesToAllBones);
		
		// do an expanding search around the mirrored location until we find bones near it
		TArray<int32> NearbyBones;
		float SearchRadius = MinSearchRadius;
		while(SearchRadius <= MaxSearchRadius)
		{
			if (GetBonesWithinDistance(SearchRadius, DistancesToAllBones, NearbyBones))
			{
				break;
			}
			SearchRadius += SearchStep;
		};
		
		if (NearbyBones.IsEmpty())
		{
			OutMirroredIndices.Reset();
			return false;
		}

		// found the nearby bone with the best matching name
		int32 BestMatchBoneIndex = GetBoneWithClosestName(BoneNames[BoneIndex], NearbyBones);
		OutMirroredIndices.Add(BestMatchBoneIndex);
	}
	
	return true;
}
