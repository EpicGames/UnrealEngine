// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/Solvers/IKRigBodyMoverSolver.h"
#include "Rig/IKRigDataTypes.h"
#include "Rig/IKRigSkeleton.h"
#include "Rig/Solvers/PointsToRotation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigBodyMoverSolver)

#define LOCTEXT_NAMESPACE "BodyMoverSolver"

void FIKRigBodyMoverSolver::Initialize(const FIKRigSkeleton& InIKRigSkeleton)
{
	BodyBoneIndex = InIKRigSkeleton.GetBoneIndexFromName(Settings.StartBone);
}

void FIKRigBodyMoverSolver::Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals)
{
	// no body bone specified
	if (BodyBoneIndex == INDEX_NONE)
	{
		return;
	}

	// no goals connected
	if (AllGoalSettings.IsEmpty())
	{
		return;
	}
	
	// ensure body bone exists
	check(InIKRigSkeleton.RefPoseGlobal.IsValidIndex(BodyBoneIndex));
	
	// calculate a "best fit" transform from deformed goal locations
	TArray<FVector> InitialPoints;
	TArray<FVector> CurrentPoints;
	for (const FIKRigBodyMoverGoalSettings& GoalSetting : AllGoalSettings)
	{
		const FIKRigGoal* Goal = InGoals.FindGoalByName(GoalSetting.Goal);
		if (!Goal)
		{
			return;
		}

		const int32 BoneIndex = InIKRigSkeleton.GetBoneIndexFromName(GoalSetting.BoneName);
		const FVector InitialPosition = InIKRigSkeleton.CurrentPoseGlobal[BoneIndex].GetTranslation();
		const FVector GoalPosition = Goal->FinalBlendedPosition;
		const FVector FinalPosition = FMath::Lerp(InitialPosition, GoalPosition, GoalSetting.InfluenceMultiplier);
		
		InitialPoints.Add(InitialPosition);
		CurrentPoints.Add(FinalPosition);
	}
	
	FVector InitialCentroid;
	FVector CurrentCentroid;
	const FQuat RotationOffset = GetRotationFromDeformedPoints(
		InitialPoints,
		CurrentPoints,
		InitialCentroid,
		CurrentCentroid);

	// alpha blend the position offset and add it to the current bone location
	const FVector Offset = (CurrentCentroid - InitialCentroid);
	const FVector Weight(
		Offset.X > 0.f ? Settings.PositionPositiveX : Settings.PositionNegativeX,
		Offset.Y > 0.f ? Settings.PositionPositiveY : Settings.PositionNegativeY,
		Offset.Z > 0.f ? Settings.PositionPositiveZ : Settings.PositionNegativeZ);

	// the bone transform to modify
	FTransform& CurrentBodyTransform = InIKRigSkeleton.CurrentPoseGlobal[BodyBoneIndex];
	CurrentBodyTransform.AddToTranslation(Offset * (Weight*Settings.PositionAlpha));

	// do per-axis alpha blend
	FVector Euler = RotationOffset.Euler() * FVector(Settings.RotateXAlpha, Settings.RotateYAlpha, Settings.RotateZAlpha);
	FQuat FinalRotationOffset = FQuat::MakeFromEuler(Euler);
	// alpha blend the entire rotation offset
	FinalRotationOffset = FQuat::FastLerp(FQuat::Identity, FinalRotationOffset, Settings.RotationAlpha).GetNormalized();
	// add rotation offset to original rotation
	CurrentBodyTransform.SetRotation(FinalRotationOffset * CurrentBodyTransform.GetRotation());

	// do FK update of children
	InIKRigSkeleton.PropagateGlobalPoseBelowBone(BodyBoneIndex);
}

void FIKRigBodyMoverSolver::GetRequiredBones(TSet<FName>& OutRequiredBones) const
{
	OutRequiredBones.Add(Settings.StartBone);
}

void FIKRigBodyMoverSolver::GetRequiredGoals(TSet<FName>& OutRequiredGoals) const
{
	for (const FIKRigBodyMoverGoalSettings& GoalSetting : AllGoalSettings)
	{
		OutRequiredGoals.Add(GoalSetting.Goal);
	}
}

FIKRigSolverSettingsBase* FIKRigBodyMoverSolver::GetSolverSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRigBodyMoverSolver::GetSolverSettingsType() const
{
	return FIKRigBodyMoverSettings::StaticStruct();
}

void FIKRigBodyMoverSolver::AddGoal(const UIKRigEffectorGoal& InNewGoal)
{
	FIKRigBodyMoverGoalSettings GoalSettings;
	GoalSettings.Goal = InNewGoal.GoalName;
	GoalSettings.BoneName = InNewGoal.BoneName;
	AllGoalSettings.Add(GoalSettings);
}

void FIKRigBodyMoverSolver::OnGoalRemoved(const FName& InGoalName)
{
	// check goal exists in this solver
	const int32 GoalIndex = GetIndexOfGoal(InGoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return;
	}

	// remove it
	AllGoalSettings.RemoveAt(GoalIndex);
}

void FIKRigBodyMoverSolver::OnGoalRenamed(const FName& InOldName, const FName& InNewName)
{
	// ensure goal even exists in this solver
	const int32 GoalIndex = GetIndexOfGoal(InOldName);
	if (GoalIndex == INDEX_NONE)
	{
		return;
	}

	// rename
	AllGoalSettings[GoalIndex].Goal = InNewName;
}

void FIKRigBodyMoverSolver::OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName)
{
	// ensure goal even exists in this solver
	const int32 GoalIndex = GetIndexOfGoal(InGoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return;
	}

	// rename
	AllGoalSettings[GoalIndex].BoneName = InNewBoneName;
}

FIKRigGoalSettingsBase* FIKRigBodyMoverSolver::GetGoalSettings(const FName& InGoalName)
{
	const int32 GoalIndex = GetIndexOfGoal(InGoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return nullptr;
	}

	return &AllGoalSettings[GoalIndex];
}

const UScriptStruct* FIKRigBodyMoverSolver::GetGoalSettingsType() const
{
	return FIKRigBodyMoverGoalSettings::StaticStruct();
}

void FIKRigBodyMoverSolver::GetGoalsWithSettings(TSet<FName>& OutGoalsWithSettings) const
{
	for (const FIKRigBodyMoverGoalSettings& GoalSettings : AllGoalSettings)
	{
		OutGoalsWithSettings.Add(GoalSettings.Goal);
	}
}

void FIKRigBodyMoverSolver::SetStartBone(const FName& InStartBoneName)
{
	Settings.StartBone = InStartBoneName;
}

FIKRigBodyMoverSettings UIKRigBodyMoverController::GetSolverSettings()
{
	return *static_cast<FIKRigBodyMoverSettings*>(SolverToControl->GetSolverSettings());
}

void UIKRigBodyMoverController::SetSolverSettings(FIKRigBodyMoverSettings InSettings)
{
	SolverToControl->SetSolverSettings(&InSettings);
}

FIKRigBodyMoverGoalSettings UIKRigBodyMoverController::GetGoalSettings(const FName InGoalName)
{
	if (FIKRigGoalSettingsBase* GoalSettings = SolverToControl->GetGoalSettings(InGoalName))
	{
		return *static_cast<FIKRigBodyMoverGoalSettings*>(GoalSettings);
	}
	return FIKRigBodyMoverGoalSettings();
}

void UIKRigBodyMoverController::SetGoalSettings(const FName InGoalName, FIKRigBodyMoverGoalSettings InSettings)
{
	SolverToControl->SetGoalSettings(InGoalName, &InSettings);
}

#if WITH_EDITOR

UIKRigSolverControllerBase* FIKRigBodyMoverSolver::GetSolverController(UObject* Outer)
{
	return CreateControllerIfNeeded(Outer, UIKRigBodyMoverController::StaticClass());
}

FText FIKRigBodyMoverSolver::GetNiceName() const
{
	return FText(LOCTEXT("SolverName", "Body Mover"));
}

bool FIKRigBodyMoverSolver::GetWarningMessage(FText& OutWarningMessage) const
{
	if (Settings.StartBone == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingRoot", "Missing start bone.");
		return true;
	}

	if (AllGoalSettings.IsEmpty())
	{
		OutWarningMessage = LOCTEXT("MissingGoal", "Missing goals.");
		return true;
	}
	
	return false;
}

bool FIKRigBodyMoverSolver::IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const
{
	return InIKRigSkeleton.IsBoneInDirectLineage(InBoneName, Settings.StartBone);
}

#endif

int32 FIKRigBodyMoverSolver::GetIndexOfGoal(const FName& InGoalName) const
{
	return AllGoalSettings.IndexOfByPredicate([&InGoalName](const FIKRigBodyMoverGoalSettings& InElement)
	{
		return InElement.Goal == InGoalName;
	});
}

#undef LOCTEXT_NAMESPACE

