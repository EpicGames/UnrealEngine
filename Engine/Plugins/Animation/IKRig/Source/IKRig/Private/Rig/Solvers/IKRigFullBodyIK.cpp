// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/Solvers/IKRigFullBodyIK.h"

#include "Rig/IKRigDataTypes.h"
#include "Rig/IKRigSkeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigFullBodyIK)

#define LOCTEXT_NAMESPACE "FFBIKSolver"

void FIKRigFullBodyIKSolver::Initialize(const FIKRigSkeleton& InSkeleton)
{	
	// check how many effectors are assigned to a bone
	int NumEffectors = 0;
	for (const FIKRigFBIKGoalSettings& Effector : AllGoalSettings)
	{
		if (InSkeleton.GetBoneIndexFromName(Effector.BoneName) != INDEX_NONE)
		{
			++NumEffectors; // bone is set and exists!
		}
	}

	// validate inputs are ready to be initialized
	const bool bHasEffectors = NumEffectors > 0;
	const bool bRootIsAssigned = Settings.RootBone != NAME_None;
	if (!(bHasEffectors && bRootIsAssigned))
	{
		return; // not setup yet
	}

	// reset all internal data
	Solver.Reset();

	// create bones
	for (int BoneIndex = 0; BoneIndex < InSkeleton.BoneNames.Num(); ++BoneIndex)
	{
		const FName& Name = InSkeleton.BoneNames[BoneIndex];

		// get the parent bone solver index
		const int32 ParentIndex = InSkeleton.GetParentIndexThatIsNotExcluded(BoneIndex);
		const FTransform OrigTransform = InSkeleton.RefPoseGlobal[BoneIndex];
		const FVector InOrigPosition = OrigTransform.GetLocation();
		const FQuat InOrigRotation = OrigTransform.GetRotation();
		const bool bIsRoot = Name == Settings.RootBone;
		Solver.AddBone(Name, ParentIndex, InOrigPosition, InOrigRotation, bIsRoot);
	}

	// create effectors
	for (FIKRigFBIKGoalSettings& Effector : AllGoalSettings)
	{
		Effector.IndexInSolver = Solver.AddEffector(Effector.BoneName);
	}
		
	// initialize solver
	Solver.Initialize();
}

void FIKRigFullBodyIKSolver::Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals)
{
	if (!Solver.IsReadyToSimulate())
	{
		return;
	}

	if (Solver.GetNumBones() != InIKRigSkeleton.BoneNames.Num())
	{
		return;
	}

	TArray<FTransform>& InOutTransforms = InIKRigSkeleton.CurrentPoseGlobal;
	
	// set bones to input pose
	for(int32 BoneIndex = 0; BoneIndex < Solver.GetNumBones(); BoneIndex++)
	{
		Solver.SetBoneTransform(BoneIndex, InOutTransforms[BoneIndex]);
	}

	// update bone settings
	for (const FIKRigFBIKBoneSettings& BoneSetting : AllBoneSettings)
	{
		const int32 BoneIndex = Solver.GetBoneIndex(BoneSetting.Bone);
		if (PBIK::FBoneSettings* InternalSettings = Solver.GetBoneSettings(BoneIndex))
		{
			BoneSetting.CopyToCoreStruct(*InternalSettings);
		}
	}

	// update effectors
	for (const FIKRigFBIKGoalSettings& GoalSettings : AllGoalSettings)
	{
		if (GoalSettings.IndexInSolver < 0)
		{
			continue;
		}
		
		const FIKRigGoal* Goal = InGoals.FindGoalByName(GoalSettings.Goal);
		if (!Goal)
		{
			return;
		}

		PBIK::FEffectorSettings EffectorSettings;
		EffectorSettings.PositionAlpha = 1.0f; // this is constant because IKRig manages offset alphas itself
		EffectorSettings.RotationAlpha = 1.0f; // this is constant because IKRig manages offset alphas itself
		EffectorSettings.StrengthAlpha = GoalSettings.StrengthAlpha;
		EffectorSettings.ChainDepth = GoalSettings.ChainDepth;
		EffectorSettings.PullChainAlpha = GoalSettings.PullChainAlpha;
		EffectorSettings.PinRotation = GoalSettings.PinRotation;
		
		Solver.SetEffectorGoal(
			GoalSettings.IndexInSolver,
			Goal->FinalBlendedPosition,
			Goal->FinalBlendedRotation,
			EffectorSettings);
	}

	// update settings
	FPBIKSolverSettings SolverSettings;
	SolverSettings.Iterations = Settings.Iterations;
	SolverSettings.SubIterations = Settings.SubIterations;
	SolverSettings.MassMultiplier = Settings.MassMultiplier;
	SolverSettings.bAllowStretch = Settings.bAllowStretch;
	SolverSettings.RootBehavior = Settings.RootBehavior;
	SolverSettings.PrePullRootSettings = Settings.PrePullRootSettings;
	SolverSettings.GlobalPullChainAlpha = Settings.GlobalPullChainAlpha;
	SolverSettings.MaxAngle = Settings.MaxAngle;
	SolverSettings.OverRelaxation = Settings.OverRelaxation;

	// solve
	Solver.Solve(SolverSettings);

	// copy transforms back
	for(int32 BoneIndex = 0; BoneIndex < Solver.GetNumBones(); BoneIndex++)
	{
		Solver.GetBoneGlobalTransform(BoneIndex, InOutTransforms[BoneIndex]);
	}
}

void FIKRigFullBodyIKSolver::GetRequiredBones(TSet<FName>& OutRequiredBones) const
{
	OutRequiredBones.Add(Settings.RootBone);
	// TODO in the future make this bone agnostic, stop storing goal bone names and find dynamically at Init() time
	for (const FIKRigFBIKGoalSettings& GoalSettings : AllGoalSettings)
	{
		OutRequiredBones.Add(GoalSettings.BoneName);
	}
}

void FIKRigFullBodyIKSolver::GetRequiredGoals(TSet<FName>& OutRequiredGoals) const
{
	for (const FIKRigFBIKGoalSettings& GoalSettings : AllGoalSettings)
	{
		OutRequiredGoals.Add(GoalSettings.Goal);
	}
}

FIKRigSolverSettingsBase* FIKRigFullBodyIKSolver::GetSolverSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRigFullBodyIKSolver::GetSolverSettingsType() const
{
	return FIKRigFBIKSettings::StaticStruct();
}

// GOAL SETTINGS

void FIKRigFullBodyIKSolver::AddGoal(const UIKRigEffectorGoal& InNewGoal)
{
	AllGoalSettings.Emplace(InNewGoal.GoalName, InNewGoal.BoneName);
}

void FIKRigFullBodyIKSolver::OnGoalRenamed(const FName& InOldName, const FName& InNewName)
{
	// ensure goal even exists in this solver
	const int32 GoalIndex = GetIndexOfGoal(InOldName);
	if (GoalIndex == INDEX_NONE)
	{
		return;
	}

	// rename
	AllGoalSettings[GoalIndex].Goal = InNewName;
}

void FIKRigFullBodyIKSolver::OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InBoneName)
{
	// ensure goal even exists in this solver
	const int32 GoalIndex = GetIndexOfGoal(InGoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return;
	}

	// goal moved to different bone
	AllGoalSettings[GoalIndex].BoneName = InBoneName;
}

void FIKRigFullBodyIKSolver::OnGoalRemoved(const FName& InGoalName)
{
	// ensure goal even exists in this solver
	const int32 GoalIndex = GetIndexOfGoal(InGoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return;
	}

	// remove it
	AllGoalSettings.RemoveAt(GoalIndex);
}

bool FIKRigFullBodyIKSolver::UsesCustomGoalSettings() const
{
	return true;
}

FIKRigGoalSettingsBase* FIKRigFullBodyIKSolver::GetGoalSettings(const FName& InGoalName)
{
	const int32 GoalIndex = GetIndexOfGoal(InGoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return nullptr;
	}

	return &AllGoalSettings[GoalIndex];
}

const UScriptStruct* FIKRigFullBodyIKSolver::GetGoalSettingsType() const
{
	return FIKRigFBIKGoalSettings::StaticStruct();
}

void FIKRigFullBodyIKSolver::GetGoalsWithSettings(TSet<FName>& OutGoalsWithSettings) const
{
	for (const FIKRigFBIKGoalSettings& GoalSettings : AllGoalSettings)
	{
		OutGoalsWithSettings.Add(GoalSettings.Goal);
	}
}

// START BONE

bool FIKRigFullBodyIKSolver::UsesStartBone() const
{
	return true;
}

FName FIKRigFullBodyIKSolver::GetStartBone() const
{
	return Settings.RootBone;
}

void FIKRigFullBodyIKSolver::SetStartBone(const FName& InStartBoneName)
{
	Settings.RootBone = InStartBoneName;
}

// BONE SETTINGS

bool FIKRigFullBodyIKSolver::UsesCustomBoneSettings() const
{
	return true;
}

void FIKRigFullBodyIKSolver::AddSettingsToBone(const FName& InBoneName)
{
	if (AllBoneSettings.ContainsByPredicate([&](const FIKRigFBIKBoneSettings& Element){return Element.Bone == InBoneName;}))
	{
		// bone already has settings
		return;
	}
	
	AllBoneSettings.Emplace(InBoneName);
}

void FIKRigFullBodyIKSolver::RemoveSettingsOnBone(const FName& InBoneName)
{
	AllBoneSettings.RemoveAll([&](const FIKRigFBIKBoneSettings& Element)
	{
		return Element.Bone == InBoneName;
	});
}

FIKRigBoneSettingsBase* FIKRigFullBodyIKSolver::GetBoneSettings(const FName& InBoneName)
{
	for (FIKRigFBIKBoneSettings& BoneSetting : AllBoneSettings)
	{
		if (BoneSetting.Bone == InBoneName)
		{
			return &BoneSetting;
		}
	}
	
	return nullptr;
}

const UScriptStruct* FIKRigFullBodyIKSolver::GetBoneSettingsType() const
{
	return FIKRigFBIKBoneSettings::StaticStruct();
}

bool FIKRigFullBodyIKSolver::HasSettingsOnBone(const FName& InBoneName) const
{
	for (const FIKRigFBIKBoneSettings& BoneSetting : AllBoneSettings)
	{
		if (BoneSetting.Bone == InBoneName)
		{
			return true;
		}
	}

	return false;
}

void FIKRigFullBodyIKSolver::GetBonesWithSettings(TSet<FName>& OutBonesWithSettings) const
{
	for (const FIKRigFBIKBoneSettings& BoneSetting : AllBoneSettings)
	{
		OutBonesWithSettings.Add(BoneSetting.Bone);
	}
}

#if WITH_EDITOR

UIKRigSolverControllerBase* FIKRigFullBodyIKSolver::GetSolverController(UObject* Outer)
{
	return CreateControllerIfNeeded(Outer, UIKRigFBIKController::StaticClass());
}

FText FIKRigFullBodyIKSolver::GetNiceName() const
{
	return FText(LOCTEXT("SolverName", "Full Body IK"));
}

bool FIKRigFullBodyIKSolver::GetWarningMessage(FText& OutWarningMessage) const
{
	if (Settings.RootBone == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingStart", "Missing start bone.");
		return true;
	}

	if (AllGoalSettings.IsEmpty())
	{
		OutWarningMessage = LOCTEXT("MissingGoal", "Missing goals.");
		return true;
	}
	
	return false;
}

bool FIKRigFullBodyIKSolver::IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const
{
	// nothing is affected by solver without a root bone assigned or at least 1 effector
	if (Settings.RootBone == NAME_None || AllGoalSettings.IsEmpty())
	{
		return false;
	}

	// has to be BELOW root
	if (!InIKRigSkeleton.IsBoneInDirectLineage(InBoneName, Settings.RootBone))
	{
		return false;
	}

	// has to be ABOVE an effector
	for (const FIKRigFBIKGoalSettings& Effector : AllGoalSettings)
	{
		if (InIKRigSkeleton.IsBoneInDirectLineage(Effector.BoneName, InBoneName))
		{
			return true;
		}
	}
	
	return false;
}

#endif

int32 FIKRigFullBodyIKSolver::GetIndexOfGoal(const FName& InGoalName) const
{
	return AllGoalSettings.IndexOfByPredicate([InGoalName](const FIKRigFBIKGoalSettings& Element)
	{
		return Element.Goal == InGoalName;
	});
}

//
// BEGIN CONTROLLER
//

FIKRigFBIKSettings UIKRigFBIKController::GetSolverSettings()
{
	return *static_cast<FIKRigFBIKSettings*>(SolverToControl->GetSolverSettings());
}

void UIKRigFBIKController::SetSolverSettings(FIKRigFBIKSettings InSettings)
{
	SolverToControl->SetSolverSettings(&InSettings);
}

FIKRigFBIKGoalSettings UIKRigFBIKController::GetGoalSettings(const FName InGoalName)
{
	if (FIKRigGoalSettingsBase* GoalSettings = SolverToControl->GetGoalSettings(InGoalName))
	{
		return *static_cast<FIKRigFBIKGoalSettings*>(GoalSettings);
	}
	return FIKRigFBIKGoalSettings();
}

void UIKRigFBIKController::SetGoalSettings(const FName InGoalName, FIKRigFBIKGoalSettings InSettings)
{
	SolverToControl->SetGoalSettings(InGoalName, &InSettings);
}

FIKRigFBIKBoneSettings UIKRigFBIKController::GetBoneSettings(const FName InBoneName)
{
	if (FIKRigFBIKBoneSettings* BoneSettings = static_cast<FIKRigFBIKBoneSettings*>(SolverToControl->GetBoneSettings(InBoneName)))
	{
		return *BoneSettings;
	}
	
	FIKRigFBIKBoneSettings NewBoneSettings;
	NewBoneSettings.Bone = InBoneName;
	return NewBoneSettings;
}

void UIKRigFBIKController::SetBoneSettings(const FName InBoneName, FIKRigFBIKBoneSettings InSettings)
{
	SolverToControl->SetBoneSettings(InBoneName, &InSettings);
}

#undef LOCTEXT_NAMESPACE

