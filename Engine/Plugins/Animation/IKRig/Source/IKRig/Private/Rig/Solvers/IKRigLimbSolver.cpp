// Copyright Epic Games, Inc. All Rights Reserved.


#include "Rig/Solvers/IKRigLimbSolver.h"

#include "Rig/IKRigDataTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigLimbSolver)

#define LOCTEXT_NAMESPACE "IKRig_LimbSolver"

void FIKRigLimbSolver::Initialize(const FIKRigSkeleton& InIKRigSkeleton)
{
	Solver.Reset();
	ChildrenToUpdate.Empty();
	
	if (Settings.GoalName == NAME_None || Settings.EndBone == NAME_None || Settings.StartBone == NAME_None)
	{
		return;
	}

	int32 BoneIndex = InIKRigSkeleton.GetBoneIndexFromName(Settings.EndBone);
	const int32 RootIndex = InIKRigSkeleton.GetBoneIndexFromName(Settings.StartBone);
	if (BoneIndex == INDEX_NONE || RootIndex == INDEX_NONE)
	{
		return;
	}

	// populate indices
	TArray<int32> BoneIndices( {BoneIndex} );
	BoneIndex = InIKRigSkeleton.GetParentIndex(BoneIndex);
	while (BoneIndex != INDEX_NONE && BoneIndex >= RootIndex)
	{
		BoneIndices.Add(BoneIndex);
		BoneIndex = InIKRigSkeleton.GetParentIndex(BoneIndex);
	};

	// if chain is not long enough
	if (BoneIndices.Num() < 3)
	{
		return;
	}

	// sort the chain from root to end
	Algo::Reverse(BoneIndices);

	// initialize solver
	for (int32 Index: BoneIndices)
	{
		const FVector Location = InIKRigSkeleton.CurrentPoseGlobal[Index].GetLocation();
		Solver.AddLink(Location, Index);
	}

	const bool bInitialized = Solver.Initialize();
	if (bInitialized)
	{
		// store children that needs propagation once solved
		TArray<int32> Children;
		for (int32 Index = 0; Index < BoneIndices.Num()-1; ++Index)
		{
			// store children if not already handled by the solver (part if the links)
			InIKRigSkeleton.GetChildIndices(BoneIndices[Index], Children);
			const int32 NextIndex = BoneIndices[Index+1];
			for (const int32 ChildIndex: Children)
			{
				if (ChildIndex != NextIndex)
				{
					ChildrenToUpdate.Add(ChildIndex);
					GatherChildren(ChildIndex, InIKRigSkeleton, ChildrenToUpdate);
				}
			}
		}
		// store end bone children
		GatherChildren(BoneIndices.Last(), InIKRigSkeleton, ChildrenToUpdate);
	}
}

void FIKRigLimbSolver::Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals)
{
	if (Solver.NumLinks() < 3)
	{
		return;
	}

	// fetch the goal transform
	const FIKRigGoal* IKGoal = InGoals.FindGoalByName(Settings.GoalName);
	if (!IKGoal)
	{
		return;
	}
	const FVector& GoalLocation = IKGoal->FinalBlendedPosition;
	const FQuat& GoalRotation = IKGoal->FinalBlendedRotation;

	// run the solve
	const bool bModifiedLimb = Solver.Solve(
		InIKRigSkeleton.CurrentPoseGlobal,
		GoalLocation,
		GoalRotation,
		Settings);

	// propagate if needed
	if (bModifiedLimb)
	{
		// update chain bones local transform
		for (int32 Index = 0; Index < Solver.NumLinks(); Index++)
		{
			InIKRigSkeleton.UpdateLocalTransformFromGlobal(Solver.GetBoneIndex(Index));
		}

		// propagate to children
		for (const int32 ChildIndex: ChildrenToUpdate)
		{
			InIKRigSkeleton.UpdateGlobalTransformFromLocal(ChildIndex);
		}
	}
}

void FIKRigLimbSolver::GetRequiredBones(TSet<FName>& OutRequiredBones) const
{
	OutRequiredBones.Add(Settings.StartBone);
	OutRequiredBones.Add(Settings.EndBone);
}

void FIKRigLimbSolver::GetRequiredGoals(TSet<FName>& OutRequiredGoals) const
{
	OutRequiredGoals.Add(Settings.GoalName);
}

FIKRigSolverSettingsBase* FIKRigLimbSolver::GetSolverSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRigLimbSolver::GetSolverSettingsType() const
{
	return FIKRigLimbSolverSettings::StaticStruct();
}

void FIKRigLimbSolver::AddGoal(const UIKRigEffectorGoal& InNewGoal)
{
	Settings.GoalName = InNewGoal.GoalName;
	Settings.EndBone = InNewGoal.BoneName;
}

void FIKRigLimbSolver::OnGoalRenamed(const FName& InOldName, const FName& InNewName)
{
	if (Settings.GoalName == InOldName)
	{
		Settings.GoalName = InNewName;
	}
}

void FIKRigLimbSolver::OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName)
{
	if (Settings.GoalName == InGoalName)
	{
		Settings.EndBone = InNewBoneName;
	}
}

void FIKRigLimbSolver::OnGoalRemoved(const FName& InGoalName)
{
	if (Settings.GoalName == InGoalName)
	{
		Settings.GoalName = NAME_None;
		Settings.EndBone = NAME_None;
	}
}

bool FIKRigLimbSolver::UsesStartBone() const
{
	return true;
}

void FIKRigLimbSolver::SetStartBone(const FName& InStartBoneName)
{
	Settings.StartBone = InStartBoneName;
}

FName FIKRigLimbSolver::GetStartBone() const
{
	return Settings.StartBone;
}

FIKRigLimbSolverSettings UIKRigLimbSolverController::GetSolverSettings()
{
	return *static_cast<FIKRigLimbSolverSettings*>(SolverToControl->GetSolverSettings());
}

void UIKRigLimbSolverController::SetSolverSettings(FIKRigLimbSolverSettings InSettings)
{
	SolverToControl->SetSolverSettings(&InSettings);
}

#if WITH_EDITOR

UIKRigSolverControllerBase* FIKRigLimbSolver::GetSolverController(UObject* Outer)
{
	return CreateControllerIfNeeded(Outer, UIKRigLimbSolverController::StaticClass());
}


FText FIKRigLimbSolver::GetNiceName() const
{
	return FText(LOCTEXT("SolverName", "Limb IK"));
}

bool FIKRigLimbSolver::GetWarningMessage(FText& OutWarningMessage) const
{
	if (Settings.StartBone == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingRoot", "Missing root.");
		return true;
	}

	if (Settings.GoalName == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingGoal", "Missing goal.");
		return true;
	}

	if (Solver.NumLinks() < 3)
	{
		OutWarningMessage = LOCTEXT("Requires3BonesChain", "Requires at least 3 bones between root and goal.");
		return true;
	}
	
	return false;
}

bool FIKRigLimbSolver::IsBoneAffectedBySolver(const FName& BoneName, const FIKRigSkeleton& IKRigSkeleton) const
{
	return IKRigSkeleton.IsBoneInDirectLineage(BoneName, Settings.StartBone);
}

#endif

void FIKRigLimbSolver::GatherChildren(const int32 BoneIndex, const FIKRigSkeleton& InSkeleton, TArray<int32>& OutChildren)
{
	TArray<int32> Children;
	InSkeleton.GetChildIndices(BoneIndex, Children);
	for (int32 ChildIndex: Children)
	{
		OutChildren.Add(ChildIndex);
		GatherChildren(ChildIndex, InSkeleton, OutChildren);
	}
}

#undef LOCTEXT_NAMESPACE

