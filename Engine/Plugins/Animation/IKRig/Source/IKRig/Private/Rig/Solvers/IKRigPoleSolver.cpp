// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/Solvers/IKRigPoleSolver.h"
#include "Rig/IKRigDataTypes.h"
#include "Rig/IKRigSkeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigPoleSolver)

#define LOCTEXT_NAMESPACE "PoleSolver"

void FIKRigPoleSolver::Initialize(const FIKRigSkeleton& InIKRigSkeleton)
{
	Chain.Empty();
	
	int32 BoneIndex = InIKRigSkeleton.GetBoneIndexFromName(Settings.EndBone);
	const int32 RootIndex = InIKRigSkeleton.GetBoneIndexFromName(Settings.StartBone);

	if (BoneIndex == INDEX_NONE || RootIndex == INDEX_NONE)
	{
		return;
	}

	// populate chain
	Chain.Add(BoneIndex);
	BoneIndex = InIKRigSkeleton.GetParentIndex(BoneIndex);
	while (BoneIndex != INDEX_NONE && BoneIndex >= RootIndex)
	{
		Chain.Add(BoneIndex);
		BoneIndex = InIKRigSkeleton.GetParentIndex(BoneIndex);
	};

	// if chain is not long enough
	if (Chain.Num() < 3)
	{
		Chain.Empty();
		return;
	}
	
	// sort the chain from root to end
	Algo::Reverse(Chain);
	
	// store children that needs propagation once solved
	TArray<int32> Children;
	for (int32 Index = 0; Index < Chain.Num()-1; ++Index)
	{
		// store children if not already handled by the solver
		InIKRigSkeleton.GetChildIndices(Chain[Index], Children);
		const int32 NextIndex = Chain[Index+1];
		for (const int32 ChildIndex: Children)
		{
			if (ChildIndex != NextIndex)
			{
				ChildrenToUpdate.Add(ChildIndex);
				GatherChildren(ChildIndex, InIKRigSkeleton, ChildrenToUpdate);
			}
		}
	}
}

void FIKRigPoleSolver::Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals)
{
	if (Chain.Num() < 3)
	{
		return;
	}
	
	const FIKRigGoal* IKGoal = InGoals.FindGoalByName(Settings.AimAtGoal);
	if (!IKGoal)
	{
		return;
	}

	const bool bHasAlpha = Settings.Alpha > KINDA_SMALL_NUMBER;
	if (!bHasAlpha)
	{
		return;
	}

	const int32 RootIndex = Chain[0];
	const int32 KneeIndex = Chain[1];
	const int32 EndIndex = Chain.Last();

	TArray<FTransform>& InOutTransforms = InIKRigSkeleton.CurrentPoseGlobal;

	// initial configuration
	const FVector RootLocation = InOutTransforms[RootIndex].GetLocation();
	const FVector KneeLocation = InOutTransforms[KneeIndex].GetLocation();
	const FVector EndLocation = InOutTransforms[EndIndex].GetLocation();
	
	const FVector RootToEnd = (EndLocation - RootLocation).GetSafeNormal();
	const FVector RootToKnee = (KneeLocation - RootLocation).GetSafeNormal();
	if (RootToEnd.IsZero() || RootToKnee.IsZero())
	{
		return;
	}
	const FVector InitPlane = FVector::CrossProduct(RootToEnd, RootToKnee).GetSafeNormal();

	// target configuration
	const FVector& GoalLocation = IKGoal->FinalBlendedPosition;
	const FVector RootToPole = (GoalLocation - RootLocation).GetSafeNormal();
	if (RootToPole.IsZero())
	{
		return;
	}
	const FVector TargetPlane = FVector::CrossProduct(RootToEnd, RootToPole).GetSafeNormal();

	// compute delta rotation from InitialPlane to TargetPlane
	if (InitPlane.IsZero() || InitPlane.Equals(TargetPlane))
	{
    	return;
    }
	
	const FQuat DeltaRotation = FQuat::FindBetweenNormals(InitPlane, TargetPlane);
	if (DeltaRotation.IsIdentity())
	{
		return;
	}
	
	// update transforms
	for (int32 Index = 0; Index < Chain.Num()-1; Index++)
	{
		int32 BoneIndex = Chain[Index];
		FTransform& BoneTransform = InOutTransforms[BoneIndex];

		// rotation
		const FQuat BoneRotation = BoneTransform.GetRotation();
		const FQuat TargetRotation = FMath::Lerp(BoneRotation, DeltaRotation * BoneRotation, Settings.Alpha);
		BoneTransform.SetRotation(TargetRotation);

		// translation
		const FVector BoneTranslation = BoneTransform.GetLocation();
		const FVector TargetTranslation = FMath::Lerp( BoneTranslation, RootLocation + DeltaRotation.RotateVector(BoneTranslation - RootLocation), Settings.Alpha);
		BoneTransform.SetTranslation(TargetTranslation);
	}

	// propagate to children
	for (const int32 ChildIndex: ChildrenToUpdate)
	{
		InIKRigSkeleton.UpdateGlobalTransformFromLocal(ChildIndex);
	}
}

void FIKRigPoleSolver::GetRequiredBones(TSet<FName>& OutRequiredBones) const
{
	OutRequiredBones.Add(Settings.StartBone);
	OutRequiredBones.Add(Settings.EndBone);
}

void FIKRigPoleSolver::GetRequiredGoals(TSet<FName>& OutRequiredGoals) const
{
	OutRequiredGoals.Add(Settings.AimAtGoal);
}

FIKRigSolverSettingsBase* FIKRigPoleSolver::GetSolverSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRigPoleSolver::GetSolverSettingsType() const
{
	return FIKRigPoleSolverSettings::StaticStruct();
}

void FIKRigPoleSolver::AddGoal(const UIKRigEffectorGoal& NewGoal)
{
	Settings.AimAtGoal = NewGoal.GoalName;
}

void FIKRigPoleSolver::OnGoalRenamed(const FName& InOldName, const FName& InNewName)
{
	if (Settings.AimAtGoal == InOldName)
	{
		Settings.AimAtGoal = InNewName;
	}
}

void FIKRigPoleSolver::OnGoalRemoved(const FName& InName)
{
	if (Settings.AimAtGoal == InName)
	{
		Settings.AimAtGoal = NAME_None;
	}
}

void FIKRigPoleSolver::OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName)
{
	if (Settings.AimAtGoal == InGoalName)
	{
		Settings.EndBone = InNewBoneName;
	}
}

void FIKRigPoleSolver::SetStartBone(const FName& InRootBoneName)
{
	Settings.StartBone = InRootBoneName;
}

FName FIKRigPoleSolver::GetStartBone() const
{
	return Settings.StartBone;
}

void FIKRigPoleSolver::SetEndBone(const FName& InEndBoneName)
{
	Settings.EndBone = InEndBoneName;
}

FName FIKRigPoleSolver::GetEndBone() const
{
	return Settings.EndBone;
}

FIKRigPoleSolverSettings UIKRigPoleSolverController::GetSolverSettings()
{
	return *static_cast<FIKRigPoleSolverSettings*>(SolverToControl->GetSolverSettings());
}

void UIKRigPoleSolverController::SetSolverSettings(FIKRigPoleSolverSettings InSettings)
{
	SolverToControl->SetSolverSettings(&InSettings);
}

#if WITH_EDITOR

UIKRigSolverControllerBase* FIKRigPoleSolver::GetSolverController(UObject* Outer)
{
	return CreateControllerIfNeeded(Outer, UIKRigPoleSolverController::StaticClass());
}

FText FIKRigPoleSolver::GetNiceName() const
{
	return FText(LOCTEXT("SolverName", "Pole Solver"));
}

bool FIKRigPoleSolver::GetWarningMessage(FText& OutWarningMessage) const
{
	if (Settings.StartBone == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingRoot", "Missing root bone.");
		return true;
	}
	
	if (Settings.EndBone == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingEnd", "Missing end bone.");
		return true;
	}
	
	if (Settings.AimAtGoal == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingGoal", "Missing aim goal.");
		return true;
	}
	
	if (Chain.Num() < 3)
	{
		OutWarningMessage = LOCTEXT("Requires3BonesChain", "Requires at least 3 bones between root and end bones.");
		return true;
	}

	return false;
}

bool FIKRigPoleSolver::IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const
{
	const bool bAffected = InIKRigSkeleton.IsBoneInDirectLineage(InBoneName, Settings.StartBone);
	if (!bAffected)
	{
		return false;
	}
	
	const int32 EndIndex = Chain.IsEmpty() ? INDEX_NONE : Chain.Last();
	if (EndIndex == INDEX_NONE)
	{
		return false;
	}

	const int32 ChildIndex = InIKRigSkeleton.GetBoneIndexFromName(InBoneName);
	return ChildIndex <= EndIndex;
}

#endif

void FIKRigPoleSolver::GatherChildren(const int32 BoneIndex, const FIKRigSkeleton& InSkeleton, TArray<int32>& OutChildren)
{
	TArray<int32> Children;
	InSkeleton.GetChildIndices(BoneIndex, Children);
	for (int32 ChildIndex: Children)
	{
		OutChildren.Add(ChildIndex);
		GatherChildren(ChildIndex, InSkeleton, OutChildren);
	}
}

#undef LOCTEXT_NAMESPACE

