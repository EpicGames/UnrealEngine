// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/Solvers/IKRigSetTransform.h"
#include "Rig/IKRigDataTypes.h"
#include "Rig/IKRigSkeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigSetTransform)

#define LOCTEXT_NAMESPACE "SetTransformSolver"

void FIKRigSetTransform::Initialize(const FIKRigSkeleton& IKRigSkeleton)
{
	BoneIndex = IKRigSkeleton.GetBoneIndexFromName(Settings.BoneToAffect);
}

void FIKRigSetTransform::Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals)
{
	// no bone specified
	if (BoneIndex == INDEX_NONE)
	{
		return;
	}

	// no goal specified
	const FIKRigGoal* InGoal = InGoals.FindGoalByName(Settings.Goal);
	if (!InGoal)
	{
		return;
	}

	// check that settings are such that there is anything to do at all
	const bool bPositionEnabled = Settings.PositionAlpha > UE_KINDA_SMALL_NUMBER;
	const bool bRotationEnabled = Settings.RotationAlpha > UE_KINDA_SMALL_NUMBER;
	const bool bAnythingEnabled = bPositionEnabled || bRotationEnabled;
	const bool bHasAlpha = Settings.Alpha > UE_KINDA_SMALL_NUMBER;
	if (!(bAnythingEnabled && bHasAlpha))
	{
		return;
	}

	FTransform& CurrentTransform = InIKRigSkeleton.CurrentPoseGlobal[BoneIndex];	

	if (bPositionEnabled)
	{
		const FVector TargetPosition = FMath::Lerp(CurrentTransform.GetTranslation(), InGoal->FinalBlendedPosition, Settings.PositionAlpha * Settings.Alpha);
		CurrentTransform.SetLocation(TargetPosition);
	}
	
	if (bRotationEnabled)
	{
		const FQuat TargetRotation = FMath::Lerp(CurrentTransform.GetRotation(), InGoal->FinalBlendedRotation, Settings.RotationAlpha * Settings.Alpha);
		CurrentTransform.SetRotation(TargetRotation);
	}

	if (Settings.bPropagateToChildren)
	{
		InIKRigSkeleton.PropagateGlobalPoseBelowBone(BoneIndex);	
	}
}

void FIKRigSetTransform::GetRequiredBones(TSet<FName>& OutRequiredBones) const
{
	OutRequiredBones.Add(Settings.BoneToAffect);
}

void FIKRigSetTransform::GetRequiredGoals(TSet<FName>& OutRequiredGoals) const
{
	if (Settings.Goal != NAME_Name)
	{
		OutRequiredGoals.Add(Settings.Goal);
	}
}

FIKRigSolverSettingsBase* FIKRigSetTransform::GetSolverSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRigSetTransform::GetSolverSettingsType() const
{
	return FIKRigSetTransformSettings::StaticStruct();
}

void FIKRigSetTransform::OnGoalRemoved(const FName& InGoalName)
{
	if (Settings.Goal == InGoalName)
	{
		Settings.Goal = NAME_None;
		Settings.BoneToAffect = NAME_None;
	}	
}

void FIKRigSetTransform::AddGoal(const UIKRigEffectorGoal& InNewGoal)
{
	Settings.Goal = InNewGoal.GoalName;
	Settings.BoneToAffect = InNewGoal.BoneName;
}

void FIKRigSetTransform::OnGoalRenamed(const FName& InOldName, const FName& InNewName)
{
	if (Settings.Goal == InOldName)
	{
		Settings.Goal = InNewName;
	}
}

void FIKRigSetTransform::OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName)
{
	if (Settings.Goal == InGoalName)
	{
		Settings.BoneToAffect = InNewBoneName;
	}
}

FIKRigSetTransformSettings UIKRigSetTransformController::GetSolverSettings()
{
	return *static_cast<FIKRigSetTransformSettings*>(SolverToControl->GetSolverSettings());
}

void UIKRigSetTransformController::SetSolverSettings(FIKRigSetTransformSettings InSettings)
{
	SolverToControl->SetSolverSettings(&InSettings);
}

#if WITH_EDITOR

UIKRigSolverControllerBase* FIKRigSetTransform::GetSolverController(UObject* Outer)
{
	return CreateControllerIfNeeded(Outer, UIKRigSetTransformController::StaticClass());
}

FText FIKRigSetTransform::GetNiceName() const
{
	return FText(LOCTEXT("SolverName", "Set Transform"));
}

bool FIKRigSetTransform::GetWarningMessage(FText& OutWarningMessage) const
{
	if (Settings.BoneToAffect == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingGoal", "Missing goal.");
		return true;
	}
	return false;
}

bool FIKRigSetTransform::IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const
{
	return InIKRigSkeleton.IsBoneInDirectLineage(InBoneName, Settings.BoneToAffect);
}

#endif

#undef LOCTEXT_NAMESPACE

