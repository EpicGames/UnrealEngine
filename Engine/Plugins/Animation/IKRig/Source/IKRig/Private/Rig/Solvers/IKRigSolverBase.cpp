// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/Solvers/IKRigSolverBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigSolverBase)

void FIKRigSolverBase::UpdateSettingsFromAsset(const FIKRigSolverBase& InAssetSolver)
{
	FIKRigSolverBase& AssetSolver = const_cast<FIKRigSolverBase&>(InAssetSolver);

	// copy enabled flag
	SetEnabled(AssetSolver.IsEnabled());
	
	// copy solver settings
	const UScriptStruct* SolverSettingsType = GetSolverSettingsType();
	const FIKRigSolverSettingsBase* CopyFrom = AssetSolver.GetSolverSettings();
	FIKRigSolverSettingsBase* CopyTo = GetSolverSettings();
	CopyAllEditableProperties(SolverSettingsType, CopyFrom, CopyTo);

	// (optional) copy goal settings
	if (InAssetSolver.UsesCustomGoalSettings())
	{
		const UScriptStruct* GoalSettingsType = GetGoalSettingsType();
		TSet<FName> GoalsWithSettings;
		InAssetSolver.GetGoalsWithSettings(GoalsWithSettings);
		for (FName Goal : GoalsWithSettings)
		{
			const FIKRigGoalSettingsBase* CopyFromGoalSettings = AssetSolver.GetGoalSettings(Goal);
			FIKRigGoalSettingsBase* CopyToGoalSettings = GetGoalSettings(Goal);
			CopyAllEditableProperties(GoalSettingsType, CopyFromGoalSettings, CopyToGoalSettings);	
		}
	}
	
	// (optional) copy bones settings
	if (InAssetSolver.UsesCustomBoneSettings())
	{
		const UScriptStruct* BoneSettingsType = GetBoneSettingsType();
		TSet<FName> BonesWithSettings;
		InAssetSolver.GetBonesWithSettings(BonesWithSettings);
		for (FName Bone : BonesWithSettings)
		{
			const FIKRigBoneSettingsBase* CopyFromBoneSettings = AssetSolver.GetBoneSettings(Bone);
			FIKRigBoneSettingsBase* CopyToBoneSettings = GetBoneSettings(Bone);
			CopyAllEditableProperties(BoneSettingsType, CopyFromBoneSettings, CopyToBoneSettings);	
		}
	}
}

void FIKRigSolverBase::SetSolverSettings(const FIKRigSolverSettingsBase* InSettings)
{
	CopyAllEditableProperties(GetSolverSettingsType(), InSettings, GetSolverSettings());
}

void FIKRigSolverBase::SetGoalSettings(const FName& InGoalName, FIKRigGoalSettingsBase* InSettings)
{
	ensure(UsesCustomGoalSettings());
	CopyAllEditableProperties(GetGoalSettingsType(), InSettings, GetGoalSettings(InGoalName));
}

void FIKRigSolverBase::SetBoneSettings(const FName& InBoneName, FIKRigBoneSettingsBase* InSettings)
{
	ensure(UsesCustomBoneSettings());
	CopyAllEditableProperties(GetBoneSettingsType(), InSettings, GetBoneSettings(InBoneName));
}

void FIKRigSolverBase::CopyAllEditableProperties(
	const UScriptStruct* SettingsType,
	const FIKRigSettingsBase* CopyFrom,
	FIKRigSettingsBase* CopyTo)
{
	// ensure source and destination were provided
	if (!(CopyFrom && CopyTo))
	{
		return;
	}
	
	// ensure the provided SettingsType is an IKRig settings stype
	if (!ensure(SettingsType && SettingsType->IsChildOf(FIKRigSettingsBase::StaticStruct())))
	{
		return;
	}

	// ensure CopyFrom is instance of or derives from SettingsType
	if (!ensure(SettingsType->IsChildOf(CopyFrom->StaticStruct())))
	{
		return;
	}

	// ensure CopyTo is instance of or derives from SettingsType
	if (!ensure(SettingsType->IsChildOf(CopyTo->StaticStruct())))
	{
		return;
	}

	// iterate through all properties of the SettingsType
	for (TFieldIterator<FProperty> It(SettingsType); It; ++It)
	{
		const FProperty* Property = *It;

		// don't copy properties unless they are marked BlueprintReadWrite
		if (Property->HasAnyPropertyFlags(CPF_BlueprintReadOnly | CPF_Transient))
		{
			continue;
		}

		// copy the value from CopyFrom to CopyTo
		const uint8* SrcValuePtr = Property->ContainerPtrToValuePtr<uint8>(CopyFrom);
		uint8* DestValuePtr = Property->ContainerPtrToValuePtr<uint8>(CopyTo);
		if (SrcValuePtr && DestValuePtr)
		{
			Property->CopyCompleteValue(DestValuePtr, SrcValuePtr);
		}
	}
}

#if WITH_EDITOR

UIKRigSolverControllerBase* FIKRigSolverBase::GetSolverController(UObject* Outer)
{
	return CreateControllerIfNeeded(Outer, UIKRigSolverControllerBase::StaticClass());
}

UIKRigSolverControllerBase* FIKRigSolverBase::CreateControllerIfNeeded(UObject* Outer, const UClass* ClassType)
{
	if (!Controller.IsValid())
	{
		if (ensure(ClassType && ClassType->IsChildOf(UIKRigSolverControllerBase::StaticClass())))
		{
			Controller = TStrongObjectPtr(NewObject<UIKRigSolverControllerBase>(Outer, ClassType));
			Controller->SolverToControl = this;
		}
	}
	return Controller.Get();
}

#endif

