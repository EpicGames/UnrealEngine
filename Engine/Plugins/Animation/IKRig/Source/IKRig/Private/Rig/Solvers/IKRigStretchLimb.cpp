// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/Solvers/IKRigStretchLimb.h"
#include "Rig/IKRigDataTypes.h"
#include "Rig/IKRigSkeleton.h"

#include "Algo/Accumulate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigStretchLimb)

#define LOCTEXT_NAMESPACE "IKRigStretchLimb"

void FIKRigStretchLimbSolver::Initialize(const FIKRigSkeleton& InIKRigSkeleton)
{
	bIsInitialized = false;

	// must assign a start/end bone and a goal
	if (Settings.Goal.IsNone() || Settings.EndBone.IsNone() || Settings.StartBone.IsNone())
	{
		return;
	}

	// get start and end bone
	const int32 EndBoneIndex = InIKRigSkeleton.GetBoneIndexFromName(Settings.EndBone);
	const int32 StartBoneIndex = InIKRigSkeleton.GetBoneIndexFromName(Settings.StartBone);
	if (EndBoneIndex == INDEX_NONE || StartBoneIndex == INDEX_NONE)
	{
		return;
	}

	// populate limb indices
	BoneIndices = {EndBoneIndex};
	int32 CurrentBoneIndex = InIKRigSkeleton.GetParentIndex(EndBoneIndex);
	while (CurrentBoneIndex != INDEX_NONE && CurrentBoneIndex >= StartBoneIndex)
	{
		BoneIndices.Add(CurrentBoneIndex);
		CurrentBoneIndex = InIKRigSkeleton.GetParentIndex(CurrentBoneIndex);
	};

	// limb must have at least two joints
	if (BoneIndices.Num() < 2)
	{
		return;
	}

	// sort the chain from root to end
	Algo::Reverse(BoneIndices);

	// get all children that need updated
	TArray<int32> AllChildren;
	InIKRigSkeleton.GetChildrenIndicesRecursive(StartBoneIndex, AllChildren);
	for (const int32 ChildBoneIndex : AllChildren)
	{
		if (BoneIndices.Contains(ChildBoneIndex))
		{
			continue;
		}
		ChildrenToUpdate.Add(ChildBoneIndex);
	}

	// initialize storage for bone positions prior to any translational offset
	BonePositionsPreOffset.Init(FVector::ZeroVector, BoneIndices.Num());

	// initialize pole vector params
	PoleVectorParams.Reset(BoneIndices.Num());
	const FVector Start = InIKRigSkeleton.RefPoseGlobal[StartBoneIndex].GetTranslation();
	const FVector End = InIKRigSkeleton.RefPoseGlobal[EndBoneIndex].GetTranslation();
	const FVector PoleVector = End - Start;
	InitialPoleVectorLenSq = FMath::Max(UE_DOUBLE_KINDA_SMALL_NUMBER,PoleVector.SizeSquared());
	for (const int32 BoneIndex : BoneIndices)
	{
		const FVector BonePosition = InIKRigSkeleton.RefPoseGlobal[BoneIndex].GetTranslation();
		const FVector StartToBone = BonePosition - Start;
		double T = FVector::DotProduct(StartToBone, PoleVector) / InitialPoleVectorLenSq;
		PoleVectorParams.Add(T);
	}

	// cache bone settings bone indices
	for (FIKRigStretchLimbBoneSettings& BoneSettings : AllBoneSettings)
	{
		const int32 BoneIndex = InIKRigSkeleton.GetBoneIndexFromName(BoneSettings.Bone);
		BoneSettings.CachedChainIndex = BoneIndices.Find(BoneIndex);
	}

	bIsInitialized = true;
}

void FIKRigStretchLimbSolver::Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals)
{
	if (!bIsInitialized)
	{
		return; // can't solve until initialized successfully
	}

	// get the goal
	const FIKRigGoal* Goal = InGoals.FindGoalByName(Settings.Goal);
	if (!Goal)
	{
		// should not happen
		ensureMsgf(false, TEXT("StretchLimbSolver unable to get assigned Goal."));
		return;
	}

	// update rest lengths with incoming animation
	UpdateRestLengths(InIKRigSkeleton);

	// orient the chain bones
	AimChainAtGoal(InIKRigSkeleton, Goal->FinalBlendedPosition);

	// record chain positions prior to translational offsets
	StoreBonePositions(InIKRigSkeleton);
	
	// translate the bones parallel to the pole vector
	SquashOrStretchChainParallel(InIKRigSkeleton, Goal->FinalBlendedPosition);

	// translate the bones perpendicular to the pole vector
	SquashChainPerpendicular(InIKRigSkeleton);

	// solve distance constraints
	RunFABRIK(InIKRigSkeleton, Goal->FinalBlendedPosition);

	// allow final stretching to hit goal
	StretchChainFinal(InIKRigSkeleton, Goal->FinalBlendedPosition);

	// done moving bones, update their orientations
	UpdateBoneOrientations(InIKRigSkeleton);

	// orient end bone to goal
	RotateEndBoneWithGoal(InIKRigSkeleton, Goal);

	// update all the children bones not in the chain
	UpdateFKChildren(InIKRigSkeleton);
}

void FIKRigStretchLimbSolver::AimChainAtGoal(
	FIKRigSkeleton& InIKRigSkeleton,
	const FVector& GoalLocation) const
{
	if (Settings.RotationMode != EStretchLimbRotationMode::OrientToGoal)
	{
		return;
	}

	// calc:
	// 1. Current Chain Vector: start to end
	// 2. New Chain Vector: start to goal
	const FVector& ChainStart = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[0]].GetLocation();
	const FVector& ChainEnd = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices.Last()].GetLocation();
	const FVector CurrentChainVector = ChainEnd - ChainStart;
	const FVector NewChainVector = GoalLocation - ChainStart;
	
	// rotate the whole bone chain to aim at the goal
	const FQuat RotationDelta = FQuat::FindBetweenVectors(CurrentChainVector, NewChainVector);
	const FVector Origin = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[0]].GetLocation();
	for (const int32 BoneIndex : BoneIndices)
	{
		FTransform& BoneTransform = InIKRigSkeleton.CurrentPoseGlobal[BoneIndex];
		const FVector RelativePosition = BoneTransform.GetLocation() - Origin;
		const FVector RotatedRelativePosition = RotationDelta.RotateVector(RelativePosition);
		BoneTransform.SetLocation(RotatedRelativePosition + Origin);
		BoneTransform.SetRotation(RotationDelta * BoneTransform.GetRotation());
	}
}

void FIKRigStretchLimbSolver::SquashOrStretchChainParallel(
	FIKRigSkeleton& InIKRigSkeleton,
	const FVector& GoalLocation) const
{
	// calc current and new pole vector (based on goal)
	const FVector& ChainStart = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[0]].GetLocation();
	const FVector& ChainEnd = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices.Last()].GetLocation();
	const FVector CurrentPoleVector = ChainEnd - ChainStart;

	// clamp goal to percent distance along chain (optionally prevents overextension)
	FVector ClampedGoal = GoalLocation;
	if (Settings.bEnableStretching)
	{
		const FVector UnclampedPoleVector = GoalLocation - ChainStart;
		const FVector ClampedPoleVector = UnclampedPoleVector.GetClampedToMaxSize(TotalRestLength * Settings.StretchStartPercent);
		ClampedGoal = ChainStart + ClampedPoleVector;
	}

	// new pole vector is from chain origin to clamped goal location
	const FVector NewPoleVector = ClampedGoal - ChainStart;

	// squash/stretch chain in it's input pose to reach goal
	ApplyParallelOffsetToChain(InIKRigSkeleton, ChainStart, NewPoleVector, CurrentPoleVector);
}

void FIKRigStretchLimbSolver::ApplyParallelOffsetToChain(
	FIKRigSkeleton& InIKRigSkeleton,
	const FVector& ChainStart,
	const FVector& InNewPoleVector,
	const FVector& CurrentPoleVector) const
{
	// compare length of the chain in its current configuration with the length it would need to reach goal
	const double NewChainLength = InNewPoleVector.Size();
	const double CurrentChainLength = CurrentPoleVector.Size();
	if (FMath::IsNearlyEqual(CurrentChainLength, NewChainLength))
	{
		return; // nothing to squash or stretch
	}

	for (int32 ChainIndex = 1; ChainIndex < BoneIndices.Num(); ++ChainIndex)
	{
		FTransform& BoneGlobalTransform = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[ChainIndex]];
		const FVector CurrentProjOnPoleVector = ChainStart + CurrentPoleVector * PoleVectorParams[ChainIndex];
		const FVector CurrentProjToBone = BoneGlobalTransform.GetLocation() - CurrentProjOnPoleVector;
		const FVector NewProjOnPoleVector = ChainStart + InNewPoleVector * PoleVectorParams[ChainIndex];
		BoneGlobalTransform.SetLocation(NewProjOnPoleVector + CurrentProjToBone);
	}
}

void FIKRigStretchLimbSolver::SquashChainPerpendicular(FIKRigSkeleton& InIKRigSkeleton)
{
	if (BoneIndices.Num() < 3)
	{
		return;
	}

	if (Settings.SquashMode == EStretchLimbSquashMode::None || FMath::IsNearlyZero(Settings.SquashStrength))
	{
		return;
	}
	
	// calc pole vector
	const FVector& ChainStart = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[0]].GetLocation();
	const FVector& ChainEnd = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices.Last()].GetLocation();
	const FVector PoleVector = ChainEnd - ChainStart;
	const double NewPoleVectorLenSq = FMath::Max(1.e-2,PoleVector.SizeSquared());
	if (NewPoleVectorLenSq >= InitialPoleVectorLenSq)
	{
		return; // limb is stretched
	}
	const FVector PoleNorm = PoleVector.GetSafeNormal();

	// calc squash direction for each bone (skipping start/end)
	TArray<FVector> SquashDirections;
	SquashDirections.Init(FVector::UpVector, BoneIndices.Num());
	for (int32 ChainIndex = 1; ChainIndex < BoneIndices.Num() - 1 ; ++ChainIndex)
	{
		const int32 BoneIndex = BoneIndices[ChainIndex];
		const FVector& BonePosition = InIKRigSkeleton.CurrentPoseGlobal[BoneIndex].GetLocation();
		const FVector StartToBone = BonePosition - ChainStart;

		// signed distance along the line direction
		const double T = FVector::DotProduct(StartToBone, PoleNorm);

		// closest point on the pole vector
		const FVector& ClosestPoint = ChainStart + T * PoleNorm;
		const FVector BendDirection = (BonePosition - ClosestPoint).GetSafeNormal();
		SquashDirections[ChainIndex] = BendDirection;
	}

	// override squash directions from bone settings
	for (FIKRigStretchLimbBoneSettings BoneSetting : AllBoneSettings)
	{
		if (BoneSetting.CachedChainIndex == INDEX_NONE)
		{
			continue;
		}

		const FTransform& BoneTransform = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[BoneSetting.CachedChainIndex]];
		const FVector SquashDirection = BoneTransform.GetRotation().RotateVector(BoneSetting.SquashDirection);
		SquashDirections[BoneSetting.CachedChainIndex] = SquashDirection;
	}

	// push bones along squash directions
	const double SquashRatio = 1.0 - NewPoleVectorLenSq / InitialPoleVectorLenSq;
	for (int32 ChainIndex = 1; ChainIndex < BoneIndices.Num() - 1 ; ++ChainIndex)
	{
		auto GetSquashFalloff = [](double InT, EStretchLimbSquashMode InSquashMode) -> double
		{
			if (InSquashMode == EStretchLimbSquashMode::Uniform)
			{
				return 1;
			}
				
			if (InSquashMode == EStretchLimbSquashMode::Bulge)
			{
				InT = FMath::Clamp(InT, 0.0f, 1.0f);
				return 2.0f * FMath::Sqrt(InT * (1.0f - InT)); // 0 at 0, 0 at 1, 1 at 0.5
			}

			checkNoEntry();
			return 1.0;
		};

		const int32 BoneIndex = BoneIndices[ChainIndex];
		const FVector& BendDirection = SquashDirections[ChainIndex];
		const double Falloff = GetSquashFalloff(PoleVectorParams[ChainIndex], Settings.SquashMode);
		InIKRigSkeleton.CurrentPoseGlobal[BoneIndex].AddToTranslation(Settings.SquashStrength * BendDirection * Falloff * SquashRatio);
	}
}

void FIKRigStretchLimbSolver::StretchChainFinal(FIKRigSkeleton& InIKRigSkeleton, const FVector& GoalLocation) const
{
	if (!Settings.bEnableStretching)
	{
		return;
	}
	
	// calc current and new pole vector (based on goal)
	const FVector& ChainStart = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[0]].GetLocation();
	const FVector& ChainEnd = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices.Last()].GetLocation();
	const FVector CurrentPoleVector = ChainEnd - ChainStart;
	FVector NewPoleVector = GoalLocation - ChainStart;

	if (Settings.MaximumStretchDistance >= 0.f)
	{
		// hard clamp
		NewPoleVector = NewPoleVector.GetClampedToMaxSize(TotalRestLength + Settings.MaximumStretchDistance);
	}
	
	// squash/stretch chain in it's input pose to reach goal
	ApplyParallelOffsetToChain(InIKRigSkeleton, ChainStart, NewPoleVector, CurrentPoleVector);
}

void FIKRigStretchLimbSolver::RunFABRIK(FIKRigSkeleton& InIKRigSkeleton, const FVector& GoalLocation)
{
	if (BoneIndices.Num() < 3)
	{
		return;
	}

	if (Settings.Iterations <= 0)
	{
		return;
	}

	// clamp goal to percent distance along chain (optionally prevents overextension)
	const FVector FABRIKStart = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[0]].GetLocation();
	FVector FABRIKEnd = GoalLocation;
	if (Settings.bEnableStretching)
	{
		const FVector CurrentPoleVector = FABRIKEnd - FABRIKStart;
		const FVector ClampedPoleVector = CurrentPoleVector.GetClampedToMaxSize(TotalRestLength * Settings.StretchStartPercent);
		FABRIKEnd = FABRIKStart + ClampedPoleVector;
	}

	// get positions for FABRIK
	TArray<FVector> BonePositions;
	BonePositions.Init(FVector::UpVector, BoneIndices.Num());
	for (int32 ChainIndex = 0; ChainIndex < BoneIndices.Num(); ++ChainIndex)
	{
		const int32 BoneIndex = BoneIndices[ChainIndex];
		BonePositions[ChainIndex] = InIKRigSkeleton.CurrentPoseGlobal[BoneIndex].GetLocation();
	}

	// run FABRIK iterations
	for (int32 Iteration=0; Iteration<Settings.Iterations; ++Iteration)
	{
		RunFABRIKForwardPass(FABRIKStart, BonePositions);
		RunFABRIKBackwardPass(FABRIKEnd, BonePositions);
	}

	// always finish with a forward pass
	RunFABRIKForwardPass(FABRIKStart, BonePositions);
	
	// apply FABRIK positions 
	for (int32 ChainIndex = 1; ChainIndex < BoneIndices.Num(); ++ChainIndex)
	{
		const int32 BoneIndex = BoneIndices[ChainIndex];
		InIKRigSkeleton.CurrentPoseGlobal[BoneIndex].SetLocation(BonePositions[ChainIndex]);
	}
}

void FIKRigStretchLimbSolver::StoreBonePositions(FIKRigSkeleton& InIKRigSkeleton)
{
	for (int32 ChainIndex = 0; ChainIndex < BoneIndices.Num(); ++ChainIndex)
	{
		const int32 BoneIndex = BoneIndices[ChainIndex];
		BonePositionsPreOffset[ChainIndex] = InIKRigSkeleton.CurrentPoseGlobal[BoneIndex].GetLocation();
	}
}

void FIKRigStretchLimbSolver::UpdateBoneOrientations(FIKRigSkeleton& InIKRigSkeleton)
{
	// update orientations of bones based on translational offset
	for (int32 ChainIndex = 0; ChainIndex < BoneIndices.Num() - 1; ++ChainIndex)
	{
		// get the direction of the bone prior to any translational offsets
		const FVector BoneDirectionOrig =  BonePositionsPreOffset[ChainIndex+1] - BonePositionsPreOffset[ChainIndex];

		// get the direction of the bone after translations were applied
		const int32 BoneIndex = BoneIndices[ChainIndex];
		const int32 ChildIndex = BoneIndices[ChainIndex + 1];
		FTransform& BoneGlobalTransform = InIKRigSkeleton.CurrentPoseGlobal[BoneIndex];
		const FTransform& ChildGlobalTransform = InIKRigSkeleton.CurrentPoseGlobal[ChildIndex];
		const FVector BoneDirectionNew = ChildGlobalTransform.GetLocation() - BoneGlobalTransform.GetLocation();

		// generate a swing rotation to reorient the bone towards it's new direction 
		const FQuat RotationDelta = FQuat::FindBetweenVectors(BoneDirectionOrig, BoneDirectionNew);
		BoneGlobalTransform.SetRotation( RotationDelta * BoneGlobalTransform.GetRotation() );
	}
}

void FIKRigStretchLimbSolver::RunFABRIKBackwardPass(
	const FVector& Goal,
	TArray<FVector>& BonePositions)
{
	check(BonePositions.Num() >= 2);
	check(RestLengths.Num() == BonePositions.Num());

	// backward pass: leaf to root (keep leaf fixed)
	BonePositions.Last() = Goal; // reset leaf
	for (int32 i = BonePositions.Num() - 2; i >= 0; --i)
	{
		const double RestLength = RestLengths[i+1];
		const FVector ToParent = BonePositions[i] - BonePositions[i+1];
		const double CurrentLength = ToParent.Size();
		if (CurrentLength < KINDA_SMALL_NUMBER)
		{
			continue;
		}

		// move bone towards child to satisfy rest length
		BonePositions[i] = BonePositions[i+1] + (RestLength / CurrentLength) * ToParent;
	}
}

void FIKRigStretchLimbSolver::RunFABRIKForwardPass(
	const FVector& Start,
	TArray<FVector>& BonePositions)
{
	check(BonePositions.Num() >= 2);
	check(RestLengths.Num() == BonePositions.Num());

	// forward pass: root to leaf (keep root fixed)
	BonePositions[0] = Start; // reset root
	for (int32 i = 1; i < BonePositions.Num(); ++i)
	{
		const double RestLength = RestLengths[i];
		const FVector ToChild = BonePositions[i] - BonePositions[i-1];
		const double CurrentLength = ToChild.Size();
		if (CurrentLength < KINDA_SMALL_NUMBER)
		{
			continue;
		}

		// move bone towards parent to satisfy rest length
		BonePositions[i] = BonePositions[i-1] + (RestLength / CurrentLength) * ToChild;
	}	
}

double FIKRigStretchLimbSolver::CalculateCurrentChainLength(FIKRigSkeleton& InIKRigSkeleton) const
{
	double ChainLength = 0;
	
	// iterate from tip to first child of root
	for (int32 BoneIndex = BoneIndices.Num() - 1; BoneIndex >= 1; --BoneIndex)
	{
		const FVector& BonePosition = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[BoneIndex]].GetLocation();
		const FVector& ParentPosition = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices[BoneIndex-1]].GetLocation();
		ChainLength += (ParentPosition - BonePosition).Length();
	}
	
	return ChainLength;
}

void FIKRigStretchLimbSolver::UpdateRestLengths(FIKRigSkeleton& InIKRigSkeleton)
{
	// reset/update rest lengths of bones (may be animated)
	RestLengths.Reset(BoneIndices.Num());
	for (const int32 BoneIndex : BoneIndices)
	{
		const int32 ParentIndex = InIKRigSkeleton.GetParentIndex(BoneIndex);
		if (ParentIndex == INDEX_NONE)
		{
			RestLengths.Add( InIKRigSkeleton.RefPoseGlobal[BoneIndex].GetTranslation().Size() );
			continue;
		}
		const FVector ParentPosition = InIKRigSkeleton.RefPoseGlobal[ParentIndex].GetTranslation();
		const FVector BonePosition = InIKRigSkeleton.RefPoseGlobal[BoneIndex].GetTranslation();
		RestLengths.Add((BonePosition - ParentPosition).Size());
	}

	TotalRestLength = Algo::Accumulate(RestLengths, 0.0);
}

void FIKRigStretchLimbSolver::UpdateFKChildren(FIKRigSkeleton& InIKRigSkeleton)
{
	// update chain local transforms
	for (const int32 BoneIndex : BoneIndices)
	{
		InIKRigSkeleton.UpdateLocalTransformFromGlobal(BoneIndex);
	}

	// propagate to children
	for (const int32 ChildIndex: ChildrenToUpdate)
	{
		InIKRigSkeleton.UpdateGlobalTransformFromLocal(ChildIndex);
	}
}

void FIKRigStretchLimbSolver::RotateEndBoneWithGoal(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoal* Goal)
{
	// orient end bone to goal
	FTransform& EndBoneGlobal = InIKRigSkeleton.CurrentPoseGlobal[BoneIndices.Last()];
	FQuat EndBoneGlobalRotation = FQuat::FastLerp(EndBoneGlobal.GetRotation(), Goal->FinalBlendedRotation, Settings.RotateEndBoneWithGoal);
	EndBoneGlobalRotation = EndBoneGlobalRotation.GetNormalized();
	EndBoneGlobal.SetRotation(EndBoneGlobalRotation);
}

void FIKRigStretchLimbSolver::GetRequiredBones(TSet<FName>& OutRequiredBones) const
{
	OutRequiredBones.Add(Settings.StartBone);
}

void FIKRigStretchLimbSolver::GetRequiredGoals(TSet<FName>& OutRequiredGoals) const
{
	OutRequiredGoals.Add(Settings.Goal);
}

FIKRigSolverSettingsBase* FIKRigStretchLimbSolver::GetSolverSettings()
{
	return &Settings;
}

const UScriptStruct* FIKRigStretchLimbSolver::GetSolverSettingsType() const
{
	return FIKRigStretchLimbSettings::StaticStruct();
}

void FIKRigStretchLimbSolver::AddGoal(const UIKRigEffectorGoal& InNewGoal)
{
	Settings.Goal = InNewGoal.GoalName;
	Settings.EndBone = InNewGoal.BoneName;
}

void FIKRigStretchLimbSolver::OnGoalRemoved(const FName& InGoalName)
{
	if (InGoalName == Settings.Goal)
	{
		Settings.Goal = NAME_None;
		Settings.EndBone = NAME_None;
	}
}

void FIKRigStretchLimbSolver::OnGoalRenamed(const FName& InOldName, const FName& InNewName)
{
	if (InOldName == Settings.Goal)
	{
		Settings.Goal = InNewName;
	}
}

void FIKRigStretchLimbSolver::OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName)
{
	if (InGoalName == Settings.Goal)
	{
		Settings.StartBone = InNewBoneName;
	}
}

void FIKRigStretchLimbSolver::SetStartBone(const FName& InBoneName)
{
	Settings.StartBone = InBoneName;
}

// BONE SETTINGS

bool FIKRigStretchLimbSolver::UsesCustomBoneSettings() const
{
	return true;
}

void FIKRigStretchLimbSolver::AddSettingsToBone(const FName& InBoneName)
{
	if (AllBoneSettings.ContainsByPredicate([&](const FIKRigStretchLimbBoneSettings& Element){return Element.Bone == InBoneName;}))
	{
		// bone already has settings
		return;
	}
	
	AllBoneSettings.Emplace(InBoneName);
}

void FIKRigStretchLimbSolver::RemoveSettingsOnBone(const FName& InBoneName)
{
	AllBoneSettings.RemoveAll([&](const FIKRigStretchLimbBoneSettings& Element)
	{
		return Element.Bone == InBoneName;
	});
}

FIKRigBoneSettingsBase* FIKRigStretchLimbSolver::GetBoneSettings(const FName& InBoneName)
{
	for (FIKRigStretchLimbBoneSettings& BoneSetting : AllBoneSettings)
	{
		if (BoneSetting.Bone == InBoneName)
		{
			return &BoneSetting;
		}
	}
	
	return nullptr;
}

const UScriptStruct* FIKRigStretchLimbSolver::GetBoneSettingsType() const
{
	return FIKRigStretchLimbBoneSettings::StaticStruct();
}

bool FIKRigStretchLimbSolver::HasSettingsOnBone(const FName& InBoneName) const
{
	for (const FIKRigStretchLimbBoneSettings& BoneSetting : AllBoneSettings)
	{
		if (BoneSetting.Bone == InBoneName)
		{
			return true;
		}
	}

	return false;
}

void FIKRigStretchLimbSolver::GetBonesWithSettings(TSet<FName>& OutBonesWithSettings) const
{
	for (const FIKRigStretchLimbBoneSettings& BoneSetting : AllBoneSettings)
	{
		OutBonesWithSettings.Add(BoneSetting.Bone);
	}
}

#if WITH_EDITOR

UIKRigSolverControllerBase* FIKRigStretchLimbSolver::GetSolverController(UObject* Outer)
{
	return nullptr;
}

FText FIKRigStretchLimbSolver::GetNiceName() const
{
	return FText(LOCTEXT("SolverName", "Stretch Limb"));
}

bool FIKRigStretchLimbSolver::GetWarningMessage(FText& OutWarningMessage) const
{
	if (Settings.StartBone == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingRoot", "No start bone assigned.");
		return true;
	}

	if (Settings.Goal == NAME_None)
	{
		OutWarningMessage = LOCTEXT("MissingGoal", "No goal assigned.");
		return true;
	}
	
	return false;
}

bool FIKRigStretchLimbSolver::IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const
{
	return InIKRigSkeleton.IsBoneInDirectLineage(InBoneName, Settings.StartBone);
}
#endif

#undef LOCTEXT_NAMESPACE

