// Copyright Epic Games, Inc. All Rights Reserved.

#include "Rig/Solvers/LimbSolver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LimbSolver)

void FLimbSolver::Reset()
{
	Length = 0.f;
	Links.Empty();
}

bool FLimbSolver::Initialize()
{
	Length = 0.f;

	// compute segments' length + total limb's length
	for (int32 Index = 1; Index < NumLinks(); ++Index)
	{
		FLimbLink& Link = Links[Index];
		Link.Length = static_cast<float>(FVector::Dist( Link.Location, Links[Index-1].Location));
		Length += Link.Length;
	}

	return (NumLinks() > 2);
}

bool FLimbSolver::Solve(
	TArray<FTransform>& InOutTransforms,
	const FVector& InGoalLocation,
	const FQuat& InGoalRotation,
	const FLimbSolverSettings& InSettings)
{
	// rotate root so the end aligns with the goal
	const bool bOriented = OrientTransformsTowardsGoal( InOutTransforms, InGoalLocation);

	// update links locations
	for (int32 Index = 0; Index < NumLinks(); ++Index)
	{
		const int32 BoneIndex = Links[Index].BoneIndex;
		Links[Index].Location = InOutTransforms[BoneIndex].GetLocation();
	}

	// solve ik
	const bool bReached = AnimationCore::SolveLimb(
		Links,
		Length,
		InGoalLocation,
		InSettings.ReachPrecision,
		InSettings.MaxIterations,
		InOutTransforms[Links[0].BoneIndex].GetUnitAxis(InSettings.HingeRotationAxis),
		InSettings.bEnableLimit,
		InSettings.MinRotationAngle,
		InSettings.ReachStepAlpha,
		InSettings.bAveragePull,
		InSettings.PullDistribution
		);

	const bool bAdjustedKneeTwist = InSettings.bEnableTwistCorrection ? AdjustKneeTwist(
															InOutTransforms,
															FTransform( InGoalRotation, InGoalLocation),
															InSettings.EndBoneForwardAxis ) : false;

	bool bModifiedLimb = bOriented || bReached || bAdjustedKneeTwist;
	
	// adjust end bone orientation
	const int32 EndIndex = Links.Last().BoneIndex;
	FTransform& EndTransform = InOutTransforms[EndIndex];
	if (bModifiedLimb || !EndTransform.GetRotation().Equals(InGoalRotation))
	{
		EndTransform.SetRotation(InGoalRotation);
		bModifiedLimb = true;
	}
	
	if (bModifiedLimb)
	{
		// update transforms
		UpdateTransforms(InOutTransforms);
	}

	return bModifiedLimb;
}

int32 FLimbSolver::NumLinks() const
{
	return Links.Num();
}

int32 FLimbSolver::AddLink(const FVector& InLocation, int32 InBoneIndex)
{
	return Links.Emplace(InLocation, InBoneIndex);
}

int32 FLimbSolver::GetBoneIndex(int32 Index) const
{
	check(Index >= INDEX_NONE && Index < Links.Num());
	return Links[Index].BoneIndex; 
}

bool FLimbSolver::OrientTransformsTowardsGoal(TArray<FTransform>& InOutTransforms, const FVector& InGoalLocation) const
{
	const int32 RootIndex = Links[0].BoneIndex;
	const int32 EndIndex = Links.Last().BoneIndex; 
	
	const FVector RootLocation = InOutTransforms[RootIndex].GetLocation();
	const FVector EndLocation = InOutTransforms[EndIndex].GetLocation();

	const FVector InitialDir = (EndLocation - RootLocation).GetSafeNormal();
	const FVector TargetDir = (InGoalLocation - RootLocation).GetSafeNormal();
	
	// find delta rotation take takes us from Old to New dir
	if (!InitialDir.IsZero() && !InitialDir.Equals(TargetDir))
	{
		const FQuat DeltaRotation = FQuat::FindBetweenNormals(InitialDir, TargetDir);
		if (!DeltaRotation.IsIdentity())
		{
			// rotate Leg so it is aligned with IK Target
			for (const FLimbLink& Link: Links)
			{
				FTransform& BoneTransform = InOutTransforms[Link.BoneIndex]; 
				
				BoneTransform.SetRotation(DeltaRotation * BoneTransform.GetRotation());

				const FVector BoneLocation = BoneTransform.GetLocation();
				BoneTransform.SetLocation(RootLocation + DeltaRotation.RotateVector(BoneLocation - RootLocation));
			}

			return true;
		}
	}

	return false;
}

void FLimbSolver::UpdateTransforms(TArray<FTransform>& InOutTransforms)
{
	// Rotations
	for (int32 Index = 1; Index < NumLinks(); Index++)
	{
		const int32 BoneIndex = Links[Index].BoneIndex;
		const FTransform& BoneTransform = InOutTransforms[BoneIndex];
		const FVector& TargetBoneLocation = Links[Index].Location;
		
		const int32 ParentIndex = Links[Index-1].BoneIndex;
		FTransform& ParentTransform = InOutTransforms[ParentIndex];
		const FVector& TargetParentLocation = Links[Index-1].Location;

		// calculate pre-translation vector between this bone and child
		const FVector InitialDir = (BoneTransform.GetLocation() - ParentTransform.GetLocation()).GetSafeNormal();

		// Get vector from the post-translation bone to it's child
		const FVector TargetDir = (TargetBoneLocation - TargetParentLocation).GetSafeNormal();

		const FQuat DeltaRotation = FQuat::FindBetweenNormals(InitialDir, TargetDir);
		ParentTransform.SetRotation(DeltaRotation * ParentTransform.GetRotation());
	}
 
	// Translations (do it in one pass?)
	for (int32 Index = 1; Index < NumLinks(); Index++)
	{
		const int32 BoneIndex = Links[Index].BoneIndex;
		InOutTransforms[BoneIndex].SetTranslation(Links[Index].Location);
	}
}

bool FLimbSolver::AdjustKneeTwist(TArray<FTransform>& InOutTransforms, const FTransform& InGoalTransform, const EAxis::Type FootBoneForwardAxis )
{
	const int32 EndIndex = Links.Last().BoneIndex;
	const FVector FootFKLocation = InOutTransforms[EndIndex].GetLocation();
	const FVector FootIKLocation = InGoalTransform.GetLocation();

	const int32 RootIndex = Links[0].BoneIndex;
	const FVector HipLocation = InOutTransforms[RootIndex].GetLocation();
	const FVector FootAxisZ = (FootIKLocation - HipLocation).GetSafeNormal();

	FVector FootFKAxisX = InOutTransforms[EndIndex].GetUnitAxis(FootBoneForwardAxis);
	FVector FootIKAxisX = InGoalTransform.GetUnitAxis(FootBoneForwardAxis);

	// Reorient X Axis to be perpendicular with FootAxisZ
	const FVector TmpFKAxisY = FVector::CrossProduct(FootAxisZ, FootFKAxisX);
	FootFKAxisX = FVector::CrossProduct(TmpFKAxisY, FootAxisZ);
	
	const FVector TmpIKAxisY = FVector::CrossProduct(FootAxisZ, FootIKAxisX);
	FootIKAxisX = FVector::CrossProduct(TmpIKAxisY, FootAxisZ);

	// Compare Axis X to see if we need a rotation to be performed
	if (!FootFKAxisX.IsZero() && !FootFKAxisX.Equals(FootIKAxisX))
	{
		const FQuat DeltaRotation = FQuat::FindBetweenNormals(FootFKAxisX, FootIKAxisX);
		if (!DeltaRotation.IsIdentity())
		{
			// rotate Leg so it is aligned with IK Target
			for (const FLimbLink& Link: Links)
			{
				FTransform& BoneTransform = InOutTransforms[Link.BoneIndex]; 
				
				BoneTransform.SetRotation(DeltaRotation * BoneTransform.GetRotation());

				const FVector BoneLocation = BoneTransform.GetLocation();
				BoneTransform.SetLocation(HipLocation + DeltaRotation.RotateVector(BoneLocation - HipLocation));
			}

			return true;
		}
	}

	return false;
}

namespace AnimationCore
{
	void SolveStraightLimb(TArray<FLimbLink>& InOutLinks, const FVector& GoalLocation)
	{
		const FVector& RootLocation = InOutLinks[0].Location;
		const FVector Direction = (GoalLocation - RootLocation).GetSafeNormal();

		for (int32 Index = 1; Index < InOutLinks.Num(); Index++)
		{
			InOutLinks[Index].Location = InOutLinks[Index-1].Location + Direction * InOutLinks[Index].Length;
		}
	}

	void SolveTwoBoneIK(TArray<FLimbLink>& InOutLinks, const FVector& InTargetLocation, const FVector& HingeRotationAxis)
	{
		check(InOutLinks.Num() == 3);

		FVector& pA = InOutLinks[2].Location; // Foot
		FVector& pB = InOutLinks[1].Location; // Knee
		FVector& pC = InOutLinks[0].Location; // Hip / Root
		
		// Move foot directly to target.
		pA = InTargetLocation;

		const FVector HipToFoot = pA - pC;

		// Use Law of Cosines to work out solution.
		// At this point we know the target location is reachable, and we are already aligned with that location.
		// So the leg is in the right plane.
		const float a = InOutLinks[1].Length;	// hip to knee
		const float b = static_cast<float>(HipToFoot.Length());		// hip to foot
		const float c = InOutLinks[2].Length;	// knee to foot
		
		const float Two_ab = 2.f * a * b;
		const float CosC = !FMath::IsNearlyZero(Two_ab) ? (a * a + b * b - c * c) / Two_ab : 0.f;
 		const float C = FMath::Acos(CosC);
		
		// Project Knee onto Hip to Foot line.
		const FVector HipToFootDir = !FMath::IsNearlyZero(b) ? HipToFoot / b : FVector::ZeroVector;
		const FVector HipToKnee = pB - pC;
		const FVector ProjKnee = pC + HipToKnee.ProjectOnToNormal(HipToFootDir);

		const FVector ProjKneeToKnee = (pB - ProjKnee);
		FVector BendDir = ProjKneeToKnee.GetSafeNormal(UE_KINDA_SMALL_NUMBER);
		
		// If we have a HingeRotationAxis defined, we can cache 'BendDir'
		// and use it when we can't determine it. (When limb is straight without a bend).
		// We do this instead of using an explicit one, so we carry over the pole vector that animators use. 
		// So they can animate it, and we try to extract it from the animation.
		if ((HingeRotationAxis != FVector::ZeroVector) && (HipToFootDir != FVector::ZeroVector) && !FMath::IsNearlyZero(a))
		{
			const FVector HipToKneeDir = HipToKnee / a;
			const float KneeBendDot = static_cast<float>(FVector::DotProduct(HipToKneeDir, HipToFootDir));

			FVector& CachedRealBendDir = InOutLinks[1].RealBendDir;
			FVector& CachedBaseBendDir = InOutLinks[1].BaseBendDir;

			// Valid 'bend', cache 'BendDir'
			if ((BendDir != FVector::ZeroVector) && (KneeBendDot < 0.99f))
			{
				CachedRealBendDir = BendDir;
				CachedBaseBendDir = FVector::CrossProduct(HingeRotationAxis, HipToFootDir);
			}
			// Limb is too straight, can't determine BendDir accurately, so use cached value if possible.
			else 
			{
				// If we have cached 'BendDir', then reorient it based on 'HingeRotationAxis'
				if (CachedRealBendDir != FVector::ZeroVector)
				{
					const FVector CurrentBaseBendDir = FVector::CrossProduct(HingeRotationAxis, HipToFootDir);
					const FQuat DeltaCachedToCurrBendDir = FQuat::FindBetweenNormals(CachedBaseBendDir, CurrentBaseBendDir);
					BendDir = DeltaCachedToCurrBendDir.RotateVector(CachedRealBendDir);
				}
			}
		}

		// We just combine both lines into one to save a multiplication.
		// const FVector NewProjectedKneeLoc = pC + HipToFootDir * a * CosC;
		// const FVector NewKneeLoc = NewProjectedKneeLoc + Dir_LegLineToKnee * a * FMath::Sin(C);
		const FVector NewKneeLoc = pC + a * (HipToFootDir * CosC + BendDir * FMath::Sin(C));
		pB = NewKneeLoc;
	}

	FVector FindPlaneNormal(const TArray<FLimbLink>& InLinks, const FVector& RootLocation, const FVector& TargetLocation)
	{
		const FVector AxisX = (TargetLocation - RootLocation).GetSafeNormal();

		for (int32 Index = 1; Index < InLinks.Num(); ++Index)
		{
			const FVector AxisY = (InLinks[Index].Location - RootLocation).GetSafeNormal();
			const FVector PlaneNormal = FVector::CrossProduct(AxisX, AxisY);

			// Make sure we have a valid normal (Axes were not coplanar).
			if (PlaneNormal.SizeSquared() > UE_SMALL_NUMBER)
			{
				return PlaneNormal.GetUnsafeNormal();
			}
		}

		// All links are co-planar?
		return FVector::UpVector;
	}

	void FABRIK_ApplyLinkConstraints_Forward(TArray<FLimbLink>& InOutLinks, int32 LinkIndex, float MinRotationAngleRadians)
	{
		if ((LinkIndex <= 0) || (LinkIndex >= InOutLinks.Num() - 1))
		{
			return;
		}
	
		const FLimbLink& ChildLink = InOutLinks[LinkIndex + 1];
		const FLimbLink& CurrentLink = InOutLinks[LinkIndex];
		FLimbLink& ParentLink = InOutLinks[LinkIndex - 1];
	
		const FVector ChildAxisX = (ChildLink.Location - CurrentLink.Location).GetSafeNormal();
		const FVector ChildAxisY = FVector::CrossProduct(CurrentLink.LinkAxisZ, ChildAxisX);
		const FVector ParentAxisX = (ParentLink.Location - CurrentLink.Location).GetSafeNormal();
	
		const float ParentCos = static_cast<float>(FVector::DotProduct(ParentAxisX, ChildAxisX));
		const float ParentSin = static_cast<float>(FVector::DotProduct(ParentAxisX, ChildAxisY));
	
		const bool bNeedsReorient = (ParentSin < 0.f) || (ParentCos > FMath::Cos(MinRotationAngleRadians));
	
		// Parent Link needs to be reoriented.
		if (bNeedsReorient)
		{
			// folding over itself.
			if (ParentCos > 0.f)
			{
				// Enforce minimum angle.
				ParentLink.Location = CurrentLink.Location + CurrentLink.Length * (FMath::Cos(MinRotationAngleRadians) * ChildAxisX + FMath::Sin(MinRotationAngleRadians) * ChildAxisY);
			}
			else
			{
				// When opening up leg, allow it to extend in a full straight line.
				ParentLink.Location = CurrentLink.Location - ChildAxisX * CurrentLink.Length;
			}
		}
	}
	
	void FABRIK_ApplyLinkConstraints_Backward(TArray<FLimbLink>& InOutLinks, int32 LinkIndex, float MinRotationAngleRadians)
	{
		if ((LinkIndex <= 0) || (LinkIndex >= InOutLinks.Num() - 1))
		{
			return;
		}
	
		FLimbLink& ChildLink = InOutLinks[LinkIndex + 1];
		const FLimbLink& CurrentLink = InOutLinks[LinkIndex];
		const FLimbLink& ParentLink = InOutLinks[LinkIndex - 1];
	
		const FVector ParentAxisX = (ParentLink.Location - CurrentLink.Location).GetSafeNormal();
		const FVector ParentAxisY = FVector::CrossProduct(CurrentLink.LinkAxisZ, ParentAxisX);
		const FVector ChildAxisX = (ChildLink.Location - CurrentLink.Location).GetSafeNormal();
	
		const float ChildCos = static_cast<float>(FVector::DotProduct(ChildAxisX, ParentAxisX));
		const float ChildSin = static_cast<float>(FVector::DotProduct(ChildAxisX, ParentAxisY));

		const float CosMinAngle = FMath::Cos(MinRotationAngleRadians);
		
		// Parent Link needs to be reoriented.
		const bool bNeedsReorient = (ChildSin > 0.f) || (ChildCos > CosMinAngle);
		if (bNeedsReorient)
		{
			// folding over itself.
			if (ChildCos > 0.f)
			{
				// Enforce minimum angle.
				const float SinMinAngle = FMath::Sin(MinRotationAngleRadians);
				ChildLink.Location = CurrentLink.Location + ChildLink.Length * (CosMinAngle * ParentAxisX - SinMinAngle * ParentAxisY);
			}
			else
			{
				// When opening up leg, allow it to extend in a full straight line.
				ChildLink.Location = CurrentLink.Location - ParentAxisX * ChildLink.Length;
			}
		}
	}
	
	void  FABRIK_ForwardReach(
		TArray<FLimbLink>& InOutLinks,
		const FVector& InTargetLocation,
		float ReachStepAlpha,
		bool bEnableRotationLimit,
		float MinRotationAngleRadians)
	{
		const int32 NumLinks = InOutLinks.Num();
		
		// Move end effector towards target
		// If we are compressing the chain, limit displacement.
		// Due to how FABRIK works, if we push the target past the parent's joint, we flip the bone.
		{
			FVector EndEffectorToTarget = InTargetLocation - InOutLinks.Last().Location;

			FVector EndEffectorToTargetDir;
			float EndEffectToTargetSize;
			EndEffectorToTarget.ToDirectionAndLength(EndEffectorToTargetDir, EndEffectToTargetSize);

			float Displacement = EndEffectToTargetSize;
			for (int32 Index = NumLinks-1; Index >= 1; Index--)
			{
				FVector EndEffectorToParent = InOutLinks[Index].Location - InOutLinks.Last().Location;
				float ParentDisplacement = static_cast<float>(FVector::DotProduct(EndEffectorToParent, EndEffectorToTargetDir));

				Displacement = (ParentDisplacement > 0.f) ? FMath::Min(Displacement, ParentDisplacement * ReachStepAlpha) : Displacement;
			}

			InOutLinks.Last().Location += EndEffectorToTargetDir * Displacement;
		}

		// "Forward Reaching" stage - adjust bones from end effector.
		for (int32 Index = NumLinks-2; Index > 0; Index--)
		{
			const FLimbLink& ChildLink = InOutLinks[Index+1];
			FLimbLink& CurrentLink = InOutLinks[Index];

			CurrentLink.Location = ChildLink.Location + (CurrentLink.Location - ChildLink.Location).GetUnsafeNormal() * ChildLink.Length;

			if (bEnableRotationLimit)
			{
				FABRIK_ApplyLinkConstraints_Forward(InOutLinks, Index, MinRotationAngleRadians);
			}
		}
	}

	void FABRIK_BackwardReach(
		TArray<FLimbLink>& InOutLinks,
		const FVector& InRootTargetLocation,
		float ReachStepAlpha,
		bool bEnableRotationLimit,
		float MinRotationAngleRadians)
	{
		const int32 NumLinks = InOutLinks.Num();
		
		// Move Root back towards RootTarget
		// If we are compressing the chain, limit displacement.
		// Due to how FABRIK works, if we push the target past the parent's joint, we flip the bone.
		{
			FVector RootToRootTarget = InRootTargetLocation - InOutLinks[0].Location;
			
			FVector RootToRootTargetDir;
			float RootToRootTargetSize;
			RootToRootTarget.ToDirectionAndLength(RootToRootTargetDir, RootToRootTargetSize);

			float Displacement = RootToRootTargetSize;
			
			for (int32 Index = 0; Index < NumLinks-1; Index++)
			{
				FVector RootToChild = InOutLinks[Index].Location - InOutLinks[0].Location;
				float ChildDisplacement = static_cast<float>(FVector::DotProduct(RootToChild, RootToRootTargetDir));
				
				Displacement = (ChildDisplacement > 0.f) ? FMath::Min(Displacement, ChildDisplacement * ReachStepAlpha) : Displacement;				
			}

			InOutLinks[0].Location += RootToRootTargetDir * Displacement;
		}

		// "Backward Reaching" stage - adjust bones from root.
		for (int32 Index = 1; Index < NumLinks-1; Index++)
		{
			const FLimbLink& ParentLink = InOutLinks[Index-1];
			FLimbLink& CurrentLink = InOutLinks[Index];
			CurrentLink.Location = ParentLink.Location + (CurrentLink.Location - ParentLink.Location).GetUnsafeNormal() * CurrentLink.Length;

			if (bEnableRotationLimit)
			{
				FABRIK_ApplyLinkConstraints_Backward(InOutLinks, Index, MinRotationAngleRadians);
			}
		}
	}
	
	void SolveFABRIK(
		TArray<FLimbLink>& InOutLinks,
		const FVector& InTargetLocation,
		float InReachPrecision,
		int32 InMaxIterations,
		bool bUseAngleLimit,
		float MinRotationAngleRadians,
		float ReachStepAlpha,
		float PullDistribution,
		bool bAveragePull)
	{
		const int32 NumLinks = InOutLinks.Num();
		
		// Make sure precision is not too small.
		const float ReachPrecision = FMath::Max(InReachPrecision, UE_KINDA_SMALL_NUMBER);

		const FVector RootTargetLocation = InOutLinks[0].Location;

		// Check distance between foot and foot target location
		float Slop = static_cast<float>(FVector::Dist(InOutLinks.Last().Location, InTargetLocation));
		
		if (Slop > ReachPrecision || bUseAngleLimit)
		{
			if (bUseAngleLimit)
			{
				// Since we've previously aligned the foot with the IK Target, we're solving IK in 2D space on a single plane.
				// Find Plane Normal, to use in rotation constraints.
				const FVector PlaneNormal = FindPlaneNormal(InOutLinks, RootTargetLocation, InTargetLocation);

				for (int32 Index = NumLinks-2; Index > 0; Index--)
				{
					const FLimbLink& ChildLink = InOutLinks[Index + 1];
					FLimbLink& CurrentLink = InOutLinks[Index];
					const FLimbLink& ParentLink = InOutLinks[Index - 1];

					const FVector ChildAxisX = (ChildLink.Location - CurrentLink.Location).GetSafeNormal();
					const FVector ChildAxisY = FVector::CrossProduct(PlaneNormal, ChildAxisX);
					const FVector ParentAxisX = (ParentLink.Location - CurrentLink.Location).GetSafeNormal();

					// Orient Z, so that ChildAxisY points 'up' and produces positive Sin values.
					CurrentLink.LinkAxisZ = FVector::DotProduct(ParentAxisX, ChildAxisY) > 0.f ? PlaneNormal : -PlaneNormal;
				}
			}

			// Re-position limb to distribute pull
			const FVector EndToTarget = InTargetLocation - InOutLinks.Last().Location;
			const FVector RootToTarget = RootTargetLocation - InOutLinks[0].Location;
			const FVector PullDistributionOffset = PullDistribution * (EndToTarget) + (1.f - PullDistribution) * (RootToTarget);
			for (int32 Index = 0; Index < NumLinks; Index++)
			{
				InOutLinks[Index].Location += PullDistributionOffset;
			}
			
			int32 IterationCount = 1;
			const int32 MaxIterations = FMath::Max(InMaxIterations, 1);
			do
			{
				const float PreviousSlop = Slop;

				// Pull averaging only has a visual impact when we have more than 2 bones (3 links).
				const bool bNeedPullAveraging =(NumLinks > 3) && bAveragePull && (Slop > 1.f);
				if (bNeedPullAveraging)
				{	
					TArray<FLimbLink> ForwardPull = InOutLinks;
					FABRIK_ForwardReach(ForwardPull, InTargetLocation, ReachStepAlpha, bUseAngleLimit, MinRotationAngleRadians);

					TArray<FLimbLink> BackwardPull = InOutLinks;
					FABRIK_BackwardReach(BackwardPull,RootTargetLocation, ReachStepAlpha, bUseAngleLimit, MinRotationAngleRadians);

					// Average pulls
					for (int32 LinkIndex = 0; LinkIndex < NumLinks; LinkIndex++)
					{
						InOutLinks[LinkIndex].Location = 0.5f * (ForwardPull[LinkIndex].Location + BackwardPull[LinkIndex].Location);
					}
				}
				else
				{
					FABRIK_ForwardReach(InOutLinks, InTargetLocation, ReachStepAlpha, bUseAngleLimit, MinRotationAngleRadians);
					FABRIK_BackwardReach(InOutLinks, RootTargetLocation, ReachStepAlpha, bUseAngleLimit, MinRotationAngleRadians);
				}
				
				Slop = static_cast<float>(FVector::Dist( InOutLinks.Last().Location, InTargetLocation) + FVector::Dist(InOutLinks[0].Location, RootTargetLocation));

				// Abort if we're not getting closer and enter a deadlock.
				if (Slop > PreviousSlop)
				{
					break;
				}

			} while ((Slop > ReachPrecision) && (++IterationCount < MaxIterations));
			
			// Make sure our root is back at our root target.
			if (!InOutLinks[0].Location.Equals(RootTargetLocation))
			{
				FABRIK_BackwardReach(InOutLinks, RootTargetLocation, ReachStepAlpha, bUseAngleLimit, MinRotationAngleRadians);
			}

			// Replace end bone based on how close we got to the end target.
			FLimbLink& EndLink = InOutLinks[NumLinks-1];
			const FLimbLink& ParentLink = InOutLinks[NumLinks-2];
			EndLink.Location = ParentLink.Location + (EndLink.Location - ParentLink.Location).GetUnsafeNormal() * EndLink.Length;
		}
	}
	
	bool SolveLimb(
		TArray<FLimbLink>& InOutLinks,
		float LimbLength,
		const FVector& GoalLocation,
		float Precision,
		int32 InMaxIterations,
		const FVector& HingeRotationAxis,
		bool bUseAngleLimit,
		float MinRotationAngle,
		float ReachStepAlpha,
		bool bAveragePull,
		float PullDistribution
		)
	{
		const FVector& EndLocation = InOutLinks.Last().Location;
		const bool FootAtGoal = EndLocation.Equals(GoalLocation, Precision);
		
		if (FootAtGoal && !bUseAngleLimit)
		{
			return false;
		}
		
		const bool bDirectSolve = ( InOutLinks.Num() == 3 );
		if (bDirectSolve)
		{
			const FVector& RootLocation = InOutLinks[0].Location;
			
			// If we can't reach, we just go in a straight line towards the target,
			if (FVector::DistSquared(RootLocation, GoalLocation) >= FMath::Square(LimbLength))
			{
				SolveStraightLimb(InOutLinks, GoalLocation);
			}
			else
			{
				SolveTwoBoneIK(InOutLinks, GoalLocation, HingeRotationAxis);
			}
		}
		else
		{
			const float MinRotationAngleRadians = FMath::DegreesToRadians(FMath::Clamp(MinRotationAngle, 0.f, 90.f));
			SolveFABRIK(InOutLinks, GoalLocation, Precision, InMaxIterations, bUseAngleLimit, MinRotationAngleRadians, ReachStepAlpha, PullDistribution, bAveragePull);
		}
		
		return true;
	}
}
