// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"

#include "Rig/IKRigDataTypes.h"
#include "IKRigInterface.h"

#include "IKRigComponent.generated.h"


UCLASS(MinimalAPI, ClassGroup = IKRig, BlueprintType, Blueprintable, EditInlineNew, meta = (BlueprintSpawnableComponent))
class UIKRigComponent : public UActorComponent, public IIKGoalCreatorInterface
{
	GENERATED_BODY()

public:

	/** Set an IK Rig Goal position and rotation (assumed in Component Space of Skeletal Mesh) with separate alpha values. */
	UFUNCTION(BlueprintCallable, Category=IKRigGoals)
    void SetIKRigGoalPositionAndRotation(
        const FName GoalName,
        const FVector Position,
        const FQuat Rotation,
        const float PositionAlpha,
        const float RotationAlpha)
	{
		GoalContainer.SetIKGoal(
			FIKRigGoal(
				GoalName,
				NAME_None,
				Position,
				Rotation,
				PositionAlpha,
				RotationAlpha,
				EIKRigGoalSpace::Component,
				EIKRigGoalSpace::Component,
				true));
	};

	/** Set an IK Rig Goal transform (assumed in Component Space of Skeletal Mesh) with separate alpha values. */
	UFUNCTION(BlueprintCallable, Category=IKRigGoals)
    void SetIKRigGoalTransform(
        const FName GoalName,
        const FTransform Transform,
        const float PositionAlpha,
        const float RotationAlpha)
	{
		const FVector Position = Transform.GetTranslation();
		const FQuat Rotation = Transform.GetRotation();
		GoalContainer.SetIKGoal(
			FIKRigGoal(
				GoalName,
				NAME_None,
				Position,
				Rotation,
				PositionAlpha,
				RotationAlpha,
				EIKRigGoalSpace::Component,
				EIKRigGoalSpace::Component,
				true));
	};
	
	/** Apply a IKRigGoal and store it on this rig. Goal transform assumed in Component Space of Skeletal Mesh. */
	UFUNCTION(BlueprintCallable, Category=IKRigGoals)
    void SetIKRigGoal(const FIKRigGoal& Goal)
	{
		GoalContainer.SetIKGoal(Goal);
	};

	/** Remove all stored goals in this component. */
	UFUNCTION(BlueprintCallable, Category=IKRigGoals)
    void ClearAllGoals(){ GoalContainer.Empty(); };

	// BEGIN IIKRigGoalCreator interface
	/** Called from the IK Rig Anim node to obtain all the goals that have been set on this actor.*/
	virtual void AddIKGoals_Implementation(TMap<FName, FIKRigGoal>& OutGoals) override
	{
		const TArray<FIKRigGoal>& Goals = GoalContainer.GetGoalArray();
		for (const FIKRigGoal& Goal : Goals)
		{
			OutGoals.Add(Goal.Name, Goal);
		}
	};
	// END IIKRigGoalCreator interface

private:
	FIKRigGoalContainer GoalContainer;
};
