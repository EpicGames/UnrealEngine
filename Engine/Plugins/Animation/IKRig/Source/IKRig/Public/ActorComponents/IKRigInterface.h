// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/IKRigDataTypes.h"
#include "UObject/Interface.h"

#include "IKRigInterface.generated.h"

UINTERFACE(MinimalAPI, Blueprintable)
class UIKGoalCreatorInterface : public UInterface
{
	GENERATED_BODY()
};

class IIKGoalCreatorInterface
{    
	GENERATED_BODY()

public:
	
	/** Add your own goals to the OutGoals map (careful not to remove existing goals in the map!)*/
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category=IKRigGoals)
    void AddIKGoals(TMap<FName, FIKRigGoal>& OutGoals);
};
