// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Rig/IKRigDataTypes.h"
#include "Animation/AnimNode_CustomProperty.h"
#include "Animation/InputScaleBias.h"
#include <functional>

#include "Rig/IKRigProcessor.h"
#include "AnimNode_IKRig.generated.h"

#define UE_API IKRIG_API

class IIKGoalCreatorInterface;
class UIKRigDefinition;

	
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_IKRig : public FAnimNode_CustomProperty
{
	GENERATED_BODY()

	UE_API FAnimNode_IKRig();
	
	/** The input pose to start the IK solve relative to. */
	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink Source;

	/** The IK rig to use to modify the incoming source pose. */
	UPROPERTY(EditAnywhere, Category = RigDefinition, meta = (NeverAsPin))
	TObjectPtr<UIKRigDefinition> RigDefinitionAsset = nullptr;

	/** The input goal transforms used by the IK Rig solvers.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Goal, meta=(NeverAsPin))
	TArray<FIKRigGoal> Goals;

	/** optionally ignore the input pose and start from the reference pose each solve */
	UPROPERTY(EditAnywhere, Category = Solver)
	bool bStartFromRefPose = false;

#if WITH_EDITORONLY_DATA

	/** when true, goals will use the current transforms stored in the IK Rig Definition asset itself */
	bool bDriveWithSourceAsset = false;
	
	/** Toggle debug drawing of goals when node is selected.*/
	UPROPERTY(EditAnywhere, Category = Solver)
	bool bEnableDebugDraw = false;

	/** Adjust size of debug drawing.*/
	UPROPERTY(EditAnywhere, Category = Solver)
	float DebugScale = 5.0f;
	
#endif

	/** alpha value handler **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	EAnimAlphaInputType AlphaInputType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault, DisplayName = "bEnabled"))
	bool bAlphaBoolEnabled;

	// Current strength of the skeletal control
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault))
	float Alpha;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	FInputScaleBias AlphaScaleBias;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (DisplayName = "Blend Settings"))
	FInputAlphaBoolBlend AlphaBoolBlend;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha, meta = (PinShownByDefault))
	FName AlphaCurveName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Alpha)
	FInputScaleBiasClamp AlphaScaleBiasClamp;
	
public:

	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)  override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	virtual bool HasPreUpdate() const override { return true; }
	UE_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	virtual bool NeedsOnInitializeAnimInstance() const override { return true; }
	UE_API virtual void OnInitializeAnimInstance(const FAnimInstanceProxy* InProxy, const UAnimInstance* InAnimInstance) override;
	// End of FAnimNode_Base interface

	// force reinitialization
	UE_API void SetProcessorNeedsInitialized();
	
	UE_API FIKRigProcessor* GetIKRigProcessor();

private:

	/** IK Rig runtime processor */
	FIKRigProcessor IKRigProcessor;

	/** a cached list of components on the owning actor that implement the goal creator interface */
	UPROPERTY(transient)
	TArray<TObjectPtr<UActorComponent>> GoalCreators;
	TMap<FName, FIKRigGoal> GoalsFromGoalCreators;
	bool bSearchedForGoalCreators = false;
	
	TMap<FCompactPoseBoneIndex, int32, FDefaultSetAllocator, TCompactPoseBoneIndexMapKeyFuncs<int32>> CompactPoseToRigIndices;

	/** Cached functions used to update goals using custom properties to avoid looking for then when evaluating */
	using PropertyUpdateFunction = TFunction<void(const UObject*)>;
	TArray<PropertyUpdateFunction> UpdateFunctions;
	
	float ActualAlpha;
	
	UE_API void CopyInputPoseToSolver(FCompactPose& InputPose);
	UE_API void AssignGoalTargets();
	UE_API void CopyOutputPoseToAnimGraph(FCompactPose& OutputPose);
#if WITH_EDITOR
	UE_API virtual void ConditionalDebugDraw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const;
#endif

protected:

	// FAnimNode_CustomProperty interface
	virtual UClass* GetTargetClass() const override {return nullptr;}
	UE_API virtual void InitializeProperties(const UObject* InSourceInstance, UClass* InTargetClass) override;
	UE_API virtual void PropagateInputProperties(const UObject* InSourceInstance) override;
	
	
	friend class UAnimGraphNode_IKRig;
};

#undef UE_API
