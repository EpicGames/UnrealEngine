// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Animation/AnimNodeBase.h"
#include "Animation/AnimNode_CustomProperty.h"
#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Retargeter/IKRetargetProfile.h"

#include "AnimNode_RetargetPoseFromMesh.generated.h"

#define UE_API IKRIG_API

DECLARE_CYCLE_STAT(TEXT("IK Retarget"), STAT_IKRetarget, STATGROUP_Anim);

UENUM(BlueprintType)
enum class ERetargetSourceMode : uint8
{
	ParentSkeletalMeshComponent,
	CustomSkeletalMeshComponent,
	SourcePosePin
};

USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RetargetPoseFromMesh : public FAnimNode_CustomProperty
{
	GENERATED_BODY()
	
	// Input pose to be modified by the retargeter when using "Source Pose Pin" mode as the Input Pose Mode.
	UPROPERTY(EditAnywhere, Category = Links)
	FPoseLink Source;
	
	// Specify where to get the source pose to retarget from. Can be from the anim graph, or a different skeletal mesh component.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta=(NeverAsPin))
	ERetargetSourceMode RetargetFrom = ERetargetSourceMode::ParentSkeletalMeshComponent;

	// The Skeletal Mesh Component to retarget animation from. Assumed to be animated and tick BEFORE this anim instance.
	UPROPERTY(BlueprintReadWrite, transient, Category = Settings, meta=(PinShownByDefault))
	TWeakObjectPtr<USkeletalMeshComponent> SourceMeshComponent = nullptr;
	
	// Retarget asset to use. Must define a Source and Target IK Rig compatible with the SourceMeshComponent and current anim instance.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta=(PinHiddenByDefault))
	TObjectPtr<UIKRetargeter> IKRetargeterAsset = nullptr;

	// Connect a custom retarget profile to modify the retargeter's settings at runtime.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta=(PinHiddenByDefault))
	FRetargetProfile CustomRetargetProfile;

	// If true, will use the custom list of named override sets to apply. These are stored in the retarget asset.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta=(PinHiddenByDefault))
	bool bUseCustomOverrideSets = false;

	/** Apply one or more override sets stored in the retarget asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings, meta=(EditCondition="bUseCustomOverrideSets", PinHiddenByDefault))
	TArray<FName> OverrideSetsToApply;
	
	// Max LOD that this node is allowed to run.
	// For example if you have LODThreshold at 2, it will run until LOD 2 (based on 0 index) when the component LOD becomes 3, it will stop update/evaluate
	// A value of -1 forces the node to execute at all LOD levels.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (DisplayName = "LOD Threshold"))
	int32 LODThreshold = -1;
	
	// Max LOD that IK is allowed to run.
	// For example if you have LODThresholdForIK at 2, it will skip the IK pass on LODs 3 and greater.
	// This only disables IK and does not affect the Root or FK passes.
	// A value of -1 forces the node to execute at all LOD levels. 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Performance, meta = (DisplayName = "IK LOD Threshold"))
	int32 LODThresholdForIK = -1;

	// Toggle whether to print warnings about missing or incorrectly configured retarget configurations.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Debug, meta = (NeverAsPin))
	bool bSuppressWarnings = false;

	UPROPERTY()
	bool bUseAttachedParent_DEPRECATED = true;
	
	// FAnimNode_Base interface
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	virtual bool HasPreUpdate() const override { return true; }
	UE_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	virtual int32 GetLODThreshold() const override { return LODThreshold; }
	// End of FAnimNode_Base interface

	// FAnimNode_CustomProperty interface
	virtual UClass* GetTargetClass() const override {return nullptr;};
	// END FAnimNode_CustomProperty interface

	// access to the runtime processor
	UE_API FIKRetargetProcessor* GetRetargetProcessor();

	// returns true if processor is setup and ready to go, false otherwise
	UE_API bool EnsureProcessorIsInitialized(const USkeletalMeshComponent* TargetMeshComponent);

	// load deprecated properties
	UE_API bool Serialize(FArchive& Ar);
	UE_API void PostSerialize(const FArchive& Ar);

#if WITH_EDITOR
	UE_API double GetAverageExecutionTime() const;
#endif

private:
	
	// the runtime storage for variable values
	UPROPERTY(Transient)
	FInstancedPropertyBag RuntimeVariables;

	// a map of curves bound to property overrides (Name, float)
	UPROPERTY(Transient)
	TMap<FName, float> BoundCurveValues;
	int32 CachedOverrideVersion = INDEX_NONE;

	// pulls runtime values of variables out of anim instance and stores in RuntimeVariables
	void CopyInputPropertiesToVariables(const UObject* InSourceInstance);

	// fills BoundCurves from the anim instance
	void CopyBoundCurveValues(const UObject* InSourceInstance);
	
	// get a retarget profile that includes settings from the asset, plus any custom profile overrides
	UE_API FRetargetProfile GetMergedRetargetProfile(bool bEnableIK) const;

	// the runtime processor used to run the retarget and generate new poses
	FIKRetargetProcessor Processor;

	// cached transforms, copied on the game thread
	TArray<FTransform> PoseToRetargetFromComponentSpace;

	// reusable buffer when retargeting a pose from the anim graph pose pin
	TArray<FTransform> InputLocalTransforms;

	// mapping from required compact bone indices to target mesh bones expected by the retargeter
	TArray< TPair<int32, int32> > CompactToTargetBoneIndexMap;
	
	// the delta time this tick
	float DeltaTime;

	// used to determine when to look for a parent component (prevents constantly searching component hierarchy)
	bool bSearchedForParentComponent = false;

#if WITH_EDITOR
	double AverageExecutionTime;
#endif
};

template<>
struct TStructOpsTypeTraits<FAnimNode_RetargetPoseFromMesh> : public TStructOpsTypeTraitsBase2<FAnimNode_RetargetPoseFromMesh>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true,
	};
};

#undef UE_API
