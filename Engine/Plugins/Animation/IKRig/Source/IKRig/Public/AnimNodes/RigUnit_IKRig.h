// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/IKRigProcessor.h"
#include "Units/Highlevel/RigUnit_HighlevelBase.h"

#include "RigUnit_IKRig.generated.h"

class UIKRigDefinition;

#define UE_API IKRIG_API

USTRUCT()
struct FIKRigWorkData
{
	GENERATED_BODY()
	
	FIKRigWorkData() {}
	
	TArray<FCachedRigElement> CachedRigElements;
	
	FIKRigProcessor IKRigProcessor;
};

USTRUCT(BlueprintType)
struct FIKRigGoalInput
{
	GENERATED_BODY()
	
	FIKRigGoalInput() :
		GoalName(NAME_None),
		Transform(FTransform::Identity),
		PositionAlpha(1.0),
		RotationAlpha(1.0)
	{
	}

	/** The name of the goal to affect. Must match the name in the IK Rig. */
	UPROPERTY(EditAnywhere, Category=GoalSettings)
	FName GoalName;

	/** The position and rotation target for the IK Goal. */
	UPROPERTY(EditAnywhere, Category=GoalSettings)
	FTransform Transform;

	/** Range 0-1, default 1. Blends the Goal position from the input pose position to the Goal position. */
	UPROPERTY(EditAnywhere, Category=GoalSettings)
	double PositionAlpha;

	/** Range 0-1, default 1. Blends the Goal rotation from the input pose rotation to the Goal rotation. */
	UPROPERTY(EditAnywhere, Category=GoalSettings)
	double RotationAlpha;
};

/* Supply an IK Rig asset and provide goal transforms to run IK on the skeleton. */
USTRUCT(meta=(DisplayName="IK Rig", Category="Hierarchy", Keywords="IK Rig, IK", NodeColor="0 1 1"))
struct FRigUnit_IKRig : public FRigUnit_HighlevelBaseMutable
{
	GENERATED_BODY()

	FRigUnit_IKRig() {}

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	/** An IK Rig asset to be evaluated. */
	UPROPERTY(meta = (Input))
	TObjectPtr<UIKRigDefinition> IKRigAsset;

	/** A list of Goals to solve. These must match what is in the IK Rig, any missing Goals will have their alphas set to zero. */
	UPROPERTY(meta = (Input))
	TArray<FIKRigGoalInput> Goals;

	UPROPERTY(transient)
	FIKRigWorkData WorkData;

private:
	
	static void CacheRigElements(TArray<FCachedRigElement>& OutMap, const TArray<FName>& InBoneNames, URigHierarchy* InHierarchy);
};

#undef UE_API