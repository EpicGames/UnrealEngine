// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

// a UProperty metadata tag used to indicate that a property should trigger reinitialization when edited
static const TCHAR* IKRigReinitOnEditMetaLabel = TEXT("ReinitializeOnEdit");
// a UProperty metadata tag used to indicate that a property can not be overridden by a profile
static const TCHAR* IKRigNonOverrideableMetaLabel = TEXT("NotOverrideable");

class FIKRigModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
