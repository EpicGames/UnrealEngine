// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneManagement.h"

namespace IKRigDebugRendering
{
	// 8 vertices of cube
	static TArray CubeVertices =
	{
		FVector(-0.5f, 0.5f, -0.5f),
		FVector(0.5f, 0.5f, -0.5f),
		FVector(0.5f, 0.5f, 0.5f),
		FVector(-0.5f, 0.5f, 0.5f),
		
		FVector(-0.5f, -0.5f, -0.5f),
		FVector(0.5f, -0.5f, -0.5f),
		FVector(0.5f, -0.5f, 0.5f),
		FVector(-0.5f, -0.5f, 0.5f),
	};

	// 12 edges of cube
	static TArray CubeEdges =
	{
		// top
		0,1,
		1,2,
		2,3,
		3,0,

		// bottom
		4,5,
		5,6,
		6,7,
		7,4,

		// sides
		0,4,
		1,5,
		2,6,
		3,7
	};

	static void DrawWireCube(
		FPrimitiveDrawInterface* PDI,
		const FTransform& Transform,
		FLinearColor Color,
		float Size,
		float Thickness)
	{
		const float Scale = FMath::Clamp(Size, 0.01f, 1000.0f);
		for (int32 EdgeIndex = 0; EdgeIndex < CubeEdges.Num() - 1; EdgeIndex += 2)
		{
			PDI->DrawLine(
				Transform.TransformPosition(CubeVertices[CubeEdges[EdgeIndex]] * Scale),
				Transform.TransformPosition(CubeVertices[CubeEdges[EdgeIndex + 1]] * Scale),
				Color,
				SDPG_Foreground,
				Thickness);
		}
	}
}
