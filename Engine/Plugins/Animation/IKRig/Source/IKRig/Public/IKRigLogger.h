// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

#define UE_API IKRIG_API

DECLARE_LOG_CATEGORY_EXTERN(LogIKRig, Warning, All);

struct FIKRigLogger
{
	/** Assign an asset for this log to store messages for. */
	UE_API void SetLogTarget(const UObject* InAsset);
	/** Get the name this log is currently outputting to */
	UE_API FName GetLogTarget() const;
	/** Log a warning message to display to user. */
	UE_API void LogError(const FText& Message) const;
	/** Log a warning message to display to user. */
	UE_API void LogWarning(const FText& Message) const;
	/** Log a message to display to editor output log. */
	UE_API void LogInfo(const FText& Message) const;
	/** clear all the stored messages */
	UE_API void Clear() const;
	/** get a list of messages that have been logged since last Clear() */
	const TArray<FText>& GetErrors() const { return Errors; };
	const TArray<FText>& GetWarnings() const { return Warnings; };
	const TArray<FText>& GetMessages() const { return Messages; };

private:
	/** the name of the output log this logger will send messages to
	 *
	 * For the IK Rig and Retargeting editors, we desire to filter the messages that originate only from the asset
	 * that is being edited. Therefore we name the log using the unique ID of the UObject itself (valid for lifetime of UObject between loads)
	 */
	FName LogName;

	/** store messages here so we can show them during anim graph compilation */
	mutable TArray<FText> Errors;
	mutable TArray<FText> Warnings;
	mutable TArray<FText> Messages;
};

#undef UE_API
