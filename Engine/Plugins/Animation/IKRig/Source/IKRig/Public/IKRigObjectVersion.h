// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

#define UE_API IKRIG_API

// Custom serialization version for backwards compatibility during de-serialization
struct FIKRigObjectVersion
{
	FIKRigObjectVersion() = delete;
	
	enum Type
	{
		// Before any version changes were made
		BeforeCustomVersionWasAdded,

		// Retarget pose quaternions changed from pre to post multiplied
		RetargetPoseQuatPostMultiplied,

		// Chain settings moved to struct to be used in profiles
		ChainSettingsConvertedToStruct,
		
		// Pin Bone "Pin Type" converted into separate trans/rot/scale toggles
		// Pin Bone "Maintain Offset" converted from bool to enum
		PinBoneTypeAndOffsetsUpgraded,

		// we switched "Use Attach Parent" from a bool to an enum on retarget pose from mesh node
		UseAttachedParentDeprecated,

		// modularized retargeting internals (converted phases to ops)
		ModularRetargeterOps,

		// ops control their own chain mapping (no more global chain map on retarget asset)
		OpsOwnChainMapping,

		// copy base pose op converted to use FBoneReference 
		CopyBasePoseConvertedToBoneReference,
		
		// changed default behavior of BlendToSource on IK Goals to not be affected by Pelvis motion
		RemovedPelvisMotionFromSourceGoals,

		// moved chain mapping in FKChainsOp from Op to Op-settings for read-only API access
		MovedChainMappingToFKChainOpSettings,

		// added "bEnableDebugDraw" to base op settings
		AddedDebugDrawTogglePerOp,

		// added local/global offsets to pelvis motion op
		AddedLocalGlobalOffsetsToPelvisOp,

		// the "Retarget IK Goals" op is deprecated and features are split into separate ops
		IKChainsOpSplitIntoSeparateOps,

		// IK Rig retarget definition now stores separate root bone and pelvis bone
		AddedRootBoneToRetargetDefinition,

		// the op settings now contain bEnabled instead of the ops themselves
		EnabledStateMovedFromOpToOpSettings,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;
};

#undef UE_API
