// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IKRetargetChainMapping.generated.h"

#define UE_API IKRIG_API

class UIKRigDefinition;
class URetargetChainSettings;
enum class ERetargetSourceOrTarget : uint8;

UENUM()
enum class EAutoMapChainType : uint8
{
	Exact, // map chains that have exactly the same name (case insensitive)
	Fuzzy, // map chains to the chain with the closest name (levenshtein distance)
	Clear, // clear all mappings, set them to None
};

USTRUCT()
struct FRetargetChainPair
{
	GENERATED_BODY()

	FRetargetChainPair() = default;
	FRetargetChainPair(const FName InTargetChainName, const FName InSourceChainName) :
		TargetChainName(InTargetChainName), SourceChainName(InSourceChainName) {}

	UPROPERTY()
	FName TargetChainName = NAME_None;
	
	UPROPERTY()
	FName SourceChainName = NAME_None;
};

USTRUCT()
struct FRetargetChainMapping
{
	GENERATED_BODY()

public:

	/* cleans the chain mapping to reflect the chains in the provided source/target IK Rigs
	 * NOTE: this will maintain existing mappings while removing unused ones and adding missing ones  */
	UE_API void ReinitializeWithIKRigs(const UIKRigDefinition* InSourceIKRig, const UIKRigDefinition* InTargetIKRig);

	/** returns true if both a source and target IK Rig have been loaded */
	UE_API bool IsReady() const;

	/** returns true if any chains have been loaded, regardless of if they are mapped */
	UE_API bool HasAnyChains() const;

	/* get one of the IK Rigs used in this mapping */
	UE_API const UIKRigDefinition* GetIKRig(ERetargetSourceOrTarget SourceOrTarget);

	/** Check if the chain exists.
	 * @param InChainName the name of either a source or target chain
	 * @param InSourceOrTarget the side that InChainName belongs to (either source or target)
	 * @return true if the chain is present in the mapping */
	UE_API bool HasChain(const FName& InChainName, ERetargetSourceOrTarget InSourceOrTarget) const;

	/** Get the name of the bone chain mapped to this chain.
	 * @param InChainName the name of either a source or target chain
	 * @param InSourceOrTarget the side that InChainName belongs to (either source or target)
	 * @return the name of the chain that InChainName is mapped to (possibly NAME_None) */
	UE_API FName GetChainMappedTo(const FName InChainName, ERetargetSourceOrTarget InSourceOrTarget) const;

	/** returns the chain pair belonging to the supplied chain or nullptr if not found
	* @param InChainName the name of either a source or target chain
	* @param InSourceOrTarget the side that InChainName belongs to (either source or target)*/
	UE_API FRetargetChainPair* FindChainPair(const FName& InChainName, ERetargetSourceOrTarget InSourceOrTarget) const;
	UE_API const FRetargetChainPair* FindChainPairConst(const FName& InChainName, ERetargetSourceOrTarget InSourceOrTarget) const;

	/** Get read-write access to the mapping from target to source chains (by name) */
	UE_API TArray<FRetargetChainPair>& GetChainPairs();
	UE_API const TArray<FRetargetChainPair>& GetChainPairs() const;

	/** Get the names of all the chains in either the source or target IK Rig */
	UE_API TArray<FName> GetChainNames(ERetargetSourceOrTarget InSourceOrTarget) const;

	/** Map target chain to a source chain.
	 * @param InTargetChainName the name of a target chain
	 * @param InSourceChainName the name of a source chain*/
	UE_API void SetChainMapping(const FName InTargetChainName, const FName InSourceChainName);

	/* sort mapping hierarchically (root to leaf based on Target IK Rig hierarchy) */
	UE_API void SortMapping();

	/* automatically map the source and target chains together */
	UE_API void AutoMapChains(const EAutoMapChainType AutoMapType, const bool bForceRemap);
	
	UE_API PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void LoadFromDeprecatedChainSettings(const TArray<URetargetChainSettings*>& InChainSettings);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
	
	/** mapping of target to source bone chains by name
	 * NOTE: this is an array instead of a TMap because it needs to be sorted by hierarchy*/
	UPROPERTY()
	TArray<FRetargetChainPair> ChainMap;

	UPROPERTY()
	TObjectPtr<const UIKRigDefinition> SourceIKRig = nullptr;

	UPROPERTY()
	TObjectPtr<const UIKRigDefinition> TargetIKRig = nullptr;
};

#undef UE_API
