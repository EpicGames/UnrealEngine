// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IKRetargetOps.h"
#include "IKRetargetSettings.h"
#include "Chaos/Character/CharacterGroundConstraintContainer.h"
#include "Rig/IKRigDefinition.h"
#include "Retargeter/IKRetargetChainMapping.h"
#include "RetargetOps/RunIKRigOp.h"

struct FIKRetargetRunIKRigOp;

namespace IKRetargetOpUtils
{
	/**
	 * Synchronizes a custom chain settings array with the chains in an IK Rig asset.
	 *
	 * Convenience function for retarget ops that need to keep an array of custom chain
	 * settings in sync with the chains defined in an IK Rig asset.
	 *
	 * @tparam T  Custom struct type representing chain settings.
	 *
	 * @param InOutChainSettings   Array of chain settings to synchronize.
	 * @param InRetargetOp         Retarget operation providing access to the IK Rig.
	 * @param bSkipUnmappedChains  If true, unmapped chains are ignored.
	 * @param bSkipNonIKChains     If true, chains without IK goals are ignored.
	 *
	 * @note  To use this function, your type T must:
	 *        1. Implement:
	 *           - FName GetName()
	 *           - void SetName(FName InChainName)
	 *        2. Provide a constructor that accepts the chain name.
	 *
	 * @note  This function will early-return if no valid chains are found. It does NOT clear the array,
	 *        allowing users to reassign a different rig and preserve compatible settings.
	 */
	template<typename T>
	void SynchronizeChainSettingsWithIKRig(
		TArray<T>& InOutChainSettings,
		const FIKRetargetOpBase* InRetargetOp,
		const bool bSkipUnmappedChains,
		const bool bSkipNonIKChains)
	{
		// NOTE: this function early returns if there's no valid chains found: we do NOT clear the settings
		// this allows users to clear and reassign a different rig and potentially retain/restore compatible settings
		
		if (!InRetargetOp)
		{
			return;
		}

		const UIKRigDefinition* TargetIKRig = InRetargetOp->GetCustomTargetIKRig();
		if (!TargetIKRig)
		{
			return;
		}

		const TArray<FBoneChain>& TargetChains = TargetIKRig->GetRetargetChains();
		if (TargetChains.IsEmpty())
		{
			return;
		}

		const FRetargetChainMapping* ChainMapping = InRetargetOp->GetChainMapping();
		if (bSkipUnmappedChains && !ChainMapping)
		{
			return;
		}

		// find the target chains that children ops should deal with
		TArray<FName> RequiredTargetChains;
		for (const FBoneChain& TargetChain : TargetChains)
		{
			if (bSkipNonIKChains && TargetChain.IKGoalName == NAME_None)
			{
				continue; // skip non-IK chains
			}

			if (bSkipUnmappedChains && ChainMapping)
			{
				const FName SourceChain = ChainMapping->GetChainMappedTo(TargetChain.ChainName, ERetargetSourceOrTarget::Target);
				if (SourceChain == NAME_None)
				{
					continue; // skip unmapped chains
				}
			}
			
			RequiredTargetChains.Add(TargetChain.ChainName);
		}

		if (RequiredTargetChains.IsEmpty())
		{
			return;
		}

		// remove chains that are not required
		InOutChainSettings.RemoveAll([&RequiredTargetChains](const T& InChainSettings)
		{
			return !RequiredTargetChains.Contains(InChainSettings.GetName());
		});
    	
		// add any required chains not already present
		for (FName RequiredTargetChain : RequiredTargetChains)
		{
			bool bFound = false;
			for (const T& ChainSettings : InOutChainSettings)
			{
				if (ChainSettings.GetName() == RequiredTargetChain)
				{
					bFound = true;
					break;
				}
			}
			if (!bFound)
			{
				InOutChainSettings.Emplace(RequiredTargetChain);
			}
		}
	};

	/**
	 * Responds to an IK Rig chain being renamed by updating the corresponding entry in a settings array.
	 *
	 * Convenience function for retarget ops that maintain custom per-chain settings.
	 *
	 * @tparam T  Custom struct type representing chain settings.
	 *
	 * @param InOutChainSettings  Array of chain settings to update.
	 * @param InOldChainName      Previous name of the chain.
	 * @param InNewChainName      New name of the chain.
	 *
	 * @note  Type T must implement:
	 *        - FName GetName()
	 *        - void SetName(FName InChainName)
	 */
	template<typename T>
	void OnRetargetChainRenamed(TArray<T>& InOutChainSettings, const FName InOldChainName, const FName InNewChainName)
	{
		for (T& ChainSettings : InOutChainSettings)
		{
			if (ChainSettings.GetName() == InOldChainName)
			{
				ChainSettings.SetName(InNewChainName);
			}
		}
	};

	/**
	 * Copies matching chain settings from one array to another at runtime.
	 *
	 * Convenience function for retarget ops that need to copy per-chain settings
	 * between runtime instances of the same operation.
	 *
	 * @tparam T  Custom struct type representing chain settings.
	 *
	 * @param InSettingsToCopyFrom  Source array of chain settings.
	 * @param InSettingsToCopyTo    Destination array to update.
	 *
	 * @note  Only copies settings for chains that exist in both arrays.
	 * @note  Type T must implement:
	 *        - FName GetName()
	 *        - void SetName(FName InChainName)
	 */
	template<typename T>
	void CopyChainSettingsAtRuntime(const TArray<T>& InSettingsToCopyFrom, TArray<T>& InSettingsToCopyTo)
	{
		for (const T& SettingsToCopyFrom : InSettingsToCopyFrom)
		{
			for (T& SettingsToCopyTo : InSettingsToCopyTo)
			{
				if (SettingsToCopyFrom.GetName() == SettingsToCopyTo.GetName())
				{
					SettingsToCopyTo = SettingsToCopyFrom;
					break;
				}
			}
		}
	};

	/**
	 * Resets the settings for a specific IK chain to its default values.
	 *
	 * Convenience function for retarget ops that manage per-chain settings arrays.
	 * Finds the matching chain by name and replaces its settings with a new default-constructed instance.
	 *
	 * @tparam T  Custom struct type representing chain settings.
	 *
	 * @param InChainSettingsArray  Array of chain settings to search and modify.
	 * @param InChainName           Name of the chain to reset.
	 *
	 * @note  Type T must:
	 *        - Implement FName GetName()
	 *        - Provide a constructor that accepts a chain name (FName)
	 *
	 * @note  If no matching chain is found, this function does nothing.
	 */
	template<typename T>
	void ResetChainSettingsToDefault(TArray<T>& InChainSettingsArray, const FName& InChainName)
	{
		for (T& ChainSettings : InChainSettingsArray)
		{
			if (ChainSettings.GetName() == InChainName)
			{
				// reset
				ChainSettings = T(InChainName);
				return;
			}			
		}
	}

	/**
	 * Checks whether the settings for a specific chain are at their default values.
	 *
	 * Generic utility function for retarget ops that manage per-chain settings arrays.
	 * Compares the specified chain’s current settings to a default-constructed instance.
	 *
	 * @tparam T  Custom struct type representing chain settings.
	 *
	 * @param InChainSettingsArray  Array of chain settings to search.
	 * @param InChainName           Name of the chain to compare.
	 *
	 * @return True if the chain’s settings match the default values, or if the chain is not found.
	 *
	 * @note  Type T must:
	 *        - Implement FName GetName()
	 *        - Provide a constructor that accepts a chain name (FName)
	 *        - Implement operator== for equality comparison
	 */
	template<typename T>
	bool AreChainSettingsAtDefault(const TArray<T>& InChainSettingsArray, const FName& InChainName)
	{
		for (const T& ChainSettings : InChainSettingsArray)
		{
			if (ChainSettings.GetName() == InChainName)
			{
				const T DefaultSettings(InChainName);
				return ChainSettings == DefaultSettings;
			}
		}

		return true;
	}

	/**
	 * Returns a raw pointer to the memory of a specific chain’s settings instance.
	 *
	 * @tparam T  Custom struct type representing chain settings.
	 *
	 * @param InChainSettingsArray  Array of chain settings to search.
	 * @param InChainName           Name of the chain whose settings memory address should be returned.
	 *
	 * @return Pointer to the chain’s settings in memory, or nullptr if the chain is not found.
	 *
	 * @note  Type T must implement:
	 *        - FName GetName()
	 *
	 * @warning  The returned pointer refers to memory owned by the array. It becomes invalid if
	 *           the array is modified or reallocated. Use with caution.
	 */
	template<typename T>
	uint8* GetChainSettingsMemory(TArray<T>& InChainSettingsArray, const FName& InChainName)
	{
		for (T& ChainSettings : InChainSettingsArray)
		{
			if (ChainSettings.GetName() == InChainName)
			{
				return reinterpret_cast<uint8*>(&ChainSettings);
			}
		}

		return nullptr;
	}
}
