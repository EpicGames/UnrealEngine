// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IKRigObjectVersion.h"

#include "IKRetargetOps.generated.h"

#define UE_API IKRIG_API

class UIKRigDefinition;
struct FIKRetargetProcessor;
struct FRetargetSkeleton;
struct FTargetSkeleton;
struct FIKRigLogger;
class UIKRetargeter;
struct FInstancedStruct;
class USkeleton;
class UAnimSequence;
struct FPoseContext;
struct FRetargetChainMapping;
enum class ERetargetSourceOrTarget : uint8;
struct FRetargetPropertyOverrideNode;

#if WITH_EDITOR
class FPrimitiveDrawInterface;
class FIKRetargetEditorController;
struct FIKRetargetDebugDrawState;
class IPropertyHandle;
#endif

/*
 * Because the settings structs in the retarget op stack are not UObjects, we cannot use the IBoneReferenceSkeletonProvider interface
 * Instead, if Op settings require a FBoneReference with a bone selector widget then the settings must inherit either directly from this
 * (in the case of sub structs), or indirectly from the FIKRetargetOpSettingBase for the op's main settings struct.
 **/
USTRUCT(BlueprintType)
struct FIKRetargetOpSkeletonProvider
{
	GENERATED_BODY()
	
	virtual ~FIKRetargetOpSkeletonProvider() {};

#if WITH_EDITORONLY_DATA
public:
	/** a reference to the skeletons for bone selector widgets (can return these from GetSkeleton) */
	mutable const USkeleton* SourceSkeletonAsset;
	mutable const USkeleton* TargetSkeletonAsset;
	/** allow settings to provide a skeleton for any given FBoneReference widget. Defaults to the target skeleton. */
	virtual USkeleton* GetSkeleton(const FName InPropertyName) { return const_cast<USkeleton*>(TargetSkeletonAsset); };
	/** optional alternative to GetSkeleton that provides the full PropertyHandle. This is used if GetSkeleton(const FName) returns null
	 * NOTE: the property handle is necessary to help differentiate properties in copies of the same struct type */ 
	virtual USkeleton* GetSkeleton(const IPropertyHandle* InPropertyHandle) { return const_cast<USkeleton*>(TargetSkeletonAsset); };
#endif
};

/** This is the base class for defining editable settings for a custom retargeting operation.
 * All user-configurable properties for an "op" should be stored within a single instance of a subclass of this type.
 * These settings will automatically be:
 * 1. Displayed in the details panel when the op is selected
 * 2. Saved/loaded with the op in the retarget asset
 * 3. Applied to the op at runtime as part of a profile
 * 
 * NOTE: the derived type must be returned by the op's GetSettingsType() and GetSettings()
 * 
 * NOTE: UProperties that require reinitialization when modified must be marked meta=(ReinitializeOnEdit)
 * When modified, in the editor, these properties will trigger a reinitialization at which point
 * the runtime Op will get the latest values automatically.
 */
USTRUCT(BlueprintType)
struct FIKRetargetOpSettingsBase : public FIKRetargetOpSkeletonProvider
{
	GENERATED_BODY()
	
	virtual ~FIKRetargetOpSettingsBase() = default;

	/** Enable/disable the entire operation. Disabled ops are skipped entirely and do not run at all.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Enable Op")
	bool bEnabled = true;

	/** The maximum LOD that this Op is allowed to run at.
	 * For example if you have LODThreshold of 2, the Op will run until LOD 2 (based on 0 index). When the component LOD becomes 3, it will stop running.
	 * A value of -1 forces the Op to execute at all LOD levels. Default is -1. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Level of Detail")
	int32 LODThreshold = INDEX_NONE;

	/** toggle on/off all debug drawing on the op. */
	UPROPERTY()
	bool bDebugDraw = true;

	/** (required) override to specify how settings should be applied in a way that will not require reinitialization (ie runtime compatible)*/
	virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) { checkNoEntry(); };

	// NOTE ON RETARGET OP POSTLOAD() / SERIALIZE() BEHAVIOR:
	// If custom op settings require load-patching for backwards compatibility, it is NOT enough to rely on the archive
	// being tagged with custom versioning from the UIKRetargeter itself. This is because these settings structs can be stored
	// OUTSIDE of the retargeter (ie, in BP classes), which means they must take full control of their own back compatibility.
	//
	// BUT, because the settings are also stored as FInstancedStruct inside the retarget asset, and because these instanced structs
	// do not run custom serialization, the patch must ALSO be applied explicitly from within FIKRetargetOpBase::PostLoad().
	//
	// This leads us to this recommended pattern for op settings that need serialization patches:
	// 1. Override virtual FIKRetargetOpSettingsBase::PostLoad() in your custom settings struct.
	//		This will be called by the retargeter to apply patching for any op settings stored in the asset.
	// 2. Add the type trait WithSerializer and implement bool MyOp::Serialize().
	// 3. Call SerializeOpWithVersion() within the custom MyOp::Serialize()
	//		This will ensure the settings are saved with a custom version and it will call OpSettings::PostLoad() after loading
	//
	// Following this pattern ensures that op settings are properly patched both when they are stored in a retarget asset
	// as an instanced struct AND if they are stored alone as a BP variable.
	//
	template <typename T>
	static bool SerializeOpWithVersion(FArchive& Ar, T& Self)
	{
		Ar.UsingCustomVersion(FIKRigObjectVersion::GUID);
		T::StaticStruct()->SerializeTaggedProperties(Ar, reinterpret_cast<uint8*>(&Self), T::StaticStruct(), nullptr, nullptr);

		if (Ar.IsLoading())
		{
			const FIKRigObjectVersion::Type CustomVersion = static_cast<FIKRigObjectVersion::Type>(Ar.CustomVer(FIKRigObjectVersion::GUID));
			Self.PostLoad(CustomVersion);
		}
		
		return true;
	}

	/** (optional) called during the PostLoad of the owning asset. Can be used to apply patches to the data for backwards compatibility.
	 * NOTE: see description above on correct setup for retarget op settings serialization patches */
	virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) {};
	
	/** (optional, but recommended) provide a custom controller type (deriving from UIKRetargetOpControllerBase) as an API for editing the op
	 * NOTE: this type will automatically be lazy instantiated when/if the user calls GetController() on the op. */
	UE_API virtual const UClass* GetControllerType() const;
	
	/** provide a scripting object to edit your custom settings type in the editor via BP/Python
	* NOTE: this returns an instance of whatever type is returned by GetControllerType(). This can be custom per op. */
	UE_API UIKRetargetOpControllerBase* GetController(UObject* Outer);

private:

	// helper function for settings structs to instantiate their own controller
	UE_API UIKRetargetOpControllerBase* CreateControllerIfNeeded(UObject* Outer);
	
	// the controller used to edit this op by script/blueprint (lazy instantiated when needed)
	TStrongObjectPtr<UIKRetargetOpControllerBase> Controller = nullptr;

#if WITH_EDITORONLY_DATA
public:
	/** the op these settings belong to */
	UPROPERTY(transient)
	FName OwningOpName;
	/** the instance of this op currently running in the editor viewport */
	mutable FIKRetargetOpSettingsBase* EditorInstance;
	/** assigned by the processor when an editor instance is initialized, used for UI reflection for bone selectors
	 * NOTE: if your op has sub-settings structs that inherit from FIKRetargetOpSkeletonProvider, override this to give them pointers to the skeletons */
	UE_API virtual void AssignSkeletonAssets(const USkeleton* InSourceSkeleton, const USkeleton* InTargetSkeleton);
#endif
};

#if WITH_EDITOR
struct FOpExecutionTimer
{
	/** reset the running average execution time */
	void Reset();

	/** reset the per-frame time and update the running average*/
	void StartNewFrame();
	
	/** call this to add some execution time for this frame
	 * NOTE: this may be called multiple times per frame */
	void AddTimeThisFrame(const double InSeconds);

	/** returns the current running average execution time using a 30 frame exponential moving average */
	UE_API double GetRunningAverage() const;

private:
	/** running average of past n frames */
	double RunningAverageTime = 0.0f;
	/** profiling data; time in seconds of last execution */
	double TimeThisFrame = 0.0f;
};
#endif

/**
 * This is the base class for defining operations that live in the retargeter "op" stack.
 * These operations are executed in order by calling the virtual Run() function on each one in order.
 * The Run() function takes an input pose on the source skeletal mesh and affects the output pose on the target mesh.
 * NOTE: any user defined settings associated with a retarget op must be aggregated into a custom UStruct derived from FIKRetargetOpSettingsBase
 * This ensures that the settings are user editable in the details panel with full undo/redo support and serialization.
*/ 
USTRUCT(BlueprintType)
struct FIKRetargetOpBase
{
	GENERATED_BODY()
	
public:

	virtual ~FIKRetargetOpBase() = default;

	/** (optional) override to cache internal data when initializing the processor
	 * NOTE: you must set bIsInitialized to true to inform the retargeter that this op is ok to execute. */
	virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) { bIsInitialized = true; return true; };

	/** (optional) override to evaluate this operation and modify the output pose */
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose){};
	
	/** (optional) a second pass of initialization that ops can use after Op::Initialize() is called on all ops. */
	virtual void PostInitialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		FIKRigLogger& InLog) {};

	/** (optional) called during the PostLoad of the owning asset. Can be used to apply patches to the data for backwards compatibility. */
	UE_API virtual void PostLoad(const FIKRigObjectVersion::Type InVersion);
	
	/** (optional) override to automate initial setup after being added to the stack */
	virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) {};

	/** (optional) override to react after user assigns a new IK Rig to all ops
	 * @param SourceOrTarget: whether the source or target IK Rig was changed
	 * @param InIKRig: the IK rig asset to assign to this op (will be NULL if user cleared an IK Rig)
	 * @param InParentOp: the parent for this op (may be null if unparented) */
	virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) {};
	
	/** get if this op is enabled */
	bool IsEnabled() const { return GetSettingsConst()->bEnabled; };

	/** turn this operation on/off (will be skipped during execution if disabled) */
	void SetEnabled(const bool bEnabled){ GetSettings()->bEnabled = bEnabled; };

	/** return true if this op is initialized and ready to run */
	bool IsInitialized() const { return bIsInitialized; };

	/** (required) override and return a pointer to the settings struct used by this operation
	 * NOTE: it is not permitted to modify the settings from inside this getter. Treat as a const function. */ 
	virtual FIKRetargetOpSettingsBase* GetSettings() { checkNoEntry(); return nullptr; };

	/** const access to the settings */
	const FIKRetargetOpSettingsBase* GetSettingsConst() const { return const_cast<FIKRetargetOpBase*>(this)->GetSettings(); }
	
	// Old API (deprecated):
	UE_DEPRECATED(5.7, "SetSettings(const FIKRetargetOpSettingsBase*) is deprecated. "
					"Use GetSettings()->CopySettingsAtRuntime(const FIKRetargetOpSettingsBase*) instead")
	UE_API virtual void SetSettings(const FIKRetargetOpSettingsBase* InSettings);
	
	/** (required) override and return the type used to house the settings for this operation */
	virtual const UScriptStruct* GetSettingsType() const { checkNoEntry(); return nullptr; };

	/** (required) override and return the type of this op (the derived subclass) */
	virtual const UScriptStruct* GetType() const { checkNoEntry(); return FIKRetargetOpBase::StaticStruct(); };

	/** (optional) override to reset any internal state when animation playback is reset or stopped (ie, springs / dampers etc) */
	virtual void OnPlaybackReset() {};

	/** (optional) override to get any data from the source or target skeletal mesh component
	 * NOTE: this is called during AnimGraph::PreUpdate() which runs on the main thread, use caution. */
	virtual void AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent) {};

	/** (optional) override to get any data from the anim graph as it's evaluating.
	 * NOTE: this is called during AnimGraph::Evaluate_AnyThread() BEFORE the retargeting Ops are executed. */
	virtual void AnimGraphEvaluateAnyThread(FPoseContext& Output) {};

	/** (optional) override and add any bones that your op modifies to the output TSet of bone indices
	 * Add indices of any bone that this op modifies. Any bone not registered here will be FK parented by other operations*/
	virtual void CollectRetargetedBones(TSet<int32>& OutRetargetedBones) const {};

	/** (optional) ops can optionally behave as a 'parent' where child ops must be executed first. */
	virtual bool CanHaveChildOps() const { return false; };

	/** (optional) ops can optionally behave as a 'child' where they can be parented to ops of the type returned by this function. */
	virtual const UScriptStruct* GetParentOpType() const { return nullptr; };

	/** (optional) override to give this op a chance to initialize before it's children are initialized
	 * This is executed right after the previous op in the stack and right before the first child op Initialize() is called
	 * NOTE: unless this is a parent op, this will have no effect */
	virtual void InitializeBeforeChildren(
		FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		FIKRigLogger& Log) {};
	
	/** (optional) override to give this op a chance to do work before it's children are run
	 * This is executed right after the previous op in the stack and right before the first child op Run() is called
	 * NOTE: unless this is a parent op, this will have no effect */
	virtual void RunBeforeChildren(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose){};

	/** (optional) override to give this op a chance to do work after it's parent is run
	* This is executed right after the parent op in the stack and right before the next op after parent
	* NOTE: unless this is a child op, this will have no effect */
	virtual void RunAfterParent(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose){};

	/** (optional) return true to disallow multiple copies of this op in the stack */
	virtual bool IsSingleton() const { return false; };

	/** (optional) return true to disallow adding this op to the stack */
	virtual bool IsAbstract() const { return false; };

	/** (optional) override and supply the target IK Rig this op references
	 * NOTE: The retarget processor will resolve the bone chains in the rig so that they can be queried by the op. */
	virtual const UIKRigDefinition* GetCustomTargetIKRig() const { return nullptr; };
	
	/** (optional) ops can optionally store their own chain mapping, this allows outside systems to query/edit it.
	 * NOTE: this is conceptually const and should not alter the op state, but only return the chain mapping! */
	virtual FRetargetChainMapping* GetChainMapping() { return nullptr; };

	/** wraps the non-const GetChainMapping() to provide const access */
	const FRetargetChainMapping* GetChainMapping() const { return const_cast<FIKRetargetOpBase*>(this)->GetChainMapping(); };

	/** (optional) implement this if the op stores chain settings by name to allow the settings to be maintained after a chain is renamed
	 * NOTE: this is only called when the target IK Rig this op references has a chain that is renamed.
	 * NOTE: ops do not have to update chain mappings in this function, that is managed for them if GetChainMapping() is implemented */
	virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) {};

	/** (optional) override to allow ops to react when a property marked "ReinitializeOnEdit" is modified
	 * NOTE: this is called before Initialize() is called again to give op a chance to auto-configure itself based on its new state
	 * NOTE: this is called in several places where InPropertyChangedEvent may be null */
	virtual void OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent) {};

	/** (optional) override to allow ops to react when a property marked "ReinitializeOnEdit" on the parent op is modified
	 * NOTE: InPropertyChangedEvent will be null at load time giving op a chance to clean based on parent state.
	 * NOTE: this is called in several places where InPropertyChangedEvent may be null */
	virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) {};

#if WITH_EDITOR
	
	/** get the nice name to display in the viewport (defaults to "DisplayName" UStruct metadata) */
	UE_API FName GetDefaultName() const;

	/** (optional) override to display a warning message in the op stack */
	UE_API virtual FText GetWarningMessage() const;

	/** (optional) override to draw debug info into the editor viewport when the Op is selected */
	virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InTargetTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const {};

	/** (required if using DebugDraw()) return true to display the debug draw toggle in the UI */
	virtual bool HasDebugDrawing() { return false; }

	/** (optional) override to support resetting all settings for a given chain */
	virtual void ResetChainSettingsToDefault(const FName& InChainName) {};

	/** (optional) override to support "Reset to Default" UI for chains */
	virtual bool AreChainSettingsAtDefault(const FName& InChainName) { return true; };

	/** (required for any op that stores settings per-chain) return the memory for the settings struct associated with a particular chain
	 * NOTE: implementing this can automate some boilerplate UI setup, allows systems to display chain settings and get/set them. */
	virtual uint8* GetChainSettingsMemory(const FName InChainName) { checkNoEntry(); return NULL; };
	
	/** (optional) override to update op with anim sequence information when batch retargeting anim sequences
	* NOTE: this should only be called for batch (offline) retargeting where there is no anim-instance running */
	virtual void UpdateFromAnimSequence(UAnimSequence* Sequence, float FrameTime) {};

	/** (optional) override this to customize which properties are available to be overridden in a profile.
	 * NOTE: the default implementation returns all overrideable properties */
	UE_API virtual void GetOverrideableProperties(TSharedPtr<FRetargetPropertyOverrideNode>& OutRootNode) const;

	/** the instance of this op currently running in the editor viewport */
	mutable FIKRetargetOpBase* EditorInstance;

	/** a timer for the op profiler */
	FOpExecutionTimer Timer;

	/** Called during PostLoad to allow ops to be split into multiple ops during version upgrade.
     * Return an array of new op instances that replace this op.
     * If empty, no splitting occurred and the original op is preserved.
     * @param OutNewOps - Out array which will hold new op instances (if splitting occurred)
     * @return true if this op is being replaced by OutNewOps, false to keep original op
	 */
	virtual bool PostLoadSplitOp(TArray<FInstancedStruct>& OutNewOps)
	{
		return false; // No split by default.
	}
#endif

	/** Get the name of this op (may be customized by user) */
	UE_API FName GetName() const;

	/** Set the name of this op
	 * NOTE: InName is assumed to be unique in the stack (this constraint is enforced when setting the name from the controller) */
	UE_API void SetName(const FName InName);
	
	/** Set the name of the op this op is "parented" to. This enforces execution order.
	 * NOTE: InName is assumed to refer to an op that exists.
	 * NOTE: Renaming the parent op through the controller will auto-update this. */
	UE_API void SetParentOpName(const FName InName);

	/** Get the name of the op this op is parented to (None for root level ops)*/
	UE_API FName GetParentOpName() const;

	/** wholesale copy all settings from the input settings into this op.
	 * NOTE: InSettings is assumed to be castable to the type returned by GetSettingsType()
	 * NOTE: be careful calling this on an initialized op as it may invalidate it's runtime state (use SetSettings instead) */
	UE_API void CopySettingsRaw(const FIKRetargetOpSettingsBase* InSettings, const TArray<FName>& InPropertiesToIgnore);

	/** a convenience function to copy all properties from one struct to another while ignoring some */
	static UE_API void CopyStructProperties(
		const UStruct* InStructType,
		const void* InSrcStruct,
		void* InOutDestStruct,
		const TArray<FName>& InPropertiesToIgnore);

	using FFrameValues = TArray<TArray<TOptional<float>>>;

	struct FCurveData
	{
		TArray<FName> Names;
		TArray<int32> Flags;
		TArray<FLinearColor> Colors;
	};

	bool GetDeprecatedEnabledState() const {return bIsEnabled_DEPRECATED; }

private:
	
	/** the text label given to the op, used to refer to it from script */
	UPROPERTY()
	FName Name;

	/** (optional) some ops are considered as a group, this is the name of the group parent */
	UPROPERTY()
	FName ParentOpName;

	/** DEPRECATED enabled flag, moved to base op settings */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "bIsEnabled is deprecated. Please use FIKRetargetOpSettingsBase::bEnabled instead."))
	bool bIsEnabled_DEPRECATED = true;

protected:
	
	/** set to true when the op is ready to run */
	bool bIsInitialized = false;
};

/**
 * This is the base class for defining a custom controller for a given retarget op type.
 * Controllers provide an API for editing ops from blueprint or python.
 *
 * To use a controller:
 * 1. Get a controller by calling UIKRetargeterController::GetOpController(int InOpIndex)
 * 2. Cast the returned UIKRetargetOpControllerBase* to the type of controller used by the op you want to modify.
 * 3. Call the public getter/setters to modify the op as desired
 *
 * NOTE: these controllers are necessary because the op UStructs do not support UFunctions
*/ 
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetOpControllerBase : public UObject
{
	GENERATED_BODY()
	
public:
	
	// the op settings this controller controls
	FIKRetargetOpSettingsBase* OpSettingsToControl = nullptr;
};


//
// BEGIN LEGACY OP BASE
//

// NOTE: This type has been replaced by FRetargetOpBase. URetargetOpBase-based ops no longer work, please refactor into the new FRetargetOpBase struct.
UCLASS(MinimalAPI)
class URetargetOpBase : public UObject
{
	GENERATED_BODY()
	
	public:

	// this is the deprecation upgrade path for solvers that inherit from UIKRigSolver to convert them to FIKRigSolverBase
	// override this and supply a struct into OutInstancedStruct that derives from FIKRigSolverBase that implements your custom solver.
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct){};
	
	virtual bool Initialize(
	const FIKRetargetProcessor& Processor,
		const FRetargetSkeleton& SourceSkeleton,
		const FTargetSkeleton& TargetSkeleton,
		FIKRigLogger& Log) { return false; };
	virtual void Run(
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InOutSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose){};

	UPROPERTY()
	bool bIsEnabled = true;

	bool bIsInitialized = false;
	
#if WITH_EDITOR
	virtual void OnAddedToStack(const UIKRetargeter* Asset) {};
	virtual FText GetNiceName() const { return FText::FromString(TEXT("Default Op Name")); };
	virtual FText WarningMessage() const { return FText::GetEmpty(); };
#endif
};

// NOTE: This type is no longer in use except to load old stacks of UObject-based ops.
UCLASS(MinimalAPI)
class URetargetOpStack : public UObject
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY()
	TArray<TObjectPtr<URetargetOpBase>> RetargetOps_DEPRECATED;
};

//
// END LEGACY OP BASE
//

#undef UE_API
