// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FIKRetargetProcessor;
struct FInstancedPropertyBag;

struct FIKRetargetRuntimeOverrideManager
{
	struct FOpOverridesKey
	{
	   FName OverrideSetName;
	   FName OpName;

	   FOpOverridesKey() = default;
	   FOpOverridesKey(FName ProfileName, FName OpName) : OverrideSetName(ProfileName), OpName(OpName){}
	   
	   bool operator==(const FOpOverridesKey& Other) const
	   {
		  return OverrideSetName == Other.OverrideSetName && OpName == Other.OpName;
	   }
	   
	   friend uint32 GetTypeHash(const FOpOverridesKey& Key)
	   {
		  uint32 Hash = GetTypeHash(Key.OverrideSetName);
		  return HashCombine(Hash, GetTypeHash(Key.OpName));
	   }
	};
	
	// fast cache for safe numeric casting in the hot path
	enum class ECurveCoercionType : uint8
	{
	   None,
	   Bool,
	   Float,
	   Double,
	   Int32,
	   Int64,
	   Byte
	};

	// coercion type for variable bindings when PropertyBag type differs from op settings type
	enum class EVariableCoercionType : uint8
	{
	   None,          // no variable bound
	   Direct,        // same size, raw memcpy
	   FloatToDouble, // PropertyBag float (4 bytes) -> op settings double (8 bytes)
	   DoubleToFloat, // PropertyBag double (8 bytes) -> op settings float (4 bytes)
	};

	struct FCompiledPropertyOverride
	{
	   int32 SourcePoolOffset = INDEX_NONE; // offset into CachedValuePool (if hardcoded)
	   int32 BagMemoryOffset = INDEX_NONE;  // offset into FInstancedPropertyBag memory (if bound to var)

	   FName BoundCurveName = NAME_None;    // name of the curve (if bound to curve)
	   ECurveCoercionType CurveCoercionType = ECurveCoercionType::None; // the target type for curve casting
	   EVariableCoercionType VarCoercionType = EVariableCoercionType::None; // coercion for variable binding

	   void* DestPtr = nullptr;             // pointer to processor op settings memory
	   int32 Size = 0;                      // size of the property data

	   // value cache invalidate/update stuff
	   FOpOverridesKey OpOverrideKey; 
	   FProperty* Property = nullptr;         
	   uint32 PropertyPathHash = 0;      
	};

	void ApplyPropertyOverridesToOps(
	   FIKRetargetProcessor& Processor,
	   const TArrayView<const FName> InOverrideSetsToApply,
	   FInstancedPropertyBag* InVariables,
	   TMap<FName, float>* InBoundCurveValues);

	void DirtyProfileCache() { bIsCacheDirty = true; };

private:

	static void DebugPrintVariables(FInstancedPropertyBag* InVariables);
	
	void UpdateCachedValues(FIKRetargetProcessor& Processor);
	
	void UpdateFullListOfOverrideSets(FIKRetargetProcessor& Processor, const TArrayView<const FName> InOverrideSetsToApply);

	void RebuildCache(FIKRetargetProcessor& Processor, FInstancedPropertyBag* InVariables);

	static uint32 GetOverrideSetArrayHash(const TArray<FName>& InProfileNames);

	TArray<FName> OverrideSetsToApply; 
	TArray<uint8> CachedValuePool;
	TArray<FCompiledPropertyOverride> CompiledOverrides;
	TMap<FOpOverridesKey, int32> CachedVersionPerOp;
	
	int32 CachedAssetVersion;
	uint32 OverrideSetHash;
	bool bIsCacheDirty = true;
};