// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakFieldPtr.h"

#include "IKRetargetOverrides.generated.h"

#define UE_API IKRIG_API

struct FIKRetargetProcessor;


// A centralized utility for parsing a property path string
// Intended for property path strings with properties separated by "->" with support for array syntax like "[0]"
class FPropertyPathParser
{
public:
	
	struct FPropertyPathStep
	{
		FString PropertyName;
		int32 ArrayIndex = INDEX_NONE;

		bool IsArray() const { return ArrayIndex != INDEX_NONE; }
	};
	
	IKRIG_API static TArray<FPropertyPathStep> ParsePath(const FString& InPath);
};

#if WITH_EDITOR
// What is this for?
// This type is used to store a customized property tree that ops can supply to tell outside systems which of the op's settings can be overridden.
// For example, virtual FIKRetargetOpBase::GetOverrideableProperties() takes a root node and allows the op to add a hierarchy of nodes underneath it
// with customized labels. This allows ops to tell UI code how to group and organize setting overrides.
//
// Why not FPropertyNode?
//		1. FPropertyNode is an editor-only type and we do not want to link Slate
//		2. This type adds support for custom group labels
//
struct FRetargetPropertyOverrideNode: public TSharedFromThis<FRetargetPropertyOverrideNode>
{
	/** the name of this submenu (e.g., "Settings" or "IKGoal") */
	FName NodeName;
    
	/** sub-menus under this one */
	TMap<FName, TSharedPtr<FRetargetPropertyOverrideNode>> SubNodes;
    
	/** the actual leaf properties in this specific section */
	struct FRetargetPropertyOverrideEntry
	{
		FText DisplayName;
		FString PropertyPath; 
	};
	TArray<FRetargetPropertyOverrideEntry> LeafProperties;

	/** * helper to populate this node with properties from a specific struct.
     * @param InStruct The struct type to reflect.
     * @param InStructData pointer to the struct instance (needed for dynamic content like array elements)
     * @param PathPrefix The string path leading to this struct (e.g., "Chains[0]")
     */
	void AddPropertiesFromStruct(const UScriptStruct* InStruct,	const uint8* InStructData, FString PathPrefix);
};
#endif

/** Single property override for a retarget op */
USTRUCT()
struct FRetargetOpPropertyOverride
{
	GENERATED_BODY()

	FRetargetOpPropertyOverride() = default;
	FRetargetOpPropertyOverride(const FString& InPropertyPath) : PropertyPath(InPropertyPath)
	{
		PropertyPathHash = GetTypeHash(InPropertyPath);
	} 

	/** get access to the path to this property */
	const FString& GetPropertyPath() const { return PropertyPath; };
	/** get read-only access to the value of this override as a string */
	const FString& GetValueString() const { return ValueAsString; };
	/** get read/write access to the value of this override as a string */
	FString& GetValueStringEditable() { return ValueAsString; };
	/** get the hash of the property path (cached at construction) */
	uint32 GetPropertyPathHash() const { return PropertyPathHash; };
	
	/** set the name of the variable to bind to this property
	 * NOTE: this must go through the retargeter controller */
	void SetBoundVariableName(const FName InVariableName) { BoundVariableName = InVariableName; };
	FName GetBoundVariableName() const { return BoundVariableName; };

	/** set the name of the curve to bind to this property
	 * NOTE: this must go through the retargeter controller */
	void SetBoundCurveName(const FName InCurveName) { BoundCurveName = InCurveName; };
	FName GetBoundCurveName() const { return BoundCurveName; };
	
	/** validate that this property exists and is accessible */
	bool IsValid(const UScriptStruct* InSettingsStruct) const;

	/** return true if the given property path is at the default value */
	UE_API static bool IsStructPropertyAtDefault(const UScriptStruct* InStruct, uint8* InStructData, const FString& InPropertyPath);

	struct FPropertySegment
	{
		TWeakFieldPtr<FProperty> Property;
		int32 ArrayIndex = INDEX_NONE;
	};
	
	/** parse a property path string into segments */
	UE_API static bool GetSegmentsFromProperyPath(
		const FString& InPath,
		const UScriptStruct* InOwnerStruct,
		TArray<FPropertySegment>& OutSegments);

	/** navigate from a struct pointer to the final property pointer */
	UE_API static uint8* GetDataPointerFromPathSegments(
		uint8* InStructPtr,
		const TArray<FPropertySegment>& InSegments,
		bool bResizeArrays = false);

	/** get the leaf property from the segment chain */
	UE_API static FProperty* GetLeafProperty(const TArray<FPropertySegment>& InSegments);

	/** find a property by name with redirect support */
	static FProperty* FindProperty(
		const UStruct* InStruct,
		const FString& InNameOrDisplayName);

	/** reflection indirection because these are private */
	static FName GetPropertyPathName(){ return GET_MEMBER_NAME_CHECKED(FRetargetOpPropertyOverride, PropertyPath); }
	static FName GetValeAsStringName(){ return GET_MEMBER_NAME_CHECKED(FRetargetOpPropertyOverride, ValueAsString); }

private:
	
	/** property path using "->" separator like, "IKGoalSettings->BlendAlpha" and array syntax like, "Chains[0]->Weight"
	 * NOTE: this allows generic serialization of property overrides */
	UPROPERTY(EditAnywhere, Category=Override)
	FString PropertyPath;
	UPROPERTY()
	uint32 PropertyPathHash = 0;

	/** Value serialized as string */
	UPROPERTY(EditAnywhere, Category=Override)
	FString ValueAsString;

	/** The name of the variable this override is bound to, if any. */
	UPROPERTY()
	FName BoundVariableName = NAME_None;

	/** The name of the curve this override is bound to, if any. */
	UPROPERTY()
	FName BoundCurveName = NAME_None;
};

/** A collection of property overrides to assign to a single op. */
USTRUCT()
struct FRetargetOpOverrides
{
	GENERATED_BODY()
	
	/** the retarget op to apply these overrides to */
	UPROPERTY(EditAnywhere, Category=Override)
	FName OpName = NAME_None;
	
	/** the individual property-level overrides for this op */
	UPROPERTY(EditAnywhere, Category=Override)
	TArray<FRetargetOpPropertyOverride> PropertyOverrides;

	/** the reflection data for the struct */
	UPROPERTY(EditAnywhere, Category=Override)
	TObjectPtr<UScriptStruct> ScriptStruct;

	/** * registers a new property override for this operation.
	 * @param InPropertyPath    The string path to the property (e.g., "Settings->Alpha").
	 * @param InSettingsPtr     Pointer to the memory block containing the source value.
	 * @param InSettingsStruct  The reflection data (UScriptStruct) for the settings type.
	 * @param bRequiresReinit   True if applying this property at runtime requires a full op re-initialization.
	 * @return True if the override was successfully added.
	 */
	UE_API bool AddPropertyOverride(const FString& InPropertyPath, const uint8* InSettingsPtr, const UScriptStruct* InSettingsStruct, bool bRequiresReinit);

	/** * Removes an existing property override based on its path.
	 * @param InPropertyPath    The string path of the override to remove.
	 * @return True if a matching override was found and removed.
	 */
	UE_API bool RemovePropertyOverride(const FString& InPropertyPath);

	/** * updates the serialized value of an existing property override.
	 * @param InPropertyOverride	The override to update.
	 * @param InStructData			Pointer to the memory block containing the new value.
	 * @param InScriptStruct		The reflection data (UScriptStruct) for the settings type.
	 * @return True if the value was updated, false if no matching override exists.
	 */
	UE_API bool UpdateOverrideValue(FRetargetOpPropertyOverride& InPropertyOverride, const uint8* InStructData, const UScriptStruct* InScriptStruct);

	/** @return True if an overrides with the given property path exists in this op profile */
	UE_API bool HasPropertyOverride(const FString& InPropertyPath) const;

	/** * attempts to locate a specific property override by its path.
	 * @param InPropertyPath    The string path to search for.
	 * @return A pointer to the override data if found, otherwise nullptr.
	 */
	UE_API FRetargetOpPropertyOverride* FindPropertyOverride(const FString& InPropertyPath);

	/** * returns the total count of properties currently overridden for this operation. 
	 */
	UE_API int32 GetNumPropertyOverrides() const;

	/** Get the version of the values stored for this op (incremented on edit) */
	int32 GetValueVersion() const { return Version; }

private:

	/** incremented whenever override values change, used to update the runtime cache value */
	uint32 Version = 0;
};

USTRUCT()
struct FRetargetOverrideSet
{
	GENERATED_BODY()
	
	/** the label for this set of overrides. */
	UPROPERTY(EditAnywhere, Category=Override)
	FName Name;

	/** used to organize override sets hierarchically */
	UPROPERTY(EditAnywhere, Category=Override)
	FName ParentName;
	
	/** a collection of sparse property value overrides for ops in the stack. */
	UPROPERTY(EditAnywhere, Category=Override)
	TArray<FRetargetOpOverrides> OpOverrides;	

	/** allow an override set to be active by default */
	UPROPERTY(EditAnywhere, Category=Override)
	bool bActiveByDefault = false;

	/** Display order for sorting among sibling override sets in the editor tree view */
	UPROPERTY()
	int32 DisplayOrder = 0;

	/** get an op override by the op name */
	UE_API FRetargetOpOverrides* FindOpOverrideByOpName(const FName InOpName);
};

#undef UE_API
