// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IKRetargeter.h"
#include "IKRetargetOps.h"
#include "IKRetargetOverrideManager.h"
#include "IKRetargetSettings.h"
#include "IKRigLogger.h"
#include "Kismet/KismetMathLibrary.h"
#include "Rig/IKRigProcessor.h"

#include "IKRetargetProcessor.generated.h"

#define UE_API IKRIG_API

DECLARE_CYCLE_STAT_EXTERN(TEXT("IK Retarget Processor Run"), STAT_Anim_RetargetProcessorRun, STATGROUP_Anim, UE_API);

struct FIKRetargetOpBase;
enum class ERetargetTranslationMode : uint8;
enum class ERetargetRotationMode : uint8;
class URetargetChainSettings;
class UIKRigDefinition;
struct FIKRigProcessor;
struct FReferenceSkeleton;
struct FBoneChain;
struct FIKRetargetPose;
class UIKRetargeter;
class USkeletalMesh;
class URetargetOpBase;
struct FIKRetargetPelvisMotionOp;
struct FResolvedBoneChain;
struct FIKRetargetDebugDrawState;


struct FRetargetPoseScaleWithPivot
{
	FRetargetPoseScaleWithPivot() = default;
	FRetargetPoseScaleWithPivot(const double InFactor, const FVector& InPivot) : Factor(InFactor), Pivot(InPivot) {};

	bool operator==(const FRetargetPoseScaleWithPivot& Other) const
	{
		return FMath::IsNearlyEqual(Factor, Other.Factor);
	}

	bool ScalePose(TArray<FTransform>& InOutGlobalPose) const;
	
	double Factor = 1.0;
	FVector Pivot = FVector::ZeroVector;
};

// cached retarget pose transforms (both local and global) after it has been applied/resolved on a specific skeletal mesh
struct FResolvedRetargetPose
{
	FResolvedRetargetPose(const FName InName) : Name(InName) {}
	
	FName Name;						// the name of the retarget pose this was initialized with
	int32 Version;					// the version of the retarget pose this was initialized with (transient)
	FRetargetPoseScaleWithPivot PoseScale;			// the scale of the retarget pose this was initialized with (transient)
	
	TArray<FTransform> LocalPose;	// local space retarget pose
	TArray<FTransform> GlobalPose;	// global space retarget pose
};

// the set of all resolved/cached retarget poses in use for a source or target skeleton
struct FResolvedRetargetPoseSet
{
	FName CurrentRetargetPoseName;
	TArray<FResolvedRetargetPose> RetargetPoses;	// all the retarget poses resolved on the target skeleton

	UE_API FResolvedRetargetPose& AddOrUpdateRetargetPose(
		const FRetargetSkeleton& InSkeleton,
		const FName InRetargetPoseName,
		const FIKRetargetPose* InRetargetPose,
		const FName PelvisBoneName,
		const FRetargetPoseScaleWithPivot& InSourceScale);

	UE_API const FResolvedRetargetPose* FindRetargetPoseByName(const FName InRetargetPoseName) const;
	
	FResolvedRetargetPose& FindOrAddRetargetPose(const FName InRetargetPoseName);

	UE_API const TArray<FTransform>& GetLocalRetargetPose() const;
	
	UE_API const TArray<FTransform>& GetGlobalRetargetPose() const;

	UE_API FTransform GetGlobalRetargetPoseOfSingleBone(
		const FRetargetSkeleton& InSkeleton,
		const int32 BoneIndex,
		const TArray<FTransform>& InGlobalPose) const;

	UE_API void Reset();
};

/**
 * A retarget skeleton contains:
 * 1. hierarchy data (bones, parents)
 * 2. a retarget pose
 * 3. functions for converting Local/Global poses
 * A retarget skeleton is created for both the source and target skeleton.
 * The target skeleton uses a specialized skeleton subclass for managing a bone mask of retargeted bones
 */
struct FRetargetSkeleton
{
	TArray<FName> BoneNames;				// list of all bone names in ref skeleton order
	TArray<int32> ParentIndices;			// per-bone indices of parent bones (the hierarchy)
	FResolvedRetargetPoseSet RetargetPoses;	// the set of cached retarget poses
	const USkeletalMesh* SkeletalMesh;		// the skeletal mesh this was initialized with

	UE_API void Initialize(
		const USkeletalMesh* InSkeletalMesh,
		const ERetargetSourceOrTarget InSourceOrTarget,
		const UIKRetargeter* InRetargetAsset,
		const FName PelvisBoneName,
		const FRetargetPoseScaleWithPivot& InPoseScale);
	
	UE_API void Reset();

	UE_API int32 FindBoneIndexByName(const FName InName) const;

	UE_API int32 GetParentIndex(const int32 BoneIndex) const;

	const FTransform& GetParentTransform(const int32 BoneIndex, const TArray<FTransform>& InPose) const;

	UE_API void UpdateGlobalTransformsBelowBone(
		const int32 StartBoneIndex,
		const TArray<FTransform>& InLocalPose,
		TArray<FTransform>& OutGlobalPose) const;

	UE_API void SetGlobalTransformAndUpdateChildren(
		const int32 InBoneToSetIndex,
		const FTransform& NewTransform,
		TArray<FTransform>& InOutGlobalPose) const;

	UE_API void UpdateLocalTransformsBelowBone(
		const int32 StartBoneIndex,
		TArray<FTransform>& OutLocalPose,
		const TArray<FTransform>& InGlobalPose) const;
	
	UE_API void UpdateGlobalTransformOfSingleBone(
		const int32 BoneIndex,
		const FTransform& InLocalTransform,
		TArray<FTransform>& OutGlobalPose) const;
	
	UE_API void UpdateLocalTransformOfSingleBone(
		const int32 BoneIndex,
		TArray<FTransform>& OutLocalPose,
		const TArray<FTransform>& InGlobalPose) const;

	UE_API FTransform GetLocalTransformOfSingleBone(
		const int32 BoneIndex,
		const TArray<FTransform>& InGlobalPose) const;

	UE_API TArray<FTransform> GetLocalTransformsOfMultipleBones(
		const TArray<int32>& InBoneIndices,
		const TArray<FTransform>& InGlobalPose) const;

	UE_API void UpdateGlobalTransformsOfMultipleBones(
		const TArray<int32>& InBoneIndices,
		const TArray<FTransform>& InLocalTransforms,
		TArray<FTransform>& OutGlobalPose) const;

	UE_API int32 GetCachedEndOfBranchIndex(const int32 InBoneIndex) const;

	UE_API void GetChildrenIndices(const int32 BoneIndex, TArray<int32>& OutChildren) const;

	UE_API void GetChildrenIndicesRecursive(const int32 BoneIndex, TArray<int32>& OutChildren) const;
	
	UE_API bool IsParentOf(const int32 PotentialParentIndex, const int32 ChildBoneIndex) const;

private:
	
	/** One index per-bone. Lazy-filled on request. Stores the last element of the branch below the bone.
	 * You can iterate between in the indices stored here and the bone in question to iterate over all children recursively */
	mutable TArray<int32> CachedEndOfBranchIndices;
};

/** A retarget skeleton for the target skeleton.
 * Contains the output pose buffer for the target skeleton
 * Provides a per-bone mask specifying which bones are retargeted.
 */
struct FTargetSkeleton : public FRetargetSkeleton
{
	TArray<FTransform> OutputGlobalPose;
	TArray<FTransform> InputLocalPose;

	void Initialize(
		const USkeletalMesh* InSkeletalMesh,
		const ERetargetSourceOrTarget InSourceOrTarget,
		const UIKRetargeter* InRetargetAsset,
		const FName RetargetRootBone,
		const FRetargetPoseScaleWithPivot& InPoseScale);

	void Reset();

	void SetRetargetedBones(const TSet<int32>& InRetargetedBones);

	bool GetIsBoneRetargeted(int32 InBoneIndex) const;

	const TArray<bool>& GetRetargetedBonesMask() const;

private:
	
	/** a boolean mask with size NumBones, with value of "true" for any bone that is retargeted
	 * ie, bones that are actually posed based on a mapped source chain
	 * NOTE: this mask is only available in Op::PostInitialize()... it is built AFTER Op::Initialize() */
	bool bIsMaskInitialized = false;
	TArray<bool> IsBoneRetargeted;
};

/** A "resolved" bone chain serves a few purposes:
 * 1. takes the Start/End bone names from the IK Rig and converts them into a list of bone indices on the skeletal mesh
 * 2. stores Local/Global ref pose of the chain
 * 3. provides facilities to generate local/global poses of the chain
 * 4. parameterizes the chain allowing you to GetTransformAtChainParam()
 * 
 * The intention with this data structure is to provide read-only chain data for ops to work with as they need.
 */
struct FResolvedBoneChain
{
	UE_API FResolvedBoneChain(
		const FBoneChain& InBoneChain,
		const FRetargetSkeleton& InSkeleton,
		FIKRigLogger& InLog);

	UE_API bool IsValid() const;

	UE_API void UpdatePoseFromSkeleton(const FRetargetSkeleton& InSkeleton);
	
	UE_API void GetWarnings( const FRetargetSkeleton& Skeleton, FIKRigLogger& Log) const;

	UE_API FTransform GetTransformAtChainParam(
		const TArray<FTransform>& Transforms,
		const double& Param) const;
	
	UE_API double GetStretchAtParam(
		const TArray<FTransform>& InitialTransforms,
		const TArray<FTransform>& CurrentTransforms,
		const double& Param) const;

	int32 GetBoneClosestToParam(const double& Param) const;

	int32 GetBoneAtParam(const double& Param) const;

	int32 GetEquivalentBoneInOtherChain(const int32 InChainIndex, const FResolvedBoneChain& InOtherBoneChain) const;
	
	static UE_API double GetChainLength(const TArray<FTransform>& Transforms);

	static UE_API void FillTransformsWithLocalSpaceOfChain(
		const FRetargetSkeleton& Skeleton,
		const TArray<FTransform>& InGlobalPose,
		const TArray<int32>& BoneIndices,
		TArray<FTransform>& OutLocalTransforms);
	
	static UE_API void FillTransformsWithGlobalRetargetPoseOfChain(
		const FRetargetSkeleton& Skeleton,
		const TArray<FTransform>& InGlobalPose,
		const TArray<int32>& BoneIndices,
		TArray<FTransform>& OutGlobalTransforms);

	UE_API TArray<FTransform> GetChainTransformsFromPose(const TArray<FTransform>& InPose) const;

	FName ChainName;
	FName StartBone;
	FName EndBone;
	FName IKGoalName;
	
	TArray<FTransform> RefPoseGlobalTransforms;
	TArray<FTransform> RefPoseLocalTransforms;
	mutable TArray<FTransform> CurrentLocalTransforms; // reusable scratch space when op needs it

	TArray<float> Params;
	TArray<int32> BoneIndices;
	float InitialChainLength;

	int32 ChainParentBoneIndex;
	FTransform ChainParentInitialGlobalTransform;

private:
	
	bool bFoundStartBone = false;
	bool bFoundEndBone = false;
	bool bEndIsStartOrChildOfStart  = false;
	
	void CalculateBoneParameters(FIKRigLogger& Log);
};

/** a container for ALL the fully resolved bone chains on both
 * the source and target skeletons, as well as the mapping between them */
struct FRetargeterBoneChains
{
	/** Load all chains from the IK Rigs and resolved them on the current skeletal meshes*/
	UE_API bool Initialize(
		const UIKRetargeter* InRetargetAsset,
		const TArray<const UIKRigDefinition*>& InTargetIKRigs,
		const FRetargetSkeleton& InSourceSkeleton,
		const FRetargetSkeleton& InTargetSkeleton,
		FIKRigLogger& InLog);
	
	/** Get read only access to the resolved bone chains on either the source or the target
	 * If asking for a target chain, must supply the IK Rig it belongs to */
	UE_API const TArray<FResolvedBoneChain>* GetResolvedBoneChains(
		ERetargetSourceOrTarget SourceOrTarget,
		const UIKRigDefinition* TargetIKRig = nullptr) const;

	/** Get read only access to all resolved bone chains, for all IK Rigs used on the target */
	UE_API const TMap<const UIKRigDefinition*, TArray<FResolvedBoneChain>>& GetAllResolvedTargetBoneChains() const;

	/** Get a fully resolved bone chain by name
	 * If asking for a target chain, must supply the IK Rig it belongs to. */
	UE_API const FResolvedBoneChain* GetResolvedBoneChainByName(
		const FName InChainName,
		const ERetargetSourceOrTarget SourceOrTarget,
		const UIKRigDefinition* TargetIKRig = nullptr) const;

	/** Update all transforms stored in the chains. This is needed whenever the retarget pose is modified */
	UE_API void UpdatePoseFromSkeleton(
		const FRetargetSkeleton& InSkeleton,
		const ERetargetSourceOrTarget SourceOrTarget);

	/** Get a list of all chains that contain the bone with the given index */
	UE_API TSet<FName> GetChainsThatContainBone(int32 InBoneIndex, ERetargetSourceOrTarget SourceOrTarget) const;

	/** Reset everything (between initializations) */
	UE_API void Reset();

private:
	
	/** All resolved chains on the source mesh */
	TArray<FResolvedBoneChain> SourceBoneChains;

	/** All resolved chains on the target mesh for each IK Rig used by the ops */
	TMap<const UIKRigDefinition*, TArray<FResolvedBoneChain>> TargetBoneChains;

	/** the default IK Rig */
	const UIKRigDefinition* DefaultTargetIKRig;

	bool bIsInitialized = false;
};

enum class ERetargetBoneSpace : uint8
{
	Global,
	Local
};

enum class ERetargetOpsToSearch : uint8
{
	ProcessorOps,
	AssetOps
};

/** * Parameters for initializing the IK Retargeter.
* Initialization does a lot of validation and can fail for many reasons. 
* Check IsInitialized() afterwards.
*/
struct FRetargetInitParameters
{
	/** The skeletal mesh to pose FROM */
	const USkeletalMesh* SourceSkeletalMesh = nullptr;
	/** The skeletal mesh to pose TO */
	const USkeletalMesh* TargetSkeletalMesh = nullptr;
	/** The source asset to use for retargeting settings */
	const UIKRetargeter* RetargeterAsset = nullptr;
	/** The collection of settings used to initialize with */
	const FRetargetProfile* CustomProfile = nullptr;
	/** If true, will not output warnings during initialization */
	bool bSuppressWarnings = false;
};

/** Parameters for running the retargeter to generate a new pose. */
struct FRetargetRunParameters
{
	/** The source mesh input pose in Component/Global space */
	TArray<FTransform>* SourceGlobalPose = nullptr;
	/** The retarget profile to use for this update */
	const FRetargetProfile* Profile = nullptr;
	/** An array of override set names stored on the asset to apply */
	TArrayView<const FName> OverrideSetsToApply;
	/** A property bag housing variable values that may be bound to overrides */
	FInstancedPropertyBag* Variables = nullptr;
	/** A map of curve names to float values for any curves bound to an override */
	TMap<FName, float>* BoundCurveValues = nullptr;;
	/** Time since last tick in seconds (passed to ops) */
	float DeltaTime = 0.0f;
	/** The current LOD level for the skeletal mesh */
	int32 LOD = INDEX_NONE;
};

/** The runtime processor that converts an input pose from a source skeleton into an output pose on a target skeleton.
 * To use:
 * 1. Initialize a processor with a Source/Target skeletal mesh and a UIKRetargeter asset.
 * 2. Call ScaleSourcePose() and pass in the global space source pose (see func comments why this is necessary)
 * 3. Call RunRetargeter and pass in a source pose as an array of global-space transforms
 * 3. RunRetargeter() returns an array of global space transforms for the target skeleton.
 */
struct FIKRetargetProcessor
{

public:

	/**
	* Initialize the retargeter to enable running it.
	* @param Params - The initialization settings and source/target skeletons.
	*/
	UE_API void Initialize(const FRetargetInitParameters& Params);

	/**
	* Run the retarget to generate a new pose.
	* @param Params - The input pose and runtime configuration.
	* @return The retargeted Component/Global space pose for the target skeleton
	*/
	UE_API TArray<FTransform>& RunRetargeter(const FRetargetRunParameters& Params);

	/** Get whether this processor is ready to call RunRetargeter() and generate new poses. */
	UE_API bool IsInitialized() const;

	/** Set that this processor needs to be reinitialized. */
	UE_API void SetNeedsInitialized();

	/** Get whether this processor was initialized with these skeletal meshes and retarget asset*/
	UE_API bool WasInitializedWithTheseAssets(
		const USkeletalMesh* InSourceMesh,
		const USkeletalMesh* InTargetMesh,
		const UIKRetargeter* InRetargetAsset) const;

	/** Does a partial reinitialization (at runtime) whenever the retarget pose is swapped to a different or if the
	 * pose has been modified. Does nothing if the pose has not changed. */
	UE_API void UpdateRetargetPoseAtRuntime(const FName RetargetPoseToUseName, ERetargetSourceOrTarget SourceOrTarget);

	/** Get read-only access to either source or target skeleton. */
	UE_API const FRetargetSkeleton& GetSkeleton(ERetargetSourceOrTarget SourceOrTarget) const;

	/** Get read/write access to the target skeleton.
	 * NOTE: this skeleton contains the output pose of the target skeleton */
	UE_API FTargetSkeleton& GetTargetSkeleton();
	UE_API const FTargetSkeleton& GetTargetSkeleton() const;

	/** Get read-only access to all the fully resolved bone chains for both source and target skeletons */
	const FRetargeterBoneChains& GetBoneChains() const { return AllBoneChains; };

	/** Get read-write access so that ops can modify the IK Rig goals
	 * NOTE: pointers to goals in the container are not stable between updates, do not store them.
	 * NOTE: any op that sets a goal position must also specify the space of the goal (they are all additive by default) */
	FIKRigGoalContainer& GetIKRigGoalContainer() { return GoalContainer; };
	const FIKRigGoalContainer& GetIKRigGoalContainer() const { return GoalContainer; };
	
	/** Get read only access to the retarget ops currently running in processor */
	const TArray<FInstancedStruct>& GetRetargetOps() const {return OpStack; };

	/** Get list of ops of a given type */
	UE_API TArray<const FIKRetargetOpBase*> GetRetargetOpsByType(const UScriptStruct* OpType) const;

	/** Get an op with the given name */
	UE_API FIKRetargetOpBase* GetRetargetOpByName(const FName InOpName);

	/** Get the first op in the stack of the given type */
	template <typename T>
	const T* GetFirstRetargetOpOfType(ERetargetOpsToSearch SourceOfOps = ERetargetOpsToSearch::ProcessorOps) const
	{
		// get the op stack to search (either processor or asset)
		const TArray<FInstancedStruct>* OpsToSearch;
		if (SourceOfOps == ERetargetOpsToSearch::AssetOps)
		{
			if (!RetargeterAsset)
			{
				return nullptr;
			}
			OpsToSearch = &RetargeterAsset->GetRetargetOps();
		}
		else
		{
			OpsToSearch = &OpStack;
		}
		
		// search the op stack for first op of the given type
		for (const FInstancedStruct& OpStruct : *OpsToSearch)
		{
			if (OpStruct.GetScriptStruct()->IsChildOf(T::StaticStruct()))
			{
				return OpStruct.GetPtr<T>();
			}
		}

		return nullptr;
	}

	/** Returns true if the IK has not been forced off */
	bool IsIKForcedOff() const { return bIKForcedOff; };

	/** Get read only access to the retarget asset */
	const UIKRetargeter* GetRetargetAsset() const { return RetargeterAsset; };
	
	/** Calls OnPlaybackReset() for all ops in the stack */
	UE_API void OnPlaybackReset();

	/** Calls AnimGraphPreUpdateMainThread() for all ops in the stack */
	UE_API void OnAnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent);

	/** Calls AnimGraphEvaluateAnyThread() for all ops in the stack */
	UE_API void OnAnimGraphEvaluateAnyThread(FPoseContext& Output);

	/** Get name of the pelvis bone for either the source or target skeleton. */
	UE_API FName GetPelvisBone(ERetargetSourceOrTarget SourceOrTarget, ERetargetOpsToSearch InOpsToSearch) const;

	/**
	 * This scales the input source pose according to the source scale factor
	 * This must be called outside of the retargeter itself because otherwise the pose may be scaled multiple times.
	 * This is because we don't copy the source pose and we don't control when it's updated.
	 * By explicitly scaling outside of RunRetargeter(), outside systems can scale it whenever they update it.
	 * @param InOutSourceGlobalPose: an array of global space transforms for each bone in the source input pose
	 */
	UE_API void ApplySourceScaleToPose(TArray<FTransform>& InOutSourceGlobalPose) const;

	/** Get the scale factor for the source pose (comes from presence of SourceScaleOp)*/
	UE_API double GetSourceScaleFactor() const;

	/** Get the scale factor and pivot for the retarget pose (comes from presence of SourceScaleOp)*/
	UE_API const FRetargetPoseScaleWithPivot& GetPoseScale(const ERetargetSourceOrTarget SourceOrTarget) const;

	/** The logging system */
	FIKRigLogger Log;
	
#if WITH_EDITOR
	
private:
	DECLARE_MULTICAST_DELEGATE(FOnRetargeterInitialized);
	FOnRetargeterInitialized RetargeterInitialized;
public:
	/** Returns true if the bone is part of a retarget chain or root bone, false otherwise. */
	UE_API bool IsBoneMapped(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget) const;
	/** Returns index of the bone with the given name in either Source or Target skeleton. */
	UE_API int32 GetBoneIndexFromName(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget) const;
	/** Get access to all the fully resolved bone chain data */
	UE_API const FRetargeterBoneChains& GetBoneChains();
	/** Returns name of the chain associated with this bone. Returns NAME_None if bone is not in a chain. */
	UE_API FName GetFirstChainForBone(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget) const;
	/** Returns names of all the chains associated with this bone. Returns empty set if bone is not in a chain. */
	UE_API TSet<FName> GetAllChainNamesForBone(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget) const;
	// get the first mapping that contains the given bone, returns false if no mapping exists
	UE_API bool GetFirstChainMappingForBone(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget, FName& OutChainContainingBone, FName& OutMappedChain) const;
	/** Get a transform at a given param in a chain */
	UE_API FTransform GetGlobalRetargetPoseAtParam(const FName InChainName, const float Param, const ERetargetSourceOrTarget SourceOrTarget) const;
	/** Get a transform of a bone in the retarget pose */
	UE_API FTransform GetRetargetPoseBoneTransform(const FName InBoneName, const ERetargetSourceOrTarget SourceOrTarget, ERetargetBoneSpace BoneSpace) const;
	
	/** Get the param of the bone in it's retarget chain. Ranges from 0 to NumBonesInChain. */
	UE_API float GetParamOfBoneInChain(const FName InBoneName, const ERetargetSourceOrTarget SourceOrTarget, const FName InChain = NAME_None) const;
	/** Get the bone in the chain at the given param. */
	UE_API FName GetClosestBoneToParam(const FName InChainName, const float InParam, const ERetargetSourceOrTarget SourceOrTarget) const;
	/** Get the chain mapped to this one. */
	UE_API FName GetFirstChainMappedToChain(const FName InChainName, const ERetargetSourceOrTarget InSourceOrTarget) const;
	/** Attach a delegate to be notified whenever this processor is initialized. */
	FOnRetargeterInitialized& OnRetargeterInitialized(){ return RetargeterInitialized; };
	/** Run debug drawing on all ops in the stack */
	UE_API void DebugDrawAllOps(FPrimitiveDrawInterface* InPDI, const FIKRetargetDebugDrawState& EditorState) const;
	
	/** Run UpdateFromAnimSequence on all ops */
	UE_API void UpdateOpsFromAnimSequence(UAnimSequence* Sequence, float FrameTime);
	
#endif

	// BEGIN DEPRECATED API
	UE_DEPRECATED(5.8, "Please use the Initialize overload that takes FRetargetInitParameters.")
	UE_API void Initialize(
		const USkeletalMesh* SourceSkeleton,
		const USkeletalMesh* TargetSkeleton,
		const UIKRetargeter* InRetargeterAsset,
		const FRetargetProfile& InCustomProfile,
		const bool bSuppressWarnings = false);
	UE_DEPRECATED(5.8, "Please use the RunRetargeter overload that takes FRetargetRunParameters.")
	UE_API TArray<FTransform>& RunRetargeter(
		TArray<FTransform>& InSourceGlobalPose,
		const FRetargetProfile& InProfile,
		const float InDeltaTime,
		const int32 InLOD = -1);
	// END DEPRECATED API
	
private:

	/** If true, all IK operations are skipped
	 * NOTE: this is used both for debugging and LOD'ing IK off */
	bool bIKForcedOff = false;
	
	/** Only true once Initialize() has successfully completed.*/
	bool bIsInitialized = false;
	int32 AssetVersionInitializedWith = -2;

	/** The source asset this processor was initialized with. */
	const UIKRetargeter* RetargeterAsset = nullptr;

	/** The internal data structures used to represent the SOURCE skeleton / pose during retargeter.*/
	FRetargetSkeleton SourceSkeleton;

	/** The internal data structures used to represent the TARGET skeleton / pose during retargeter.*/
	FTargetSkeleton TargetSkeleton;

	/** the named transforms that solvers use as end effectors */
	FIKRigGoalContainer GoalContainer;

	/** Storage for all bone chain data used by Ops and editor tools */
	FRetargeterBoneChains AllBoneChains;

	/** The collection of operations to run to transfer animation from source to target. */
	TArray<FInstancedStruct> OpStack;

	/** Manages the application of stored property overrides at runtime. */
	FIKRetargetRuntimeOverrideManager OverrideManager;

	/** cached source scaling */
	mutable FRetargetPoseScaleWithPivot CurrentSourceScale;

	/** Apply the settings stored in a retarget profile. As well as any stored property overrides. */
	void UpdateSettingsAtRuntime(
		const FRetargetProfile* InProfile,
		const TArrayView<const FName> InOverrideSetsToApply,
		FInstancedPropertyBag* InVariables,
		TMap<FName, float>* InBoundCurveValues);

	/** Internal retarget phase that does simple bone-to-bone copying from source*/
	void GenerateBasePoses(TArray<FTransform>& InSourceGlobalPose);

	/** Initial setup of the retarget op stack (copies op stack from asset into processor) */
	void InitialOpStackSetup(
		const TArray<FInstancedStruct>& OpsFromAsset,
		const FRetargetProfile* InRetargetProfile);

	/** Run Initialize() on all ops then collects retargeted bones and call PostInitialize() on each Op */
	void InitializeRetargetOps();

	/** Run all post process operations on the retargeted result. */
	void RunRetargetOps(
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose,
		const double InDeltaTime,
		const int32 InLOD);

	/** Helper function to determine if an op should run this frame */
	static bool ShouldExecuteOp(const FIKRetargetOpBase& InOp, int32 InLOD);

	struct FOpEntry
	{
		FIKRetargetOpBase* Op = nullptr;
		FOpEntry* Parent = nullptr;
		TArray<FOpEntry*> Children;

		int32 StackIndex = INDEX_NONE;
		bool bExecute = false;
	};
	
	enum class ERetargetOpPhase : uint8
	{
		// initialization
		InitializeBeforeChildren,
		Initialize,
		PostInitializeProcessor,
		PostInitializeOp,
		
		// runtime
		RunBeforeChildren,
		Run,
		RunAfterParent
	};

	struct FRetargetOpCall
	{
		FRetargetOpCall(const ERetargetOpPhase InPhase, FOpEntry* InOpEntry) : Phase(InPhase), OpEntry(InOpEntry) {}
		FIKRetargetOpBase* GetOp();
		FIKRetargetOpBase* GetParentOp();
		
		ERetargetOpPhase Phase;
	private:
		FOpEntry* OpEntry;
	};

	void BuildExecutionStack(int32 InLOD, TArray<FRetargetOpCall>& OutStack);

	void BuildInitializationStack(TArray<FRetargetOpCall>& OutStack);

	void InitializeOpEntries();
	TArray<FOpEntry> OpEntries;
};

//
// BEGIN DEPRECATED UOBJECT-BASED PROCESSOR
//
// NOTE: As of 5.6, IK Retarget processor is no longer a UObject.
// The UObject based processor is here for backwards compatibility.
// If you have a system built off this processor, please convert it to the FIKRetargetProcessor as this will be removed in a future update.
//
PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS()
class UE_DEPRECATED(5.6, "UIKRetargetProcessor has been replaced by FIKRetargetProcessor. UIKRetargetProcessor is now a thin wrapper around FIKRetargetProcessor and will be removed in a future version.")
IKRIG_API UIKRetargetProcessor : public UObject
{
	GENERATED_BODY()
	
public:
	
	void Initialize(
		const USkeletalMesh *SourceSkeleton,
		const USkeletalMesh *TargetSkeleton,
		const UIKRetargeter* InRetargeterAsset,
		const FRetargetProfile& Settings,
		const bool bSuppressWarnings=false)
	{
		Processor.Initialize(SourceSkeleton, TargetSkeleton, InRetargeterAsset, Settings, bSuppressWarnings);
	}
	
	TArray<FTransform>& RunRetargeter(
		TArray<FTransform>& InSourceGlobalPose,
		const TMap<FName,float>& SpeedValuesFromCurves,
		const float DeltaTime,
		const FRetargetProfile& Profile)
	{
		return Processor.RunRetargeter(InSourceGlobalPose, Profile, DeltaTime);
	}

	void UpdateRetargetPoseAtRuntime(
		const FName NewRetargetPoseName,
		ERetargetSourceOrTarget SourceOrTarget)
	{
		Processor.UpdateRetargetPoseAtRuntime(NewRetargetPoseName, SourceOrTarget);
	}

	const FRetargetSkeleton& GetSkeleton(ERetargetSourceOrTarget SourceOrTarget) const
	{
		return Processor.GetSkeleton(SourceOrTarget);
	}
	FName GetRetargetRoot(ERetargetSourceOrTarget SourceOrTarget) const
	{
		return Processor.GetPelvisBone(SourceOrTarget, ERetargetOpsToSearch::AssetOps);
	}
	bool IsInitialized() const
	{
		return Processor.IsInitialized();
	}
	bool WasInitializedWithTheseAssets(
		const USkeletalMesh* InSourceMesh,
		const USkeletalMesh* InTargetMesh,
		const UIKRetargeter* InRetargetAsset) const
	{
		return Processor.WasInitializedWithTheseAssets(InSourceMesh, InTargetMesh, InRetargetAsset);
	}
	const TArray<TObjectPtr<URetargetOpBase>>& GetRetargetOps() const
	{
		return DummyStack;
	};
	const FRetargetGlobalSettings& GetGlobalSettings() const
	{
		return DummyGlobalSettings;
	};
	void ResetPlanting()
	{
		Processor.OnPlaybackReset();
	};
	void SetNeedsInitialized()
	{
		Processor.SetNeedsInitialized();
	};

	FIKRigLogger Log;
	
#if WITH_EDITOR
	bool IsBoneRetargeted(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget) const
	{
		return Processor.IsBoneMapped(BoneName, SourceOrTarget);
	}
	int32 GetBoneIndexFromName(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget) const
	{
		return Processor.GetBoneIndexFromName(BoneName, SourceOrTarget);
	}
	FName GetChainNameForBone(const FName BoneName, const ERetargetSourceOrTarget SourceOrTarget) const
	{
		return Processor.GetFirstChainForBone(BoneName, SourceOrTarget);
	}
	FTransform GetGlobalRetargetPoseAtParam(const FName InChainName, const float Param, const ERetargetSourceOrTarget SourceOrTarget) const
	{
		return Processor.GetGlobalRetargetPoseAtParam(InChainName, Param, SourceOrTarget);
	}
	FTransform GetRetargetPoseBoneTransform(const FName InBoneName, const ERetargetSourceOrTarget SourceOrTarget, ERetargetBoneSpace BoneSpace) const
	{
		return Processor.GetRetargetPoseBoneTransform(InBoneName, SourceOrTarget, BoneSpace);
	}
	float GetParamOfBoneInChain(const FName InBoneName, const ERetargetSourceOrTarget SourceOrTarget) const
	{
		return Processor.GetParamOfBoneInChain(InBoneName, SourceOrTarget);
	}
	FName GetClosestBoneToParam(const FName InChainName, const float InParam, const ERetargetSourceOrTarget SourceOrTarget) const
	{
		return Processor.GetClosestBoneToParam(InChainName, InParam, SourceOrTarget);
	}
	FName GetMappedChainName(const FName InChainName, const ERetargetSourceOrTarget SourceOrTarget)
	{
		return Processor.GetFirstChainMappedToChain(InChainName, SourceOrTarget);
	}
#endif


private:
	FRetargetGlobalSettings DummyGlobalSettings;
	TArray<TObjectPtr<URetargetOpBase>> DummyStack;
	FIKRetargetProcessor Processor;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
//
// END DEPRECATED UOBJECT-BASED PROCESSOR
//

#undef UE_API
