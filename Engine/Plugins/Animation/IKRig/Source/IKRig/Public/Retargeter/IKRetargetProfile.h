// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "IKRetargetOps.h"
#include "Retargeter/IKRetargetDeprecated.h"
#include "StructUtils/InstancedStruct.h"

#include "IKRetargetProfile.generated.h"

#define UE_API IKRIG_API

class UIKRetargeter;
class URetargetChainSettings;
struct FIKRetargetOpSettingsBase;

UENUM()
enum class ECopyOpSettingsContext : uint8
{
	PreInitialize,	// copies ALL settings (used in editor during setup)
	Runtime			// copy only settings that don't require reinitialization
};

/** A wholesale copy of the settings in a retarget op. */
USTRUCT(BlueprintType)
struct FRetargetOpProfile
{
	GENERATED_BODY()

	FRetargetOpProfile() = default;

	UE_API FRetargetOpProfile(
		const FName InOpName,
		const UScriptStruct* InSettingsType,
		const FIKRetargetOpSettingsBase* InOpSettings);

	/** * Applies the stored settings from this profile into the provided operation's instance data.
	 * @param InOutOpStruct     The destination operation struct to receive these settings.
	 * @param InApplyContext    The execution context (e.g., Runtime vs Editor). In Runtime mode, 
	 * overrides that require reinitialization will be skipped.
	 * @return True if the settings were compatible and successfully applied, false otherwise. 
	 */
	bool CopySettingsToOp(FInstancedStruct& InOutOpStruct, ECopyOpSettingsContext InApplyContext) const;
	
	/**
	 * Duplicates all settings from another op profile.
	 * @param OtherOpProfile    The source profile to copy settings data from.
	 */
	void CopyFromOtherOpProfile(const FRetargetOpProfile& OtherOpProfile);

	/** * Helper utility for operations to instantiate and retrieve their associated controller.
	 * @param Outer             The UObject that will own the newly created controller.
	 * @return A pointer to the existing or newly created controller base.
	 */
	UIKRetargetOpControllerBase* CreateControllerIfNeeded(UObject* Outer);

	/** * The specific name of the operation in the stack these settings should target.
	 * If NAME_None, settings are applied to all operations matching the 'SettingsToApply' type. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "IK Retarget Op Profile")
	FName OpToApplySettingsTo = NAME_None;

	/** * The collection of settings to override for a specific operation.
	 * Note: While all settings here are applied in the editor, operations independently 
	 * filter which settings are safe to apply at runtime without re-initializing. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "IK Retarget Op Profile", meta = (ExcludeBaseStruct, BaseStruct = "/Script/IKRig.IKRetargetOpSettingsBase"))
	FInstancedStruct SettingsToApply;
	
private:
	/** the controller used to edit these op settings by script/blueprint (lazy instantiated when needed) */
	TStrongObjectPtr<UIKRetargetOpControllerBase> Controller = nullptr;
};

/** A collection of FRetargetOpProfiles that can be applied at runtime to override settings for one or more ops. */
USTRUCT(BlueprintType)
struct FRetargetProfile
{
	GENERATED_BODY()
	
public:

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// implicit copy assignment operator raises deprecation warnings (remove this once the deprecated properties are gone)
	FRetargetProfile() = default;
	FRetargetProfile(const FRetargetProfile&) = default;
	FRetargetProfile(FRetargetProfile&&) = default;
	FRetargetProfile& operator=(const FRetargetProfile&) = default;
	FRetargetProfile& operator=(FRetargetProfile&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** A polymorphic list of override settings to apply to retargeting operations in the stack */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings")
	TArray<FRetargetOpProfile> RetargetOpProfiles;

	/** If true, the TARGET Retarget Pose specified in this profile will be applied to the Retargeter (when plugged into the Retargeter). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Retarget Poses", meta = (DisplayName = "Override Target Retarget Pose"))
	bool bApplyTargetRetargetPose = false;
	
	/** Override the TARGET Retarget Pose to use when this profile is active.
	 * The pose must be present in the Retarget Asset and is not applied unless bApplyTargetRetargetPose is true. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Retarget Poses", meta=(EditCondition="bApplyTargetRetargetPose"))
	FName TargetRetargetPoseName;

	/** If true, the Source Retarget Pose specified in this profile will be applied to the Retargeter (when plugged into the Retargeter). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Retarget Poses", meta = (DisplayName = "Override Source Retarget Pose"))
	bool bApplySourceRetargetPose = false;

	/** Override the SOURCE Retarget Pose to use when this profile is active.
	 * The pose must be present in the Retarget Asset and is not applied unless bApplySourceRetargetPose is true.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Retarget Poses", meta=(EditCondition="bApplySourceRetargetPose"))
	FName SourceRetargetPoseName;

	/** Globally forces all IK solving off */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "IK Retarget Settings", DisplayName="Force All IK Off")
	bool bForceAllIKOff = false;
	
	UE_API void FillProfileWithAssetSettings(const UIKRetargeter* InAsset);

	UE_API void MergeWithOtherProfile(const FRetargetProfile& OtherProfile);

	UE_API FRetargetOpProfile* FindMatchingOpProfile(const FRetargetOpProfile& OtherOpProfile);

	UE_API bool ApplyOpProfilesToOpStruct(FInstancedStruct& InOutOpStruct, const ECopyOpSettingsContext InCopyContext) const;

	UE_API FRetargetOpProfile* GetOpProfileByName(const FName InOpName);
	
	// search all the op profiles and get an array of all the settings of a particular type
	template <typename T>
	void GetOpSettingsByTypeInProfile(TArray<T*>& OutMatchingOpSettings)
	{
		const UScriptStruct* SettingsTypeToMatch = T::StaticStruct();
		
		for (FRetargetOpProfile& OpProfile : RetargetOpProfiles)
		{
			FInstancedStruct& OpSettingsStruct = OpProfile.SettingsToApply;
			if (OpSettingsStruct.GetScriptStruct() == SettingsTypeToMatch)
			{
				OutMatchingOpSettings.Add(OpSettingsStruct.GetMutablePtr<T>());
			}
		}
	}

	//
	// BEGIN DEPRECATED PROFILE SETTINGS
	// NOTE: these are still stored and applied but should be removed and replaced with Op profiles
	// NOTE: I cannot use the _DEPRECATED suffix on these properties because they live in a USTRUCT
	// and UBT will complain that deprecated properties cannot be blueprint exposed without a getter but USTRUCT does not support UFUNCTIONS/getters
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// consolidating/isolating deprecation code
	void MergeDeprecatedProperties(const FRetargetProfile& OtherProfile);
	
	// (Deprecated) If true, the Chain Settings stored in this profile will be applied to the Retargeter (when plugged into the Retargeter).
	UE_DEPRECATED(5.6, "Modifying chain settings must go through a retarget op profile.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Deprecated", meta=(DisplayName="Apply Chain Settings (Deprecated)"))
	bool bApplyChainSettings = false;
	
	// (Deprecated) A (potentially sparse) set of setting overrides for the target chains (only applied when bApplyChainSettings is true).
	UE_DEPRECATED(5.6, "Modifying chain settings must go through a retarget op profile.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Deprecated", meta=(DisplayName="Chain Settings (Deprecated)"))
	TMap<FName, FTargetChainSettings> ChainSettings;

	// (Deprecated) If true, the root settings stored in this profile will be applied to the Retargeter (when plugged into the Retargeter).
	UE_DEPRECATED(5.6, "Modifying root settings must go through a retarget op profile.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Deprecated", meta=(DisplayName="Apply Root Settings (Deprecated)"))
	bool bApplyRootSettings = false;

	// (Deprecated) Retarget settings to control behavior of the retarget root motion (not applied unless bApplyRootSettings is true)
	UE_DEPRECATED(5.6, "Modifying root settings must go through a retarget op profile.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Deprecated", meta=(DisplayName="Root Settings (Deprecated)"))
	FTargetRootSettings RootSettings;

	// (Deprecated) If true, the global settings stored in this profile will be applied to the Retargeter (when plugged into the Retargeter).
	UE_DEPRECATED(5.6, "Modifying global settings must go through a retarget op profile.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Deprecated", meta=(DisplayName="Apply Global Settings (Deprecated)"))
	bool bApplyGlobalSettings = false;

	// (Deprecated) Retarget settings to control global behavior, like Stride Warping (not applied unless bApplyGlobalSettings is true)
	UE_DEPRECATED(5.6, "Modifying global settings must go through a retarget op profile.")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Deprecated", meta=(DisplayName="Global Settings (Deprecated)"))
	FRetargetGlobalSettings GlobalSettings;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//
	// END DEPRECATED API
	//
};

UCLASS()
class URetargetProfileLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	
	UFUNCTION(BlueprintCallable, Category = "Retarget Profile")
	static FRetargetProfile CopyRetargetProfileFromRetargetAsset(const UIKRetargeter* InRetargetAsset);
	
	UFUNCTION(BlueprintPure, Category = "Retarget Profile")
	static UIKRetargetOpControllerBase* GetOpControllerFromRetargetProfile(UPARAM(ref) FRetargetProfile& InRetargetProfile, const FName InRetargetOpName);
};

#undef UE_API
