// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IKRetargetSettings.generated.h"

class UPrimitiveComponent;

// which skeleton are we referring to?
UENUM(BlueprintType)
enum class ERetargetSourceOrTarget : uint8
{
	// the SOURCE skeleton (to copy FROM)
	Source,
	// the TARGET skeleton (to copy TO)
	Target,
};

UENUM()
enum class EBasicAxis
{
	X,
	Y,
	Z,
	NegX,
	NegY,
	NegZ
};

#if WITH_EDITOR
enum class ERetargetSelectionType : uint8
{
	NONE,
	BONE,
	CHAIN,
	MESH,
	ROOT,
	OP
};

struct FIKRetargetDebugDrawState
{
	// state of all things you can select in the retarget editor viewport

	// what type of objects was last selected?
	ERetargetSelectionType LastSelectedType = ERetargetSelectionType::NONE;
	// BONE
	bool bIsRootSelected = false;
	TMap<ERetargetSourceOrTarget, TArray<FName>> SelectedBoneNames;
	// CHAINS
	TArray<FName> SelectedChains;
	// MESH
	UPrimitiveComponent* SelectedMesh = nullptr;
	// GOALS
	TArray<FName> SelectedGoals;
	// OPS
	FName LastSelectedOpName;

	// common viewport drawing color palette for selected things
	static FLinearColor Muted;
	static FLinearColor SourceColor; 
	static FLinearColor GoalColor;
	static FLinearColor MainColor;
	static FLinearColor NonSelected;
};
#endif

