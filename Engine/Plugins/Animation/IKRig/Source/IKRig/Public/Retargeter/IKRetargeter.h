// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IKRetargetChainMapping.h"
#include "IKRetargetProfile.h"
#include "IKRetargetOverrides.h"
#include "Rig/IKRigDefinition.h"
#include "IKRetargetSettings.h"
#include "StructUtils/PropertyBag.h"

#include "IKRetargeter.generated.h"

#define UE_API IKRIG_API

#if WITH_EDITOR
class FIKRetargetEditorController;
#endif

class URetargetOpStack;

USTRUCT()
struct FRetargetVariableContainer
{
	GENERATED_BODY()
	
	UPROPERTY()
	FInstancedPropertyBag Bag;
};

USTRUCT(BlueprintType)
struct FIKRetargetPose
{
	GENERATED_BODY()
	
public:
	
	FIKRetargetPose() = default;

	UE_API FQuat GetDeltaRotationForBone(const FName BoneName) const;
	UE_API void SetDeltaRotationForBone(FName BoneName, const FQuat& RotationDelta);
	const TMap<FName, FQuat>& GetAllDeltaRotations() const { return BoneRotationOffsets; };

	UE_API FVector GetRootTranslationDelta() const;
	UE_API void SetRootTranslationDelta(const FVector& TranslationDelta);
	UE_API void AddToRootTranslationDelta(const FVector& TranslationDelta);
	
	UE_API void SortHierarchically(const FIKRigSkeleton& Skeleton);
	
	int32 GetVersion() const { return Version; };
	void IncrementVersion() { ++Version; };

private:
	// a translational delta in GLOBAL space, applied only to the pelvis bone
	UPROPERTY(EditAnywhere, Category = RetargetPose)
	FVector RootTranslationOffset = FVector::ZeroVector;

	// these are LOCAL-space rotation deltas to be applied to a bone to modify it's retarget pose
	UPROPERTY(EditAnywhere, Category = RetargetPose)
	TMap<FName, FQuat> BoneRotationOffsets;
	
	// incremented by any edits to the retarget pose, indicating to any running instance that it should reinitialize
	// this is not made "editor only" to leave open the possibility of programmatically modifying a retarget pose in cooked builds
	int32 Version = INDEX_NONE;

	friend class UIKRetargeterController;
};

UCLASS(MinimalAPI, BlueprintType)
class UIKRetargeter : public UObject
{
	GENERATED_BODY()
	
public:
	
	UE_API UIKRetargeter(const FObjectInitializer& ObjectInitializer);

	// Get read-only access to the source or target IK Rig asset 
	UE_API const UIKRigDefinition* GetIKRig(ERetargetSourceOrTarget SourceOrTarget) const;
	// Get read-write access to the source IK Rig asset.
	// WARNING: do not use for editing the data model. Use Controller class instead. 
	UE_API UIKRigDefinition* GetIKRigWriteable(ERetargetSourceOrTarget SourceOrTarget) const;
	#if WITH_EDITORONLY_DATA
	// Get read-only access to preview meshes
	UE_API USkeletalMesh* GetPreviewMesh(ERetargetSourceOrTarget SourceOrTarget) const;
	#endif
	
	// Get access to the stack of retargeting operations
	const TArray<FInstancedStruct>& GetRetargetOps() const { return RetargetOps; };
	/** Get the first op in the stack of the given type */
	template <typename T>
	T* GetFirstRetargetOpOfType()
	{
		for (FInstancedStruct& OpStruct : RetargetOps)
		{
			if (OpStruct.GetScriptStruct()->IsChildOf(T::StaticStruct()))
			{
				return OpStruct.GetMutablePtr<T>();
			}
		}

		return nullptr;
	}
	/** Get all ops in the stack of the given type */
	template <typename T>
	TArray<T*> GetAllRetargetOpsOfType()
	{
		TArray<T*> Results;
    
		for (FInstancedStruct& OpStruct : RetargetOps)
		{
			if (OpStruct.GetScriptStruct()->IsChildOf(T::StaticStruct()))
			{
				if (T* Instance = OpStruct.GetMutablePtr<T>())
				{
					Results.Add(Instance);
				}
			}
		}
    
		return Results;
	}
	// Get a retarget op by name
	UE_API const FIKRetargetOpBase* GetRetargetOpByName(const FName& InOpName) const;
	// Get a retarget op by name non-const
	// NOTE: be careful with this, general op-editing should be done through a controller
	UE_API FIKRetargetOpBase* GetRetargetOpByNameNonConst(const FName& InOpName);
	// Get index of a retarget op
	UE_API int32 GetIndexOfRetargetOp(const FIKRetargetOpBase* InOp) const;
	// Get read-only access to a retarget pose 
	UE_API const FIKRetargetPose* GetCurrentRetargetPose(const ERetargetSourceOrTarget& SourceOrTarget) const;
	// Get name of the current retarget pose 
	UE_API FName GetCurrentRetargetPoseName(const ERetargetSourceOrTarget& SourceOrTarget) const;
	// Get read-only access to a retarget pose 
	UE_API const FIKRetargetPose* GetRetargetPoseByName(const ERetargetSourceOrTarget& SourceOrTarget, const FName PoseName) const;
	// Get name of default pose 
	static UE_API const FName GetDefaultPoseName();
	
	// Get the current retarget profile (may be null) 
	UE_API const FRetargetProfile* GetCurrentProfile() const;
	// Get the retarget profile by name (may be null) 
	UE_API const FRetargetProfile* GetProfileByName(const FName& ProfileName) const;

	// Get read-only access to all the override sets
	const TMap<FName, FRetargetOverrideSet>& GetOverrideSets() const { return OverrideSets; };
	// Get read-write access to all override sets (caution, all modifications should go through the controller)
	TMap<FName, FRetargetOverrideSet>& GetOverrideSetsWriteable() { return OverrideSets; };
	// Get all the override sets that are enabled
	UE_API const TArray<FName>& GetOverrideSetsToApply() const;
	// Get a set of overrides by name (may be null) 
	UE_API const FRetargetOverrideSet* GetOverrideSetByName(const FName& OverrideName) const;
	// get current version of the property overrides data (to compare against running processor instances)
	int32 GetOverrideVersion() const { return OverridesVersion; };

	// get read-write access to the variable bag (caution here, all editing should go through controller) 
	FRetargetVariableContainer& GetVariables() { return Variables; };

	// get current version of the data (to compare against running processor instances)
	int32 GetVersion() const { return Version; };
	// do this after any edit that would require running instance to reinitialize
	void IncrementVersion() const { ++Version; };
	
	// Returns true if the source IK Rig has been assigned
	UFUNCTION(BlueprintPure, Category=RetargetProfile)
	bool HasSourceIKRig() const { return SourceIKRigAsset != nullptr; }

	// Returns true if the target IK Rig has been assigned
	UFUNCTION(BlueprintPure, Category=RetargetProfile)
	bool HasTargetIKRig() const { return TargetIKRigAsset != nullptr; }

	// UObject
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
	// END UObject
	#if WITH_EDITORONLY_DATA
    	static UE_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
    #endif

#if WITH_EDITOR
	// Get name of Source IK Rig property 
	static UE_API const FName GetSourceIKRigPropertyName();
	// Get name of Target IK Rig property 
	static UE_API const FName GetTargetIKRigPropertyName();
	// Get name of Source Preview Mesh property 
	static UE_API const FName GetSourcePreviewMeshPropertyName();
	// Get name of Target Preview Mesh property 
	static UE_API const FName GetTargetPreviewMeshPropertyName();
#endif

	//
	// BEGIN DEPRECATED API
	
	// (5.6 pre ops refactor)
	//
	// These were used to get/set Root/Chain/Global settings from outside systems
	// Because these settings are now scattered across the various ops that perform the equivalent functions,
	// we cannot provide backwards compatibility.
	// Instead you can use the new op controller API to get a controller for any retarget op and get/set it's settings.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	// Get read/write access to the chain mapping
	UE_DEPRECATED(5.6, "Chain mappings are managed by individual ops now. Use the asset or op controllers to edit them.")
	FRetargetChainMapping& GetChainMapping() { return ChainMap_DEPRECATED; };

	// Get read-only access to the chain mapping
	UE_DEPRECATED(5.6, "Chain mappings are managed by individual ops now. Use Op controllers to edit them.")
	const FRetargetChainMapping& GetChainMapping() const { return ChainMap_DEPRECATED; };
	
	// Get read-only access to the per-chain settings
	UE_DEPRECATED(5.6, "Chain settings are now accessed through an Op controller.")
	const TArray<TObjectPtr<URetargetChainSettings>>& GetAllChainSettings() const { return ChainSettings_DEPRECATED; };
	
	// Get read-only access to the chain map for a given chain (null if chain not in retargeter)
	UE_DEPRECATED(5.6, "Chain mappings are now accessed from GetChainMapping().")
	UE_API const TObjectPtr<URetargetChainSettings> GetChainMapByName(const FName& TargetChainName) const;
	
	// Get read-only access to the chain settings for a given chain (null if chain not in retargeter)
	UE_DEPRECATED(5.6, "Chain settings are now accessed through an Op controller.")
	UE_API const FTargetChainSettings* GetChainSettingsByName(const FName& TargetChainName) const;
	
	// Get access to the root settings
	UE_DEPRECATED(5.6, "Root settings are now accessed through a Pelvis Motion Op.")
	URetargetRootSettings* GetRootSettingsUObject() const { return RootSettings_DEPRECATED; };
	
	// Get access to the global settings uobject
	UE_DEPRECATED(5.6, "Global settings are now accessed through various Ops depending on the feature.")
	UIKRetargetGlobalSettings* GetGlobalSettingsUObject() const { return GlobalSettings_DEPRECATED; };
	
	// Get access to the global settings itself
	UE_DEPRECATED(5.6, "Global settings are now accessed through various Ops depending on the feature.")
	const FRetargetGlobalSettings& GetGlobalSettings() const { return GlobalSettings_DEPRECATED->Settings; };
	
	// Returns the chain settings associated with a given Goal in an IK Retargeter Asset using the given profile name (optional)
	UE_DEPRECATED(5.6, "Chain settings are now accessed through an Op.")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category=RetargetAsset, meta = (DeprecatedFunction, DeprecationMessage = "Use IK Chain Op controller to get chains with goals."))
	static UE_API FTargetChainSettings GetChainUsingGoalFromRetargetAsset(
		const UIKRetargeter* RetargetAsset,
		const FName IKGoalName);
	
	// Returns the chain settings associated with a given target chain in an IK Retargeter Asset using the given profile name (optional)
	UE_DEPRECATED(5.6, "Chain settings are now accessed through an Op.")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category=RetargetAsset, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set chain settings."))
	static UE_API FTargetChainSettings GetChainSettingsFromRetargetAsset(
		const UIKRetargeter* RetargetAsset,
		const FName TargetChainName,
		const FName OptionalProfileName);
	
	// Returns the chain settings associated with a given target chain in the supplied Retarget Profile.
	UE_DEPRECATED(5.6, "Chain settings are now accessed through an Op.")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set chain settings."))
	static UE_API FTargetChainSettings GetChainSettingsFromRetargetProfile(
		UPARAM(ref) FRetargetProfile& RetargetProfile,
		const FName TargetChainName);
	
	// Returns the root settings in an IK Retargeter Asset using the given profile name (optional)
	UE_DEPRECATED(5.6, "Root settings are now accessed through a Pelvis Motion Op.")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category=RetargetAsset, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set pelvis settings."))
	static UE_API void GetRootSettingsFromRetargetAsset(
		const UIKRetargeter* RetargetAsset,
		const FName OptionalProfileName,
		UPARAM(DisplayName = "ReturnValue") FTargetRootSettings& OutSettings);
	
	// Returns the root settings in the supplied Retarget Profile.
	UE_DEPRECATED(5.6, "Root settings are now accessed through a Pelvis Motion Op.")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set pelvis settings."))
	static UE_API FTargetRootSettings GetRootSettingsFromRetargetProfile(UPARAM(ref) FRetargetProfile& RetargetProfile);
	
	// Returns the global settings in an IK Retargeter Asset using the given profile name (optional)
	UE_DEPRECATED(5.6, "Global settings are now accessed through various Op.")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category=RetargetAsset, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set settings that previously used global settings."))
	static UE_API void GetGlobalSettingsFromRetargetAsset(
		const UIKRetargeter* RetargetAsset,
		const FName OptionalProfileName,
		UPARAM(DisplayName = "ReturnValue") FRetargetGlobalSettings& OutSettings);
	
	// Returns the global settings in the supplied Retarget Profile.
	UE_DEPRECATED(5.6, "Global settings are now accessed through various Op.")
	UFUNCTION(BlueprintCallable, BlueprintPure, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set settings that previously used global settings."))
	static UE_API FRetargetGlobalSettings GetGlobalSettingsFromRetargetProfile(UPARAM(ref) FRetargetProfile& RetargetProfile);
	
	// Set the global settings in a retarget profile (will set bApplyGlobalSettings to true). 
	UFUNCTION(BlueprintCallable, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set settings that previously used global settings."))
	UE_DEPRECATED(5.6, "Global settings are now accessed through various Op.")
	static UE_API void SetGlobalSettingsInRetargetProfile(
		UPARAM(ref) FRetargetProfile& RetargetProfile,
		const FRetargetGlobalSettings& GlobalSettings);
	
	// Set the root settings in a retarget profile (will set bApplyRootSettings to true).
	UE_DEPRECATED(5.6, "Root settings are now accessed through a Pelvis Motion Op.")
	UFUNCTION(BlueprintCallable, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set settings that previously used root settings."))
	static UE_API void SetRootSettingsInRetargetProfile(
		UPARAM(ref) FRetargetProfile& RetargetProfile,
		const FTargetRootSettings& RootSettings);
	
	// Set the chain settings in a retarget profile (will set bApplyChainSettings to true).
	UE_DEPRECATED(5.6, "Chain settings are now accessed through an Op controller.")
	UFUNCTION(BlueprintCallable, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use op profiles to get/set chain settings."))
	static UE_API void SetChainSettingsInRetargetProfile(
		UPARAM(ref) FRetargetProfile& RetargetProfile,
		const FTargetChainSettings& ChainSettings,
		const FName TargetChainName);
	
	// Set the chain FK settings in a retarget profile (will set bApplyChainSettings to true).
	UE_DEPRECATED(5.6, "FK Chain settings are now accessed through an FK Chain Op.")
	UFUNCTION(BlueprintCallable, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use FK Chain Op profiles to get/set FK chain settings."))
	static UE_API void SetChainFKSettingsInRetargetProfile(
		UPARAM(ref) FRetargetProfile& RetargetProfile,
		const FTargetChainFKSettings& FKSettings,
		const FName TargetChainName);
	
	// Set the chain IK settings in a retarget profile (will set bApplyChainSettings to true).
	UE_DEPRECATED(5.6, "IK Chain settings are now accessed through an IK Chain Op.")
	UFUNCTION(BlueprintCallable, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use IK Chain Op profiles to get/set IK chain settings."))
	static UE_API void SetChainIKSettingsInRetargetProfile(
		UPARAM(ref) FRetargetProfile& RetargetProfile,
		const FTargetChainIKSettings& IKSettings,
		const FName TargetChainName);
	
	// Set the chain Speed Plant settings in a retarget profile (will set bApplyChainSettings to true).
	UE_DEPRECATED(5.6, "Speed Plant settings are now accessed through a Speed Plant Op.")
	UFUNCTION(BlueprintCallable, Category=RetargetProfile, meta = (DeprecatedFunction, DeprecationMessage = "Use Speed Plant Op profiles to get/set speed plant settings."))
	static UE_API void SetChainSpeedPlantSettingsInRetargetProfile(
		UPARAM(ref) FRetargetProfile& RetargetProfile,
		const FTargetChainSpeedPlantSettings& SpeedPlantSettings,
		const FName TargetChainName);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//
	// END DEPRECATED API
	//

private:

#if WITH_EDITOR
	// upgrade old assets to new data formats
	void PostLoadOldSettingsToNew();
	UE_API void PostLoadOldOpsToNewStructOps();
	UE_API void PostLoadConvertEverythingToOps();
	UE_API void PostLoadPutChainMappingInOps();
	UE_API void PostLoadSplitIKChainsOp();
	UE_API void PostLoadSplitOps();
#endif
	UE_API void CleanRetargetPoses();
	UE_API void CleanOpStack();
	UE_API void PostLoadOpStack();
	
	// make the provided op name valid and unique
	// NOTE: if InIndexOfOp is INDEX_None, then it assumes this is a new op
	UE_API FName GetCleanAndUniqueOpName(const FName& InOpName, const int32 InIndexOfOp);
	
	// incremented by any edits that require re-initialization
	UPROPERTY(Transient)
	mutable int32 Version = INDEX_NONE;

	// The rig to copy animation FROM.
	UPROPERTY(EditAnywhere, Category = Source)
	TObjectPtr<UIKRigDefinition> SourceIKRigAsset = nullptr;

#if WITH_EDITORONLY_DATA
	// Optional. Override the Skeletal Mesh to copy animation from. Uses the preview mesh from the Source IK Rig asset by default. 
	UPROPERTY(EditAnywhere, Category = Source)
	TSoftObjectPtr<USkeletalMesh> SourcePreviewMesh = nullptr;
#endif
	
	/** The rig to copy animation TO. Note that this is only the default target IK Rig and ops can be setup to use other IK Rigs as desired. */  
	UPROPERTY(EditAnywhere, Category = Target, DisplayName="Default Target IK Rig")
	TObjectPtr<UIKRigDefinition> TargetIKRigAsset = nullptr;

#if WITH_EDITORONLY_DATA
	// Optional. Override the Skeletal Mesh to preview the retarget on. Uses the preview mesh from the Target IK Rig asset by default. 
	UPROPERTY(EditAnywhere, Category = Target)
	TSoftObjectPtr<USkeletalMesh> TargetPreviewMesh = nullptr;
#endif
	
public:

#if WITH_EDITORONLY_DATA
	
	// The offset applied to the target mesh in the editor viewport. 
	UPROPERTY(EditAnywhere, Category = PreviewOffset)
	FVector TargetMeshOffset;

	// Scale the target mesh in the viewport for easier visualization next to the source.
	UPROPERTY(EditAnywhere, Category = PreviewOffset, meta = (UIMin = "0.01", UIMax = "10.0"))
	float TargetMeshScale = 1.0f;

	// The offset applied to the source mesh in the editor viewport. 
	UPROPERTY(EditAnywhere, Category = PreviewOffset)
	FVector SourceMeshOffset;

	// Show/hide the source skeletal mesh in the editor viewport.
	UPROPERTY(EditAnywhere, Category = PreviewVisibility)
	bool bShowSourceMesh = true;

	// Show/hide the target skeletal mesh in the editor viewport.
	UPROPERTY(EditAnywhere, Category = PreviewVisibility)
	bool bShowTargetMesh = true;

	// Show/hide the source skeleton in the editor viewport. Note: the viewport must be showing bones to see the skeleton.
	UPROPERTY(EditAnywhere, Category = PreviewVisibility)
	bool bShowSourceSkeleton = true;

	// Show/hide the target skeleton in the editor viewport. Note: the viewport must be showing bones to see the skeleton.
	UPROPERTY(EditAnywhere, Category = PreviewVisibility)
	bool bShowTargetSkeleton = true;

	// Override the source skeleton color in the editor viewport.
	UPROPERTY(EditAnywhere, Category = SkeletonColorOverride)
	bool bOverrideSourceSkeletonColor = false;

	// Override the source skeleton color in the editor viewport.
	UPROPERTY(EditAnywhere, Category = SkeletonColorOverride)
	FLinearColor SourceOverideColor = FLinearColor::Blue;

	// Override the target skeleton color in the editor viewport.
	UPROPERTY(EditAnywhere, Category = SkeletonColorOverride)
	bool bOverrideTargetSkeletonColor = false;

	// Override the target skeleton color in the editor viewport.
	UPROPERTY(EditAnywhere, Category = SkeletonColorOverride)
	FLinearColor TargetOverideColor = FLinearColor::Red;

	// When true, animation sequences with "Force Root Lock" turned On will act as though it is Off.
	// This affects only the preview in the retarget editor. Use ExportRootLockMode to control exported animation behavior.
	// This setting has no effect on runtime retargeting where root motion is copied from the source component.
	UPROPERTY(EditAnywhere, Category = RootLockSettings)
	bool bIgnoreRootLockInPreview = true;

	// Toggle debug drawing for retargeting in the viewport. 
	UPROPERTY(EditAnywhere, Category = DebugSettings)
	bool bDebugDraw = true;

	// Toggle performance profiling of the op stack. 
	UPROPERTY(EditAnywhere, Category = DebugSettings)
	bool bProfileOps = false;
	
	// The visual size of the bones in the viewport (saved between sessions). This is set from the viewport Character>Bones menu
	UPROPERTY()
	float BoneDrawSize = 1.0f;

	// The controller responsible for managing this asset's data (all editor mutation goes through this)
	UPROPERTY(Transient, DuplicateTransient, NonTransactional )
	TObjectPtr<UObject> Controller;
	
private:
	
	// only ask to fix the root height once, then warn thereafter (don't nag) 
	TSet<TObjectPtr<USkeletalMesh>> MeshesAskedToFixRootHeightFor;
#endif
	
private:

	/** polymorphic stack of retargeting operations
	 * executed in serial fashion where output of prior operation is input to the next */
	UPROPERTY(meta = (ExcludeBaseStruct, BaseStruct = "/Script/IKRig.RetargetOpBase"))
	TArray<FInstancedStruct> RetargetOps;
	
	// whole setting struct profiles stored in this asset
	UPROPERTY()
	TMap<FName, FRetargetProfile> Profiles;
	UPROPERTY()
	FName CurrentProfile = NAME_None;

	// sparse property overrides to apply at runtime
	UPROPERTY()
	TMap<FName, FRetargetOverrideSet> OverrideSets;
	// the default set of override sets to be applied at runtime
	mutable TArray<FName> OverridesToApply;
	// updated anytime overrides are modified
	mutable int32 OverridesVersion = 0;
	
	// storage for all variables (used to dynamically adjust property overrides)
	UPROPERTY()
	FRetargetVariableContainer Variables;

	// The set of retarget poses for the SOURCE skeleton.
	UPROPERTY()
	TMap<FName, FIKRetargetPose> SourceRetargetPoses;
	// The set of retarget poses for the TARGET skeleton.
	UPROPERTY()
	TMap<FName, FIKRetargetPose> TargetRetargetPoses;
	
	// The current retarget pose to use for the SOURCE.
	UPROPERTY()
	FName CurrentSourceRetargetPose;
	// The current retarget pose to use for the TARGET.
	UPROPERTY()
	FName CurrentTargetRetargetPose;

	//
	// BEGIN DEPRECATED DATA
	//
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(meta = (DeprecatedProperty))
	bool bRetargetRoot_DEPRECATED = true;
	UPROPERTY(meta = (DeprecatedProperty))
	bool bRetargetFK_DEPRECATED = true;
	UPROPERTY(meta = (DeprecatedProperty))
	bool bRetargetIK_DEPRECATED = true;
	UPROPERTY(meta = (DeprecatedProperty))
	float TargetActorOffset_DEPRECATED = 0.0f;
	UPROPERTY(meta = (DeprecatedProperty))
	float TargetActorScale_DEPRECATED = 0.0f;
	// (OLD VERSION) Before retarget poses were stored for target AND source.
	UPROPERTY(meta = (DeprecatedProperty))
	TMap<FName, FIKRetargetPose> RetargetPoses;
	UPROPERTY(meta = (DeprecatedProperty))
	FName CurrentRetargetPose;
	//
	// BEGIN deprecated data from 5.6 refactor
	//
	UPROPERTY(meta = (DeprecatedProperty))
	FRetargetChainMapping ChainMap_DEPRECATED;
	// Settings for how to map source chains to target chains.
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use FK Chain and IK Chain Op settings instead to affect chain settings."))
	TArray<TObjectPtr<URetargetChainSettings>> ChainSettings_DEPRECATED;
	// the retarget root settings 
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Pelvis Op settings to affect the pelvis motion instead."))
	TObjectPtr<URetargetRootSettings> RootSettings_DEPRECATED;
	// the retarget global settings 
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Toggled 'phases' is now done by enable/disabling ops. Warping now in Stride Warp Op."))
	TObjectPtr<UIKRetargetGlobalSettings> GlobalSettings_DEPRECATED;
	// the stack of UObject-based ops
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "The pre 5.6 stack of UObject based solvers. Use RetargetOps instead."))
	TObjectPtr<URetargetOpStack> OpStack_DEPRECATED;
	//
	// END deprecated data from 5.6 refactor
	//
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//
	// END DEPRECATED DATA
	//

	friend class UIKRetargeterController;
};

#undef UE_API
