// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetSettings.h"

#include "AdditivePoseOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "RetargetPoseOp"

USTRUCT(BlueprintType, meta = (DisplayName = "Additive Pose Op Settings"))
struct FIKRetargetAdditivePoseOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	// a retarget pose that is applied additively to the output pose of the target skeleton
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Pose")
	FName PoseToApply;

	// blend the amount of the pose to apply
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Pose", meta=(ClampMin=0, ClampMax=1))
	float Alpha = 1.0f;
	
	UE_API virtual const UClass* GetControllerType() const override;
	
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Additive Pose"))
struct FIKRetargetAdditivePoseOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;

private:
	
	void ApplyAdditivePose(FIKRetargetProcessor& InProcessor, TArray<FTransform>& OutTargetGlobalPose);

	// cached in Initialize()
	FName PelvisBoneName;

	UPROPERTY()
	FIKRetargetAdditivePoseOpSettings Settings;
};

/* The blueprint/python API for editing a Retarget Pose Op */
UCLASS(BlueprintType)
class UIKRetargetAdditivePoseController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetPoseOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetAdditivePoseOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetPoseOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetAdditivePoseOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
