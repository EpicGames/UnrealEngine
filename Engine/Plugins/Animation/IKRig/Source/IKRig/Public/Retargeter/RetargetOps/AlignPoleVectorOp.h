// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetChainMapping.h"
#include "Retargeter/IKRetargetOps.h"

#include "AlignPoleVectorOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "AlignPoleVectorOp"

struct FResolvedBoneChain;

struct FRetargetPoleVectorSettings;

struct FPoleVectorMatcher
{
	EAxis::Type SourcePoleAxis;
	EAxis::Type TargetPoleAxis;
	float TargetToSourceAngularOffsetAtRefPose;
	TArray<int32> AllChildrenWithinChain;
	const FRetargetPoleVectorSettings* Settings;
	const FResolvedBoneChain* SourceBoneChain;
	const FResolvedBoneChain* TargetBoneChain;

	bool Initialize(
		const FRetargetPoleVectorSettings* InSettings,
		const FResolvedBoneChain& InSourceBoneChain,
		const FResolvedBoneChain& InTargetBoneChain,
		const FRetargetSkeleton& InSourceSkeleton,
		const FRetargetSkeleton& InTargetSkeleton);

	void MatchPoleVector(
		const TArray<FTransform> &SourceGlobalPose,
		const FRetargetSkeleton& TargetSkeleton,
		TArray<FTransform> &OutTargetGlobalPose);
	
private:

	EAxis::Type CalculateBestPoleAxisForChain(
		const TArray<int32>& BoneIndices,
		const TArray<FTransform>& GlobalPose);
	
	static FVector CalculatePoleVector(
		const EAxis::Type& PoleAxis,
		const TArray<int32>& BoneIndices,
		const TArray<FTransform>& GlobalPose);

	static EAxis::Type GetMostDifferentAxis(
		const FTransform& Transform,
		const FVector& InNormal);

	static FVector GetChainAxisNormalized(
		const TArray<int32>& BoneIndices,
		const TArray<FTransform>& GlobalPose);
};

USTRUCT(BlueprintType)
struct FRetargetPoleVectorSettings
{
	GENERATED_BODY()

	FRetargetPoleVectorSettings() = default;
	FRetargetPoleVectorSettings(FName InTargetChainName) : TargetChainName(InTargetChainName) {}
	
	/** The name of the target chain to align pole vectors on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;

	/** Enable pole vector alignment on this chain. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Align Pole Vector", meta=(ReinitializeOnEdit))
	bool bEnabled = false;

	/** Range 0 to 1. Default 1. Matches the twist angle of this chain (along the Pole direction) to the source chain.
	*  At 0, the chain's pole vector direction will be left alone
	*  At 1, the root bone of the chain will be twist-rotated in the pole direction to match the orientation of the source chain.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Align Pole Vector", meta = (EditCondition = "bEnabled", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	double AlignAlpha = 1.0f;

	/** Range +/- 180. Default 0. An angular offset, in degrees, for the pole direction of the chain. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Align Pole Vector", meta = (EditCondition = "bEnabled", UIMin = "-180.0", UIMax = "180.0", ClampMin = "-180.0", ClampMax = "180.0"))
	double StaticAngularOffset = 0.0f;

	/** Default is False. When true, the original angular offset between the source/target pole vectors will be maintained when aligning pole vectors. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Align Pole Vector", meta = (EditCondition = "bEnabled"))
	bool MaintainOffset = false;
	
	UE_API bool operator==(const FRetargetPoleVectorSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };
};

USTRUCT(BlueprintType, meta = (DisplayName = "Pole Vector Alignment Settings"))
struct FIKRetargetAlignPoleVectorOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	/** The target IK Rig asset that contains the bones chains to retarget in this op. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "IK Rig Asset", meta=(ReinitializeOnEdit))
	TObjectPtr<const UIKRigDefinition> IKRigAsset = nullptr;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chains", meta=(TitleProperty="TargetChainName"))
	TArray<FRetargetPoleVectorSettings> ChainsToAlign;

	/** Range 0 to 1. Default 1. Blends the effect of the entire op. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0f;

	UE_API void MergePoleVectorSettings(const FRetargetPoleVectorSettings& InSettingsToMerge);

	UE_API virtual const UClass* GetControllerType() const override;
	
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Pole Vector Alignment"))
struct FIKRetargetAlignPoleVectorOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	IKRIG_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;

	UE_API virtual const UScriptStruct* GetType() const override;
	
	UE_API virtual const UIKRigDefinition* GetCustomTargetIKRig() const override;

	UE_API virtual FRetargetChainMapping* GetChainMapping() override;

	UE_API virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;
	
	UE_API virtual void OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent) override;

#if WITH_EDITOR
	UE_API virtual FText GetWarningMessage() const override;
	UE_API virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	UE_API virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	UE_API virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif
	UPROPERTY()
	FIKRetargetAlignPoleVectorOpSettings Settings;

	/* This op maintains its own chain mapping table to allow per-op mapping */
	UPROPERTY()
	FRetargetChainMapping ChainMapping;

private:
	
	void ApplyIKRigs(const UIKRigDefinition* InSourceIKRig, const UIKRigDefinition* InTargetIKRig);
	
	TArray<FPoleVectorMatcher> PoleVectorMatchers;
};

/* The blueprint/python API for editing an Align Pole Vector Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetAlignPoleVectorController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetAlignPoleVectorOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetAlignPoleVectorOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetAlignPoleVectorOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetAlignPoleVectorOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
