// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Retargeter/IKRetargetOps.h"
#include "BlendToSourceOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "BlendToSourceOp"

struct FResolvedBoneChain;
struct FIKRigGoalContainer;
struct FIKRetargetPelvisMotionOp;

USTRUCT(BlueprintType)
struct FIKRetargetBlendToSourceChainSettings
{
	GENERATED_BODY()

	FIKRetargetBlendToSourceChainSettings() = default;
	FIKRetargetBlendToSourceChainSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {}
	bool operator==(const FIKRetargetBlendToSourceChainSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };

	/** The name of the TARGET chain to blend to source on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;
	
	/** Range 0 to 1. Default 0. Blend the goal on this chain to the location of the equivalent goal on the source chain. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta=(ReinitializeOnEdit, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double BlendToSource = 0.0;

	/** Range 0 to 1. Default 1. Blends the translational component of BlendToSource on/off.
	*  At 0 the goal is placed at the input location.
	*  At 1 the goal is placed at the location of the source chain's end bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double TranslationAlpha = 1.0f;

	/** Range 0 to 1. Default 0. Blends the rotational component of BlendToSource on/off.
	*  At 0 the goal is oriented to the input rotation.
	*  At 1 the goal is oriented to the source chain's end bone rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double RotationAlpha = 0.0f;

	/** Range 0 to 1. Default 1. Weight each axis separately when using Blend To Source.
	*  At 0 the goal is placed at the input location.
	*  At 1 the goal is placed at the location of the source chain's end bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FVector TranslationPerAxisAlpha = FVector::OneVector;

	/** Range 0 to 1. Default 0. At 1, the source goal locations are fully affected by any offsets applied to the pelvis from a Pelvis Motion Op.
	* NOTE: If no Pelvis Motion Op is present, or if the "Affect IK" weights in the Pelvis Motion Op are zero, then this setting has no effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double ApplyPelvisOffset = 0.0;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Blend to Source Settings"))
struct FIKRetargetBlendToSourceOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FIKRetargetBlendToSourceChainSettings> Chains;

	/** Range 0 to 1. Default 1. Blends IK goal transform from retargeted transform (0) to source bone transform (1).
	*  At 0 the goal is placed at the input location and rotation.
	*  At 1 the goal is placed at the location and rotation of the source chain's end bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0f;

	// Adjust size of goal debug drawing in viewport
	UPROPERTY(EditAnywhere, Category = Debug)
	double DebugDrawSize = 5.0;

	virtual const UClass* GetControllerType() const override;

	virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

struct FIKChainToBlend
{
	FIKChainToBlend() = delete;
	FIKChainToBlend(
		const FResolvedBoneChain& InSourceBoneChain,
		const FResolvedBoneChain& InTargetBoneChain,
		const FIKRetargetBlendToSourceChainSettings& InSettings)
		: SourceBoneChain(&InSourceBoneChain), TargetBoneChain(&InTargetBoneChain), Settings(&InSettings){}

	void Run(
		FIKRigGoalContainer& InGoalContainer,
		const TArray<FTransform>& InSourceGlobalPose,
		const FIKRetargetBlendToSourceOpSettings& InOpSettings,
		const FIKRetargetPelvisMotionOp* InPelvisMotionOp) const;

#if WITH_EDITOR
	void DebugDraw(FPrimitiveDrawInterface* InPDI, const FTransform& InComponentTransform, const double InDrawSize) const;
#endif
	

private:
	
	// cached data
	const FResolvedBoneChain* SourceBoneChain;
	const FResolvedBoneChain* TargetBoneChain;
	const FIKRetargetBlendToSourceChainSettings* Settings;

#if WITH_EDITOR
	static UE_API FCriticalSection DebugDataMutex;
	mutable FTransform DebugOutputGoal = FTransform::Identity;
	mutable FTransform DebugSourceGoal = FTransform::Identity;
#endif
};

USTRUCT(BlueprintType, meta = (DisplayName = "Blend to Source"))
struct FIKRetargetBlendToSourceOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	virtual const UScriptStruct* GetSettingsType() const override;

	virtual const UScriptStruct* GetType() const override;

	virtual const UScriptStruct* GetParentOpType() const override;

	virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;

	virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) override;
	
	UPROPERTY()
	FIKRetargetBlendToSourceOpSettings Settings;

#if WITH_EDITOR
public:
	virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;
	virtual bool HasDebugDrawing() override { return true; };
	virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif

	void RegenerateChainSettings(const FIKRetargetOpBase* InParentOp);
	
private:

	TArray<FIKChainToBlend> ChainsToBlend;
	const FIKRetargetPelvisMotionOp* PelvisMotionOp;
};

/* The blueprint/python API for editing a Blend to Source Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetBlendToSourceController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKBlendToSourceChainSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	FIKRetargetBlendToSourceOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKBlendToSourceChainSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	void SetSettings(FIKRetargetBlendToSourceOpSettings InSettings);
};

#undef UE_API
#undef LOCTEXT_NAMESPACE
