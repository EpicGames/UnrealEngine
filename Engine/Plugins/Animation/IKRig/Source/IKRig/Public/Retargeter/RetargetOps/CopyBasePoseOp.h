// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Animation/BoneReference.h"

#include "CopyBasePoseOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "CopyBasePoseOp"

USTRUCT(BlueprintType, meta = (DisplayName = "Scale Source Settings"))
struct FIKRetargetCopyBasePoseOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	/** When true, will copy all the source bone transforms with matching names to use as a base pose. This can be useful for partial retargeting.
	 * NOTE: no retargeting is applied to the bone transforms, they are assumed to be fully compatible between source/target skeletons. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RetargetPhases)
	bool bCopyBasePose = true;

	/** Filters the bones to copy when using "Copy Base Pose". If specified, will only copy all children of the specified bone (inclusive). */
	UPROPERTY(EditAnywhere, Category = RetargetPhases, meta = (ReinitializeOnEdit, NotOverrideable, EditCondition = "bCopyBasePose"))
	FBoneReference CopyFromStart;

	/** Bones added to this list (and their children) will not have their pose copied. They will be left at the reference pose.*/
	UPROPERTY(EditAnywhere, Category = RetargetPhases, meta = (ReinitializeOnEdit, NotOverrideable, EditCondition = "bCopyBasePose"))
	TArray<FBoneReference> BonesToExclude;

	/** converted on PostLoad */
	UPROPERTY(meta = (DeprecatedProperty))
	FName CopyBasePoseRoot_DEPRECATED = NAME_None;

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	/** Custom serialization to handle conversion from deprecated CopyBasePoseRoot to CopyFromStart */
	virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) override;
	bool Serialize(FArchive& Ar);
};

// Custom serialization to load deprecated settings
template<> struct TStructOpsTypeTraits<FIKRetargetCopyBasePoseOpSettings> : public TStructOpsTypeTraitsBase2<FIKRetargetCopyBasePoseOpSettings>
{
	enum
	{
		WithSerializer = true,
	};
};

USTRUCT(BlueprintType, meta = (DisplayName = "Copy Base Pose"))
struct FIKRetargetCopyBasePoseOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	// NOTE: this op does not do anything in Initialize() or Run().
	// It is a special case op that the retargeter reads from when it needs to copy the base pose
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;
	
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;

	virtual bool IsSingleton() const override { return true; };
	
	UPROPERTY()
	FIKRetargetCopyBasePoseOpSettings Settings;

private:
	// mapping from source bones to target bone indices
	// NOTE: this map will only contain bones with the same name in both skeletons that are below the CopyFromStart (if specified)
	TMap<int32, int32> SourceToTargetBoneIndexMap;

	// children of bones that are copied that need manually updated
	TArray<int32> ChildrenToUpdate;
};

/* The blueprint/python API for editing a Copy Base Pose Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetCopyBasePoseController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetCopyBasePoseOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetCopyBasePoseOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetCopyBasePoseOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetCopyBasePoseOpSettings InSettings);

	/* Set the name of the bone to use as the "Copy From Start" in the op */
	UFUNCTION(BlueprintCallable, Category = CopyFromStart)
	UE_API void SetCopyFromStart(const FName InBoneName) const;

	/* Get the name of the bone to use as the "Copy From Start" in the op */
	UFUNCTION(BlueprintCallable, Category = CopyFromStart)
	UE_API FName GetCopyFromStart() const;

	/* Add a bone to the "BonesToExclude" array */
	UFUNCTION(BlueprintCallable, Category = ExcludedBones)
	UE_API void AddBoneToExclude(const FName InBoneName) const;

	/* Get an array of all the excluded bones. */
	UFUNCTION(BlueprintCallable, Category = ExcludedBones)
	UE_API TArray<FName> GetBonesToExclude() const;

	/* Reset the array of Bones to Exclude */
	UFUNCTION(BlueprintCallable, Category = ExcludedBones)
	UE_API void ResetBonesToExclude() const;
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
