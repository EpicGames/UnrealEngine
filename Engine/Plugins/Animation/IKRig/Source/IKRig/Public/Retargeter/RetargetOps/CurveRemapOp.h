// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/RetargetOps/RetargetCurvesOp.h"
#include "StructUtils/InstancedStruct.h"
#include "Animation/AnimCurveTypes.h"

#include "CurveRemapOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "CurveRemapOp"

USTRUCT(BlueprintType)
struct FCurveRemapPair
{
	GENERATED_BODY()
	
	// The curve name on the SOURCE skeletal mesh to copy animation data from.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	FName SourceCurve;

	// The curve name on the TARGET skeletal mesh to receive animation data.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	FName TargetCurve;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Remap Curve Settings"))
struct FIKRetargetCurveRemapOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	// Whether to copy all curves over to the target animation instance
	// NOTE: This setting also applies when exporting retargeted animations.
	// True: all source curves are copied to the target animation instance/asset
	// False: only remapped curves are copied on the target animation instance/asset
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Copy Curves")
	bool bCopyAllSourceCurves = true;

	// Toggle curve remapping on/off
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Remap Curves")
	bool bRemapCurves = true;
	
	// Add pairs of Source/Target curve names to remap. While retargeting, the animation from the source curves
	// will be redirected to the curves on the target skeletal meshes. Can be used to drive, blendshapes or other downstream systems.
	// NOTE: By default the IK Retargeter will automatically copy all equivalently named curves from the source to the
	// target. Remapping is only necessary when the target curve name(s) are different.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Remap Curves")
	TArray<FCurveRemapPair> CurvesToRemap;

	UE_API virtual const UClass* GetControllerType() const override;
	
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Remap Curves"))
struct FIKRetargetCurveRemapOp : public FIKRetargetCurvesOpBase
{
	GENERATED_BODY()

public:
	// NOTE: this op does not do anything in Initialize() or Run().
	// It implements GetCurvesToRetarget() which the retargeting anim node calls and manages
	// In the future we may remove this coupling and make the op itself do the work via callbacks
	
	virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override
	{
		bIsInitialized = true;
		return true;
	};
	
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override {};

	virtual FIKRetargetOpSettingsBase* GetSettings() override
	{
		return &Settings;
	};

	virtual void SetSettings(const FIKRetargetOpSettingsBase* InSettings) override
	{
		Settings = *reinterpret_cast<const FIKRetargetCurveRemapOpSettings*>(InSettings);
	};
	
	virtual const UScriptStruct* GetSettingsType() const override
	{
		return FIKRetargetCurveRemapOpSettings::StaticStruct();
	}
	
	virtual const UScriptStruct* GetType() const override
	{
		return FIKRetargetCurveRemapOp::StaticStruct();
	}

	bool CanHaveChildOps() const override
	{
		return true;
	}

	virtual bool IsAbstract() const override { return false; };

	virtual bool IsSingleton() const override { return true; };

	virtual bool HasCurveProcessing() const override
	{
		return true;
	}

	virtual void ProcessAnimSequenceCurves(FIKRetargetCurvesOpBase::FCurveData InCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues,
		FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues)const override;

	UE_API virtual void AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent) override;

	UE_API virtual void AnimGraphEvaluateAnyThread(FPoseContext& Output) override;

	UPROPERTY()
	FIKRetargetCurveRemapOpSettings Settings;
};

/* The blueprint/python API for editing a Curve Remap Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetCurveRemapController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetCurveRemapOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetCurveRemapOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetCurveRemapOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetCurveRemapOpSettings InSettings);
};

//
// BEGIN DEPRECATED UOBJECT-based OP
//

// NOTE: This type has been replaced with FIKRetargetCurveRemapOp.
UCLASS(MinimalAPI)
class UCurveRemapOp : public URetargetOpBase
{
	GENERATED_BODY()
	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) override
	{
		OutInstancedStruct.InitializeAs(FIKRetargetCurveRemapOp::StaticStruct());
		FIKRetargetCurveRemapOp& NewOp = OutInstancedStruct.GetMutable<FIKRetargetCurveRemapOp>();
		NewOp.SetEnabled(bIsEnabled);
		NewOp.Settings.CurvesToRemap = CurvesToRemap;
		NewOp.Settings.bCopyAllSourceCurves = bCopyAllSourceCurves;
	};
	
	UPROPERTY()
	TArray<FCurveRemapPair> CurvesToRemap;
	UPROPERTY()
	bool bCopyAllSourceCurves = true;
};

//
// END DEPRECATED UOBJECT-based OP
//

#undef LOCTEXT_NAMESPACE

#undef UE_API
