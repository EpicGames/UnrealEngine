// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetChainMapping.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetProcessor.h"

#if WITH_EDITOR
#include "HitProxies.h"
#endif

#include "FKChainsOp.generated.h"

#define LOCTEXT_NAMESPACE "FKChainOp"

struct FChainEncoderFK
{
	void Initialize(const FResolvedBoneChain* InBoneChain);
	
	void EncodePose(
		const FRetargetSkeleton& SourceSkeleton,
		const TArray<int32>& SourceBoneIndices,
		const TArray<FTransform> &InSourceGlobalPose);

	void TransformCurrentChainTransforms(const FTransform& NewParentTransform);
	
	TArray<FTransform> CurrentGlobalTransforms;
	FTransform ChainParentCurrentGlobalTransform;
	const FResolvedBoneChain* BoneChain;
};

struct FChainDecoderCachedData
{
	TArray<int32> SourceBoneIndices;
	TArray<FQuat> FromToGlobalRotationDeltas;
	TArray<double> SourceRefPoseLengths;
	TArray<double> TargetRefPoseLengths;

	void Initialize(
		const FResolvedBoneChain* InSourceBoneChain,
		const FResolvedBoneChain* InTargetBoneChain,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton);
};

struct FChainDecoderFK
{
	void Initialize(
		const FResolvedBoneChain* InSourceBoneChain,
		const FResolvedBoneChain* InTargetBoneChain,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton);
	
	void InitializeIntermediateParentIndices(
		const int32 InRetargetRootBoneIndex,
		const int32 InChainRootBoneIndex,
		const FTargetSkeleton& InTargetSkeleton);

	void UpdateIntermediateLocalTransforms(
		const FTargetSkeleton& TargetSkeleton,
		const TArray<FTransform> &InOutGlobalPose);
	
	void DecodePose(
		const FIKRetargetPelvisMotionOp* PelvisMotionOp,
		const FRetargetFKChainSettings& Settings,
		const TArray<int32>& TargetBoneIndices,
		FChainEncoderFK& SourceChain,
		const TArray<FTransform>& InSourceGlobalPose,
		const FRetargetSkeleton& SourceSkeleton,
		const FTargetSkeleton& TargetSkeleton,
		TArray<FTransform>& InOutGlobalPose);
	
private:

	FQuat RetargetRotation(
		const int32 InChainIndex,
		const FRetargetFKChainSettings& Settings,
		const TArray<int32>& TargetBoneIndices,
		const FTargetSkeleton& TargetSkeleton,
		const TArray<FTransform>& InOutGlobalPose,
		FChainEncoderFK& SourceChain,
		FTransform& OutSourceCurrentTransform,
		FTransform& OutSourceInitialTransform) const;

	FVector RetargetPosition(
		const int32 InChainIndex,
		const FQuat& NewRotation,
		const FIKRetargetPelvisMotionOp* PelvisMotionOp,
		const FRetargetFKChainSettings& Settings,
		const TArray<int32>& TargetBoneIndices,
		FChainEncoderFK& SourceChain,
		const TArray<FTransform>& InSourceGlobalPose,
		const FRetargetSkeleton& SourceSkeleton,
		const FTargetSkeleton& TargetSkeleton,
		TArray<FTransform>& InOutGlobalPose,
		const FTransform& InCurrentSourceGlobal);

	FVector RetargetScale(
		const int32 InChainIndex,
		const FTransform& OutSourceCurrentTransform,
		const FTransform& OutSourceInitialTransform) const;

	void MatchChain(
		const bool bScaleSourceChain,
		const FChainEncoderFK& SourceChain,
		const TArray<int32>& TargetBoneIndices,
		const FVector& TargetChainOrigin,
		TArray<FTransform>& InOutGlobalPose);

	void BlendByAlpha(
		const FRetargetFKChainSettings& Settings,
		const TArray<int32>& TargetBoneIndices,
		const FTargetSkeleton& TargetSkeleton,
		TArray<FTransform>& InOutGlobalPose);

	static FVector GetPointOnSplineFromIndexAndAlpha(const TArray<FVector>& InSplinePoints, const int32 InPointIndex, const double InSegmentAlpha);
	
	static void GetPointOnSplineDistanceFromPoint(
		const TArray<FVector>& InSplinePoints,
		const int32 InOriginPointIndex,
		const double InOriginPointAlpha,
		const double InTargetDistanceFromOrigin,
		int32& OutPointIndex,
		double& OutPointAlpha);
	
	void UpdateIntermediateParents(
		const FTargetSkeleton& TargetSkeleton,
		TArray<FTransform> &InOutGlobalPose);

	FChainDecoderCachedData CachedData;
	TArray<int32> IntermediateParentIndices;
	TArray<FTransform> IntermediateLocalTransforms;
	const FResolvedBoneChain* BoneChain;
};

struct FChainPairFK
{
	bool Initialize(
		const FResolvedBoneChain& InSourceBoneChain,
		const FResolvedBoneChain& InTargetBoneChain,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FRetargetFKChainSettings& InSettings,
		const FIKRigLogger& InLog);
	
	FChainEncoderFK FKEncoder;
	FChainDecoderFK FKDecoder;
	const FResolvedBoneChain* SourceBoneChain;
	const FResolvedBoneChain* TargetBoneChain;
	const FRetargetFKChainSettings* Settings;
};

UENUM(BlueprintType)
enum class EFKChainTranslationMode : uint8
{
	None							UMETA(DisplayName = "None"),
	GloballyScaled					UMETA(DisplayName = "Globally Scaled"),
	Absolute						UMETA(DisplayName = "Absolute"),
	StretchBoneLengthUniformly		UMETA(DisplayName = "Stretch Bone Length Uniformly"),
	StretchBoneLengthNonUniformly	UMETA(DisplayName = "Stretch Bone Length Non-Uniformly"),
	OrientAndScale					UMETA(DisplayName = "Orient and Scale"),
};

UENUM(BlueprintType)
enum class EFKChainRotationMode : uint8
{
	None				= 5 UMETA(DisplayName = "None"),
	Interpolated		= 0 UMETA(DisplayName = "Interpolated"),
	OneToOne			= 1 UMETA(DisplayName = "One to One"),
	OneToOneReversed	= 2 UMETA(DisplayName = "One to One Reversed"),
	MatchChain			= 3 UMETA(DisplayName = "Match Chain"),
	MatchScaledChain	= 4 UMETA(DisplayName = "Match Scaled Chain"),
	CopyLocal			= 6 UMETA(DisplayName = "Copy Local"),
};

USTRUCT(BlueprintType)
struct FRetargetFKChainSettings
{
	GENERATED_BODY()

	FRetargetFKChainSettings() = default;
	FRetargetFKChainSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {};

	/** The name of the TARGET chain to transfer animation onto. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;
	
	/** Whether to copy the shape of the chain from the source skeleton using the Rotation and Translation modes. Default is true.
	* NOTE: All FK operations run before the IK pass to copy the shape of the FK chain from the source skeleton. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "FK Chain Retarget")
	bool EnableFK = true;
	
	/** Determines how rotation is copied from the source chain to the target chain. Default is Interpolated.
	* None: The rotation of each target bone in the chain is left at the reference pose. 
	* Interpolated: Source and target chains are normalized by their length, then each target bone rotation is generated by finding the rotation at the same normalized distance on the source chain and interpolating between the neighboring bones.
	* One to One: Each target bone rotation is copied from the equivalent bone in the source chain, based on the order in the chain, starting at the root of the chain. If the target chain has more bones than the source, the extra bones will remain at their reference pose.
	* One to One Reversed: Same as One-to-One, but starting from the tip of the chain.
	* Match Chain: Uses a Spline-IK type method to orient the target joint chain to exactly match the shape of the source chain.
	* Match Scaled Chain: Same as Match Chain, but scales the source chain to the same length as the target chain before running the match procedure.
	* Copy Local: The local space rotation of the closest source bone is copied to the target. No retargeting is applied. Retarget pose is ignored.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "FK Chain Retarget")
	EFKChainRotationMode RotationMode = EFKChainRotationMode::Interpolated;

	/** Range +/- infinity. Default 1. Scales the amount of rotation that is applied.
	*  If Rotation Mode is None this parameter has no effect.
	*  Otherwise, this parameter blends the rotation of each bone in the chain from the base retarget pose (0) to the retargeted pose (1).*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "FK Chain Retarget", meta = (UIMin = "0.0", UIMax = "1.0"))
	double RotationAlpha = 1.0f;

	/** Determines how translation is copied from the source chain to the target chain. Default is None.
	* None: Translation of target bones are left unmodified from the retarget pose.
	* Globally Scaled: Translation of target bone is set to the source bone offset multiplied by the global scale of the skeleton (determined by the relative height difference between pelvis bones).
	* Absolute: Translation of target bone is set to the absolute position of the source bone.
	* Stretch Bone Length Uniformly: squashes or stretches the local translation of each bone so that the length of the target chain matches the relative length of the source chain. Accounts for differences in initial chain lengths.
	* Stretch Bone Length Non-Uniformly: same as "Stretch Bone Length Uniformly" but treats each bone independently to allow for chains that stretch non-uniformly along their length (ie the base may stretch more than the tip or vice versa).   
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "FK Chain Retarget")
	EFKChainTranslationMode TranslationMode = EFKChainTranslationMode::None;

	/** Range +/- infinity. Default 1. Scales the amount of translation that is applied. Exact behavior depends on the Translation Mode.
	*  In None Mode, this parameter has no effect.
	*  In Globally Scaled and Absolute modes, the translation offset is scaled by this parameter.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "FK Chain Retarget", meta = (UIMin = "0.0", UIMax = "1.0"))
	double TranslationAlpha = 1.0f;

	IKRIG_API bool operator==(const FRetargetFKChainSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };
};

USTRUCT(BlueprintType, meta = (DisplayName = "Retarget Chains Settings"))
struct FIKRetargetFKChainsOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	/** The target IK Rig asset that contains the bones chains to retarget in this op. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "IK Rig Asset", meta=(ReinitializeOnEdit))
	TObjectPtr<const UIKRigDefinition> IKRigAsset = nullptr;

	/** The setting for each chain to retarget */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FRetargetFKChainSettings> ChainsToRetarget;

	/** Debug draw lines on each chain in the viewport */
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDrawChainLines = true;

	/** Debug draw spheres on single-bone chains in the viewport */
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDrawSingleBoneChains = true;

	/** Debug draw chain line thickness. */
	UPROPERTY(EditAnywhere, Category = Debug)
	double ChainDrawThickness = 1.0f;

	/** Debug draw size used for single-bone chains. */
	UPROPERTY(EditAnywhere, Category = Debug)
	double ChainDrawSize = 1.0f;

	/* This op maintains its own chain mapping table to allow per-op mapping.
	 * NOTE: this is only editable through a custom UI in the details panel or the C++ API. */
	UPROPERTY()
	FRetargetChainMapping ChainMapping;

	IKRIG_API virtual const UClass* GetControllerType() const override;

	IKRIG_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	friend FIKRetargetFKChainsOp;
};

#if WITH_EDITOR
struct FFKChainDebugData
{
	FName TargetChainName;
	FTransform StartTransform;
	FTransform EndTransform;
	bool bIsSingleBoneChain = false;
};
#endif

USTRUCT(BlueprintType, meta = (DisplayName = "FK Chains"))
struct FIKRetargetFKChainsOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	IKRIG_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	IKRIG_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	IKRIG_API virtual void PostInitialize(const FIKRetargetProcessor& Processor,
		const FRetargetSkeleton& SourceSkeleton,
		const FTargetSkeleton& TargetSkeleton,
		FIKRigLogger& Log) override;

	IKRIG_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	IKRIG_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;
	
	IKRIG_API virtual void CollectRetargetedBones(TSet<int32>& OutRetargetedBones) const override;

	IKRIG_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	IKRIG_API virtual const UScriptStruct* GetSettingsType() const override;

	IKRIG_API virtual const UScriptStruct* GetType() const override;

	IKRIG_API virtual const UIKRigDefinition* GetCustomTargetIKRig() const override;

	IKRIG_API virtual FRetargetChainMapping* GetChainMapping() override;

	IKRIG_API virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;

	IKRIG_API virtual void OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent) override;

	IKRIG_API virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) override;

	UPROPERTY()
	FIKRetargetFKChainsOpSettings Settings;

#if WITH_EDITOR
public:
	
	IKRIG_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual bool HasDebugDrawing() override { return true; };
	
	IKRIG_API void SaveDebugData(const TArray<FTransform>& OutTargetGlobalPose);
	TArray<FFKChainDebugData> AllChainsDebugData;
	static IKRIG_API FCriticalSection DebugDataMutex;

	IKRIG_API virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	IKRIG_API virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	IKRIG_API virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif

private:

	void ApplyIKRigs(const UIKRigDefinition* InSourceIKRig, const UIKRigDefinition* InTargetIKRig);
	
	/** The Source/Target pairs of Bone Chains retargeted using the FK algorithm */
	TArray<FChainPairFK> ChainPairsFK;

	/** Children of the FK chains that need updated after the chains are retargeted */
	TArray<int32> NonRetargetedChildrenToUpdate;
	TArray<FTransform> ChildrenToUpdateLocalTransforms;
	
	/* Deprecated storage for chain mapping (now on settings for API access)*/
	UPROPERTY(meta = (DeprecatedProperty))
	FRetargetChainMapping ChainMapping_DEPRECATED;
};

/* The blueprint/python API for editing a FK Chains Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetFKChainsController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetFKChainsOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	IKRIG_API FIKRetargetFKChainsOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetFKChainsOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	IKRIG_API void SetSettings(FIKRetargetFKChainsOpSettings InSettings);
};

#if WITH_EDITOR
// select chains/goals to edit chain settings
struct HIKRetargetEditorChainProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( IKRIG_API );

	FName TargetChainName;
	
	HIKRetargetEditorChainProxy(const FName& InTargetChainName)
		: HHitProxy(HPP_World)
		, TargetChainName(InTargetChainName) {}

	virtual EMouseCursor::Type GetMouseCursor()
	{
		return EMouseCursor::Crosshairs;
	}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};
#endif

#undef LOCTEXT_NAMESPACE
