// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RetargetOpUtils.h"
#include "Animation/BoneReference.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetProcessor.h"

#include "FilterBoneOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "FilterBoneOp"

struct FIKRetargetOpSettingsBase;

USTRUCT(BlueprintType)
struct FFilterBoneData
{
	GENERATED_BODY()

	// The target bone to filter.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ReinitializeOnEdit))
	FBoneReference TargetBone;

	bool bIsReady = false;
	FOneEuroQuatFilter Filter;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Filter Bone Settings"))
struct FIKRetargetFilterBoneOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	/** A list of bone-pairs to copy transforms between */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Setup, meta=(ReinitializeOnEdit))
	TArray<FFilterBoneData> BonesToFilter;

	/** Range 0 to 1. Default 1. Blends the effect of the entire op. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0f;

	/** The settings to control the One-Euro filter behavior */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Op Settings")
	FOneEuroFilterSettings FilterSettings;

	/* If true, filter is reset when playback loops. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Op Settings")
	bool bResetPlayback = true;

	UE_API virtual const UClass* GetControllerType() const override;
	
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Filter Bones"))
struct FIKRetargetFilterBoneOp : public FIKRetargetOpBase
{
	GENERATED_BODY()

	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& Processor,
		const FRetargetSkeleton& SourceSkeleton,
		const FTargetSkeleton& TargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& Processor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;

	UE_API virtual void OnPlaybackReset();
	
	UPROPERTY()
	FIKRetargetFilterBoneOpSettings Settings;

private:
	void ResetFilters(TArray<FTransform>& OutTargetGlobalPose);

	bool bResetPlayback = true;
};

/* The blueprint/python API for editing a Filter Bone Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetFilterBoneController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetPinBoneOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetFilterBoneOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetPinBoneOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetFilterBoneOpSettings InSettings);

	/* Clear all the bones */
	UFUNCTION(BlueprintCallable, Category = BonePairs)
	UE_API void ClearBonesToFilter();

	/* Add a target bone to filter. */
	UFUNCTION(BlueprintCallable, Category = BonePairs)
	UE_API void AddBoneToFilter(const FName InTargetBone);

	/* Get all the bones currently stored in the op.
	 * @return an array of target bone names*/
	UFUNCTION(BlueprintCallable, Category = BonePairs)
	UE_API TArray<FName> GetAllBonesToFilter();
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
