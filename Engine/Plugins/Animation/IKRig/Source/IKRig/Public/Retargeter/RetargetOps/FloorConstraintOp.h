// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RetargetOpUtils.h"
#include "Retargeter/IKRetargetChainMapping.h"
#include "Retargeter/IKRetargetOps.h"
#include "Animation/BoneReference.h"
#include "Containers/StaticArray.h"
#include "FloorConstraintOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "FloorConstraintOp"

struct FResolvedBoneChain;
struct FIKRigGoalContainer;
struct FIKRigGoal;

struct FFloorConstraint;

USTRUCT(BlueprintType)
struct FFloorConstraintToeDefinition
{
	GENERATED_BODY()

	/** The bone located at the base of the toe (the first knuckle). */
	UPROPERTY(EditAnywhere, Category=Setup, meta=(ReinitializeOnEdit, NotOverrideable))
	FBoneReference ToeBone;

	/** The length of the toe from the ToeBone to the tip of the toe mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Toe Dimensions", meta = (ReinitializeOnEdit, ClampMin = "0.0", UIMin = "0.0", UIMax = "50.0"))
	double Length = 4.0f;

	/** Rotate the end of the toe side-to-side. Adjust this to align the length of the toe with the model. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Toe Dimensions", meta = (ReinitializeOnEdit, ClampMin = "-179.0", ClampMax = "179.0", UIMin = "-179.0", UIMax = "179.0"))
	double YawOffset = 0.0f;

	/** Push the tip of the toes up/down. Adjust this to place the tip of the toe on the bottom of the toe mesh. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Toe Dimensions", meta = (ReinitializeOnEdit, UIMin = "-10.0", UIMax = "10.0"))
	double VerticalOffset = 0.0f;
};

USTRUCT(BlueprintType)
struct FFloorConstraintToesDefinition
{
	GENERATED_BODY()

	/** Default 1.0. Range 0 to 1. Blend the effect of the toe solver on/off. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Toe Settings", meta = (ReinitializeOnEdit, ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0;

	/** Speed coefficient; increases cutoff when signal changes quickly (reactivity). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Toe Filter", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "5.0"))
	float Responsiveness = 0.02f;
	
	/** Minimum cutoff frequency for the One Euro Filter (higher = less smoothing). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Toe Filter", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "10.0"))
	float CutoffFrequency = 8.0f;

	/** Cutoff frequency for the derivative smoothing step. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Toe Filter", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "10.0"))
	float VelocityCutoffFrequency = 1.0f;
	
	/** An array of toes to bend when contacting the floor */
	UPROPERTY(EditAnywhere, Category="Toe Definitions", meta=(ReinitializeOnEdit))
	TArray<FFloorConstraintToeDefinition> AllToes;
};

USTRUCT(BlueprintType)
struct FFloorConstraintFootDefinition
{
	GENERATED_BODY()
	
	/** Default 10. The offset in cm towards the interior (middle) of the body. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Foot Dimensions", meta = (ReinitializeOnEdit, ClampMin = "0.0", UIMin = "0.0", UIMax = "50.0"))
	double MedialOffset = 10.0f;

	/** Default 10. The offset in cm towards the exterior (outside) of the body. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Foot Dimensions", meta = (ReinitializeOnEdit, ClampMin = "0.0", UIMin = "0.0", UIMax = "50.0"))
	double LateralOffset = 10.0f;

	/** Default 10. The offset in the direction of the heel (rear) of the foot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Foot Dimensions", meta = (ReinitializeOnEdit, ClampMin = "0.0", UIMin = "0.0", UIMax = "50.0"))
	double HeelOffset = 10.0f;

	/** Default 10. The offset towards the toes (front) of the foot.
	 * NOTE: if you are using toe constraints, this offset should extend to the ball of the foot.
	 * Otherwise the toes will never be below the floor and thus never bend. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Foot Dimensions", meta = (ReinitializeOnEdit, ClampMin = "0.0", UIMin = "0.0", UIMax = "50.0"))
	double ToeOffset = 10.0f;

	/** Default 0. The vertical offset used to define the "bottom" of the foot mesh.
	 * NOTE: adjust this up/down if the floor constraint puts the foot mesh below/above the ground level. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Foot Dimensions", meta = (ReinitializeOnEdit, UIMin = "-10.0", UIMax = "10.0"))
	double VerticalOffset = 0.0f;
};

USTRUCT(BlueprintType)
struct FFloorConstraintChainSettings : public FIKRetargetOpSkeletonProvider
{
	GENERATED_BODY()

	FFloorConstraintChainSettings() = default;
	FFloorConstraintChainSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {}

	/** The name of the TARGET chain to transfer animation onto. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;
	
	/** Whether to apply the floor constraint to the location of the IK goal on this chain. Default is false.
	 * When ON, the floor constraint will adjust the vertical position of the IK Goal according to the following rules.
	 * 1. When the source goal bone is LOWER than FloorHeightFalloffStart, the height of the goal is smoothly blended to the height of the source bone.
	 * 2. When the source goal bone is HIGHER than FloorHeightFalloffStart, the height of the goal is left at its normal retargeted location.
	 * NOTE: the floor is assumed to be the XY plane where Z = 0.
	 * NOTE: This only has an effect if the chain has an IK Goal assigned to it in the Target IK Rig asset.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Floor Constraint Chain Settings", meta=(ReinitializeOnEdit))
	bool EnableFloorConstraint = false;

	/** Range 0 to 1. Default is 0. Blend the effect of the constraint on this goal on/off. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Floor Constraint Chain Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0f;

	/** Range 0 to 1. Default is 0. Maintain the height different between the source and target from the retarget pose. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Floor Constraint Chain Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double MaintainHeightOffset = 0.0f;

	/** If true, will use the foot definition for more accurate foot/ground retargeting. If false, treats the foot as a point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Foot Settings", meta=(ReinitializeOnEdit))
	bool bUseFoot = false;
	
	/** If true, will use the toes definition for more accurate toe/ground collision. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Foot Settings", meta=(ReinitializeOnEdit))
	bool bUseToes = false;

	/** Define the proportions of the target foot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Foot Settings", meta=(ReinitializeOnEdit, EditCondition="bUseFoot"))
	FFloorConstraintFootDefinition Foot;

	/** Define the proportions of the target foot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Foot Settings", meta=(ReinitializeOnEdit, EditCondition="bUseToes"))
	FFloorConstraintToesDefinition Toes;

	UE_API bool operator==(const FFloorConstraintChainSettings& Other) const;

	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };
};

USTRUCT(BlueprintType, meta = (DisplayName = "Floor Goal Op Settings"))
struct FIKRetargetFloorConstraintOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	/** The per-chain settings (exposed indirectly to the UI through a detail customization) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FFloorConstraintChainSettings> ChainsToAffect;

	/** Range 0 to 1. Default is 1. Blend the effect of the entire op on/off. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0f;

	/** Range 0 to inf. Default is 8. The height in cm from the floor below which the goal is snapped directly to the source bone height.
     * NOTE: if the source bone height is greater than this value, but lower than FloorHeightFalloffEnd, then the height will smoothly blend from the source
     * bone height, to the height of the goal in its normal retargeted position. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.1", UIMin = "0.1", UIMax = "100.0"))
    double HeightFalloffOffset = 8.0f;
    
    /** Range 0 to inf. Default is 20. The height in cm from the floor below which the goal is gradually blended towards the source bone height.
     * NOTE: if the source bone is higher than this value, the height of the goal is left at its normal retargeted height. */
    UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.1", UIMin = "0.1", UIMax = "100.0"))
    double HeightFalloffDistance = 20.0f;

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

#if WITH_EDITOR
	UE_API virtual void AssignSkeletonAssets(const USkeleton* InSourceSkeleton, const USkeleton* InTargetSkeleton) override;
#endif
};

struct FToeConstraint
{
	// constant
	int32 ToeBoneIndex = INDEX_NONE;
	int32 AnkleBoneIndex = INDEX_NONE;
	bool bToeReady = false;
	FTransform InitialToeGlobal;
	FVector EndRelativeToToe;
	FVector PitchAxisRelativeToToe;
	const FFloorConstraintToeDefinition* Settings;
	const FFloorConstraintToesDefinition* AllToeSettings;

	// updated every frame
	FTransform CurrentToeGlobal;
	FVector Start;
	FVector End;
	FVector PitchAxis;
	FOneEuroScalarFilter AngleFilter;

	void Initialize(
		const FTargetSkeleton& InTargetSkeleton,
		const FFloorConstraintToesDefinition& InAllToeSettings,
		const FFloorConstraintToeDefinition& InToeSettings,
		const int32 InAnkleBoneIndex,
		FIKRigLogger& InLog);

	void RunAfterParent(
		const FTargetSkeleton& InTargetSkeleton,
		TArray<FTransform>& OutGlobalPose,
		const FFloorConstraintChainSettings* InChainSettings,
		FIKRetargetFloorConstraintOpSettings& InOpSettings,
		const double InDeltaTime);

	void UpdateFramePoints(
		const TArray<FTransform>& InGlobalPose,
		const FIKRigGoal* InGoal,
		bool bUpdateLocalOffset);

	double CalcToeCorrectionAngle();
};

struct FFootConstraint
{
	// the foot frame is made of a 4-point quad representing the base of the foot from the heel to the start of the toes
	static constexpr int32 FrontMedial   = 0;
	static constexpr int32 RearMedial    = 1;
	static constexpr int32 FrontLateral  = 2;
	static constexpr int32 RearLateral   = 3;
	static constexpr int32 Count         = 4;
	
	// constants
	bool bReadyToRun = false;
	int32 AnkleBoneIndex = INDEX_NONE;
	TStaticArray<FVector, Count> PointsOrig;

	// updated every frame
	TStaticArray<FVector, Count> Points;
	FTransform CurrentAnkleGlobal;

	void Initialize(
		const FTargetSkeleton& InTargetSkeleton,
		const FFloorConstraintFootDefinition& InFootSettings,
		const int32 InAnkleBoneIndex,
		FIKRigLogger& InLog);

	void Run(
		FIKRigGoal& InOutGoal,
		const TArray<FTransform>& OutTargetGlobalPose,
		const FFloorConstraintChainSettings& InChainSettings,
		const FIKRetargetFloorConstraintOpSettings& InOpSettings);
	
	void UpdateBasePoints(const TArray<FTransform>& InGlobalPose, const FIKRigGoal* InGoal);
	
	int32 GetIndexOfLowestPoint();
};

struct FFloorConstraint
{
	FName IKGoalName = NAME_None;
	int32 SourceEndBoneIndex = INDEX_NONE;
	int32 TargetEndBoneIndex = INDEX_NONE;
	double HeightOffsetInRefPose = 0.0;
	const FFloorConstraintChainSettings* Settings = nullptr;

	FFootConstraint FootConstraint;
	TArray<FToeConstraint> ToeConstraints;

	void Initialize(
		const FResolvedBoneChain& InSourceBoneChain,
		const FResolvedBoneChain& InTargetBoneChain,
		const FFloorConstraintChainSettings& InSettings);

	void Run(
		const FTargetSkeleton& InTargetSkeleton,
		FIKRigGoalContainer& InGoalContainer,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose,
		const FIKRetargetFloorConstraintOpSettings& InOpSettings);
};

USTRUCT(BlueprintType, meta = (DisplayName = "Floor Constraint"))
struct FIKRetargetFloorConstraintOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void RunAfterParent(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;

	UE_API virtual const UScriptStruct* GetType() const override;

	UE_API virtual const UScriptStruct* GetParentOpType() const override;

	UE_API virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;

	UE_API virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) override;

	UPROPERTY()
	FIKRetargetFloorConstraintOpSettings Settings;

#if WITH_EDITOR
	UE_API virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	UE_API virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	UE_API virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
	
	virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InTargetTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;
	virtual bool HasDebugDrawing() override { return true; }
#endif

private:
	
	TArray<FFloorConstraint> FloorConstraints;
};

/* The blueprint/python API for editing a Floor Constraint Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetFloorConstraintController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetFloorConstraintOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetFloorConstraintOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetFloorConstraintOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetFloorConstraintOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
