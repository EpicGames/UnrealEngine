// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Retargeter/IKRetargetOps.h"
#include "IKChainsOp.generated.h"

DECLARE_CYCLE_STAT(TEXT("IK Retarget Goals"), STAT_IKRetargetGoals, STATGROUP_Anim);

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "IKChainOp"

struct FIKRetargetRunIKRigOp;
struct FResolvedBoneChain;
struct FIKRetargetPelvisMotionOp;


struct FIKChainRetargeter
{
	bool Initialize(
		const FResolvedBoneChain& InSourceBoneChain,
		const FResolvedBoneChain& InTargetBoneChain,
		const FRetargetIKChainSettings& InSettings,
		FIKRigLogger& InLog);

	const FTransform& GenerateGoalTransform(
		const FIKRetargetPelvisMotionOp* InPelvisMotionOp,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& InTargetGlobalPose,
		const FRetargetSkeleton& InSourceSkeleton,
		const FRetargetSkeleton& InTargetSkeleton);

	const FResolvedBoneChain& GetSourceChain() const { return *SourceBoneChain; };
	
	const FResolvedBoneChain& GetTargetChain() const { return *TargetBoneChain; };
	
	const FRetargetIKChainSettings* GetSettings() const { return Settings; };

	const FTransform& GetGoalTransform() const {return GoalOutput; }

private:

	FQuat CalculateGoalRotation(const TArray<FTransform>& InSourceGlobalPose) const;

	FVector CalculateGoalPosition(
		const FIKRetargetPelvisMotionOp* InPelvisMotionOp,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& InTargetGlobalPose,
		const FRetargetSkeleton& InSourceSkeleton,
		const FRetargetSkeleton& InTargetSkeleton) const;

	// cached data
	FTransform GoalOutput;
	const FResolvedBoneChain* SourceBoneChain;
	const FResolvedBoneChain* TargetBoneChain;
	const FRetargetIKChainSettings* Settings;
	double InvInitialLengthSource = 1.0f;
	double InitialTargetChainLength = 1.0f;
	
	FTransform InitialTargetEndRelativeToSourceEnd;
};

USTRUCT(BlueprintType)
struct FRetargetIKChainSettings
{
	GENERATED_BODY()

	FRetargetIKChainSettings() = default;
	FRetargetIKChainSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {}

	/** The name of the TARGET chain to transfer animation onto. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;
	
	/** Whether to modify the location of the IK goal on this chain. Default is true.
	 * NOTE: This only has an effect if the chain has an IK Goal assigned to it in the Target IK Rig asset.
	 * NOTE: If off, and this chain has an IK Goal, the IK will still be evaluated, but the Goal is set to the input bone location (from the FK pass).*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit))
	bool EnableIK = true;

	/** Range 0 to 1. Default 0. Blends IK goal transform from retargeted transform (0) to source bone transform (1).
	*  At 0 the goal is placed at the retargeted location and rotation.
	*  At 1 the goal is placed at the location and rotation of the source chain's end bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend to Source", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double BlendToSource = 0.0f;

	/** Range 0 to 1. Default 1. Blends the translational component of BlendToSource on/off.
	*  At 0 the goal is placed at the retargeted location.
	*  At 1 the goal is placed at the location of the source chain's end bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend to Source", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double BlendToSourceTranslation = 1.0f;

	/** Range 0 to 1. Default 0. Blends the rotational component of BlendToSource on/off.
	*  At 0 the goal is oriented to the retargeted rotation.
	*  At 1 the goal is oriented to the source chain's end bone rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend to Source", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double BlendToSourceRotation = 0.0f;
	
	/** Range 0 to 1. Default 1. Weight each axis separately when using Blend To Source.
	*  At 0 the goal is placed at the retargeted location.
	*  At 1 the goal is placed at the location of the source chain's end bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend to Source", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FVector BlendToSourceWeights = FVector::OneVector;
	
	/** Default false. When true, the source goal locations are affected by the target pelvis motion.
	* NOTE: this setting applies all offsets generated in the Pelvis Motion Op. 
	If no Pelvis Motion Op is present, or if the "Affect IK" weights in the Pelvis Motion Op are zero, then this setting has no effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Blend to Source")
	bool ApplyPelvisOffsetToSourceGoals = false;

	/** Default 0, 0, 0. Apply a static global-space offset to IK goal position. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Offset")
	FVector StaticOffset = FVector::ZeroVector;

	/** Default 0, 0, 0. Apply a static local-space offset to IK goal position. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Offset")
	FVector StaticLocalOffset = FVector::ZeroVector;

	/** Default 0, 0, 0. Apply a static local-space offset to IK goal rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Static Offset")
	FRotator StaticRotationOffset = FRotator::ZeroRotator;

	/** Range 0 to infinity. Default 1. Scales the vertical component of the IK goal's position. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scale Translation", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "5.0"))
	double ScaleVertical = 1.0f;
	
	/** Range 0 to 5. Default 1. Brings IK goal closer (0) or further (1+) from origin of chain.
	*  At 0 the effector is placed at the origin of the chain (ie Shoulder, Hip etc).
	*  At 1 the effector is left at the end of the chain (ie Wrist, Foot etc)
	*  Values in-between 0-1 will slide the effector along the vector from the start to the end of the chain.
	*  Values greater than 1 will stretch the chain beyond the retargeted length. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scale Length", meta = (ClampMin = "0.0", ClampMax = "5.0", UIMin = "0.1", UIMax = "2.0"))
	double Extension = 1.0f;

	UE_API bool operator==(const FRetargetIKChainSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };

	/** Custom serialization to handle loading old assets */
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FRetargetIKChainSettings> : public TStructOpsTypeTraitsBase2<FRetargetIKChainSettings>
{
	enum { WithSerializer = true };
};

USTRUCT(BlueprintType, meta = (DisplayName = "Retarget IK Goals Settings"))
struct FIKRetargetIKChainsOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FRetargetIKChainSettings> ChainsToRetarget;
	
	// Draw final IK goal locations. 
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDrawFinalGoals = true;

	// Draw goal locations from source skeleton. 
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDrawSourceLocations = true;

	// Adjust size of goal debug drawing in viewport
	UPROPERTY(EditAnywhere, Category = Debug)
	double GoalDrawSize = 5.0;
	
	// Adjust thickness of goal debug drawing in viewport
	UPROPERTY(EditAnywhere, Category = Debug)
	double GoalDrawThickness = 1.0;

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) override;
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FIKRetargetIKChainsOpSettings> : public TStructOpsTypeTraitsBase2<FIKRetargetIKChainsOpSettings>
{
	enum { WithSerializer = true };
};

#if WITH_EDITOR
struct FChainDebugData
{
	FName TargetChainName;
	FTransform InputTransformStart = FTransform::Identity;
	FTransform InputTransformEnd = FTransform::Identity;
	FTransform OutputTransformEnd = FTransform::Identity;
	FTransform SourceTransformEnd = FTransform::Identity;
};
#endif

USTRUCT(BlueprintType, meta = (DisplayName = "Retarget IK Goals", Deprecated))
struct FIKRetargetIKChainsOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;

	UE_API virtual const UScriptStruct* GetType() const override;

	UE_API virtual const UScriptStruct* GetParentOpType() const override;

	UE_API virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;

	UE_API virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) override;
	
	UPROPERTY()
	FIKRetargetIKChainsOpSettings Settings;

#if WITH_EDITOR
public:
	UE_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual bool HasDebugDrawing() override { return true; };
	
	UE_API void SaveDebugData(
		const FIKRetargetProcessor& InProcessor,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& OutTargetGlobalPose);
	
	TArray<FChainDebugData> AllChainsDebugData;
	FVector DebugRootModification;
	static UE_API FCriticalSection DebugDataMutex;

	UE_API virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	UE_API virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	UE_API virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif

private:

	void RegenerateChainSettings(const FIKRetargetOpBase* InParentOp);
	
	TArray<FIKChainRetargeter> IKChainRetargeters;
};

/* The blueprint/python API for editing a IK Chains Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetIKChainsController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetIKChainsOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetIKChainsOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetIKChainsOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetIKChainsOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
