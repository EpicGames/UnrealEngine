// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Retargeter/IKRetargetOps.h"
#include "OffsetGoalsOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "BlendToSourceOp"

struct FResolvedBoneChain;
struct FIKRigGoalContainer;

USTRUCT(BlueprintType)
struct FIKRetargetOffsetGoalsChainSettings
{
	GENERATED_BODY()

	FIKRetargetOffsetGoalsChainSettings() = default;
	FIKRetargetOffsetGoalsChainSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {}
	bool operator==(const FIKRetargetOffsetGoalsChainSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };

	/** The name of the TARGET chain for these settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;

	/** Range 0 to 1. Default 0. Blend the offsets on this chain on/off. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Chain Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0f;
	
	/** Default 0,0,0. Apply a global-space offset to the IK goal position. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Chain Settings")
	FVector GlobalTranslationOffset = FVector::ZeroVector;

	/** Default 0,0,0. Apply a local-space offset to the IK goal position. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Chain Settings")
	FVector LocalTranslationOffset = FVector::ZeroVector;

	/** Default 0,0,0. Apply a global-space offset to the IK goal rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Chain Settings")
	FRotator GlobalRotationOffset = FRotator::ZeroRotator;

	/** Default 0,0,0. Apply a local-space offset to the IK goal rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Chain Settings")
	FRotator LocalRotationOffset = FRotator::ZeroRotator;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Blend to Source Settings"))
struct FIKRetargetOffsetGoalsOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FIKRetargetOffsetGoalsChainSettings> Chains;

	/** Range 0 to 1. Default 1. Blend all the offsets on/off. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0f;

	/** Range 0 to 1. Default 1. Blend the translation offsets on/off. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double TranslationAlpha = 1.0f;

	/** Range 0 to 1. Default 1. Blend the rotation offsets on/off. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double RotationAlpha = 1.0f;

	virtual const UClass* GetControllerType() const override;

	virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

struct FIKChainToOffset
{
	FIKChainToOffset() = delete;
	FIKChainToOffset(
		const FResolvedBoneChain& InTargetBoneChain,
		const FIKRetargetOffsetGoalsChainSettings& InSettings)
		: TargetBoneChain(&InTargetBoneChain)
		, Settings(&InSettings){}

	void Run(
		FIKRigGoalContainer& InGoalContainer,
		const FIKRetargetOffsetGoalsOpSettings& InOpSettings) const;

private:
	
	// cached data
	const FResolvedBoneChain* TargetBoneChain;
	const FIKRetargetOffsetGoalsChainSettings* Settings;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Offset Goals"))
struct FIKRetargetOffsetGoalsOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	virtual const UScriptStruct* GetSettingsType() const override;

	virtual const UScriptStruct* GetType() const override;

	virtual const UScriptStruct* GetParentOpType() const override;

	virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;

	virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) override;
	
	UPROPERTY()
	FIKRetargetOffsetGoalsOpSettings Settings;

#if WITH_EDITOR
public:
	virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif

	void RegenerateChainSettings(const FIKRetargetOpBase* InParentOp);

private:

	TArray<FIKChainToOffset> ChainsToOffset;
};

/* The blueprint/python API for editing a Offset IK Goals Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetOffsetGoalsController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetOffsetGoalsOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	FIKRetargetOffsetGoalsOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetOffsetGoalsOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	void SetSettings(FIKRetargetOffsetGoalsOpSettings InSettings);
};

#undef UE_API
#undef LOCTEXT_NAMESPACE
