// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargeter.h"

#if WITH_EDITOR
#include "HitProxies.h"
#endif

#include "PelvisMotionOp.generated.h"

#define LOCTEXT_NAMESPACE "PelvisMotionOp"

USTRUCT(BlueprintType, meta = (DisplayName = "Pelvis Motion Settings"))
struct FIKRetargetPelvisMotionOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	/** The Pelvis bone on the source skeleton to copy motion FROM. */
	UPROPERTY(EditAnywhere, Category=Setup, meta=(ReinitializeOnEdit, NotOverrideable))
	FBoneReference SourcePelvisBone;

	/** The Pelvis bone on the target skeleton to copy motion TO. */
	UPROPERTY(EditAnywhere, Category=Setup, meta=(ReinitializeOnEdit, NotOverrideable))
	FBoneReference TargetPelvisBone;

	/** Blend from motion scaled by the size of the character, to the absolute delta motion from the source.
	 * NOTE: Typically, this is only applied to vertical motion (Z) to prevent dampening the pelvis bob on short characters. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scaled or Absolute", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "1.0"))
	FVector BlendToAbsoluteOffset = FVector::ZeroVector;

	/** Range 0 to 1. Default is 0. Set to a value of 1 to turn the pelvis floor constraint ON.
	 * When ON, the floor constraint will adjust the vertical Pelvis motion separately according to the following rules:
	 * 1. When the source pelvis is LOWER than the ref pose, the target pelvis will be lowered proportional to their relative crotch heights.
	 * 2. When the source pelvis is HIGHER than the ref pose, the target pelvis will be raised an identical amount while maintaining any vertical offset in the ref pose.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Floor Constraint", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double FloorConstraintWeight = 0.0f;
	
	/** A negative vertical offset in cm relative to the Pelvis bone of the SOURCE.
	 * NOTE: Adjust this until the green dot is located roughly at the crotch of the character. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Floor Constraint")
	double SourceCrotchOffset = 0.0f;

	/** A negative vertical offset in cm relative to the Pelvis bone of the TARGET.
	 * NOTE: Adjust this until the green dot is located roughly at the crotch of the character. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Floor Constraint")
	double TargetCrotchOffset = 0.0f;

	/** When true, the effect of the floor constraint decays as the pelvis gets further from the floor.
	 * NOTE: This can help avoid over dampening vertical motion on the pelvis when retargeting to characters that are shorter than the source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Floor Constraint")
	bool bUseGroundFalloff = false;
	
	/** Controls the height at which the floor constraint will begin blending in as a percentage of the pelvis height in the retarget pose.
	 * NOTE: Use this to prevent the floor constraint from dampening vertical pelvis motion at normal standing heights.
	 * At 1, the constraint is active at normal standing height. At 0.1, the constraint will only activate when within 10% of the height off the floor. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Floor Constraint", meta = (ClampMin = "0.1", ClampMax = "1.0", UIMin = "0.1", UIMax = "1.0"))
	double GroundFalloffHeightPercent = 0.9f;
	
	/** Range 0 to 1. Default 1. Blends the amount of retargeted pelvis rotation to apply.
	*  At 0 the pelvis is left at the rotation from the retarget pose.
	*  At 1 the pelvis is rotated fully to match the source pelvis rotation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pelvis Rotation", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double RotationAlpha = 1.0f;

	/** Applies a static local-space rotation offset to the pelvis.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pelvis Rotation", meta = (ClampMin = "-180.0", ClampMax = "180.0", UIMin = "-180.0", UIMax = "180.0"))
	FRotator RotationOffsetLocal = FRotator::ZeroRotator;

	/** Applies a static global-space rotation offset to the pelvis.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pelvis Rotation", meta = (ClampMin = "-180.0", ClampMax = "180.0", UIMin = "-180.0", UIMax = "180.0"))
	FRotator RotationOffsetGlobal = FRotator::ZeroRotator;
	
	/** Range 0 to 1. Default 1. Blends the amount of retargeted pelvis translation to apply.
	*  At 0 the pelvis is left at the position from the retarget pose.
	*  At 1 the pelvis will follow the source motion according to the behavior defined in the subsequent settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pelvis Translation", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double TranslationAlpha = 1.0f;

	/** Applies a static local-space translation offset to the pelvis.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pelvis Translation")
	FVector TranslationOffsetLocal = FVector::ZeroVector;
	
	/** Applies a static global-space translation offset to the pelvis.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pelvis Translation")
	FVector TranslationOffsetGlobal = FVector::ZeroVector;
	
	/** Range 0 to 1. Default 0. Blends the retarget pelvis' translation to the exact source location.
	*  At 0 the pelvis is placed at the retargeted location.
	*  At 1 the pelvis is placed at the location of the source's pelvis bone.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pelvis Translation", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double BlendToSourceTranslation = 0.0f;

	/** Per-axis weights for the Blend to Source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pelvis Translation", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	FVector BlendToSourceTranslationWeights = FVector::OneVector;

	/** Default 1. Scales the translation of the pelvis in the horizontal plane (X,Y). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scale Pelvis Translation", meta = (UIMin = "0.0", UIMax = "3.0"))
	double ScaleHorizontal = 1.0f;

	/** Default 1. Scales the translation of the pelvis in the vertical direction (Z). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Scale Pelvis Translation", meta = (UIMin = "0.0", UIMax = "3.0"))
	double ScaleVertical = 1.0f;

	/** Range 0 to 1. Default 1. Control whether modifications made to the pelvis will affect the horizontal component of IK positions.
	*  At 0 the IK positions are independent of the pelvis modifications.
	*  At 1 the IK positions are calculated relative to the modified pelvis location.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Affect IK Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", DisplayName = "Affect IK Horizontal"))
	double AffectIKHorizontal = 1.0f;
	
	/** Range 0 to 1. Default 0. Control whether modifications made to the pelvis will affect the vertical component of IK positions.
	*  At 0 the IK positions are independent of the pelvis modifications.
	*  At 1 the IK positions are calculated relative to the modified pelvis location.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Affect IK Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", DisplayName = "Affect IK Vertical"))
	double AffectIKVertical = 0.0f;

	/** Adjust size of the debug drawing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Debug")
	double DebugDrawSize = 5.0f;
	
	/** Adjust thickness of the debug drawing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Debug")
	double DebugDrawThickness = 5.0f;

	IKRIG_API virtual const UClass* GetControllerType() const override;
	
	IKRIG_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	/** FBoneReference needs a skeleton for the UI widget. */
#if WITH_EDITOR
	IKRIG_API virtual USkeleton* GetSkeleton(const FName InPropertyName) override;
#endif

	UPROPERTY(meta = (DeprecatedProperty))
	FVector TranslationOffset_DEPRECATED = FVector::ZeroVector;

	UPROPERTY(meta = (DeprecatedProperty))
	FRotator RotationOffset_DEPRECATED = FRotator::ZeroRotator;
	
	UPROPERTY(meta = (DeprecatedProperty))
	bool bEnableDebugDraw_DEPRECATED = true;

	virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) override;
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FIKRetargetPelvisMotionOpSettings> : public TStructOpsTypeTraitsBase2<FIKRetargetPelvisMotionOpSettings>
{
	enum { WithSerializer = true };
};

struct FPelvisSource
{
	FName BoneName;
	int32 BoneIndex;
	FQuat InitialRotation;
	float InitialHeightInverse;
	FVector InitialPosition;
	FVector CurrentPosition;
	FQuat CurrentRotation;
};

struct FPelvisTarget
{
	FName BoneName;
	int32 BoneIndex;
	FVector InitialPosition;
	FQuat InitialRotation;
	float InitialHeight;

	FVector PelvisTranslationDelta;
	FQuat PelvisRotationDelta;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Pelvis Motion"))
struct FIKRetargetPelvisMotionOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	IKRIG_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	IKRIG_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	IKRIG_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	IKRIG_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	IKRIG_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	IKRIG_API virtual const UScriptStruct* GetSettingsType() const override;

	IKRIG_API virtual const UScriptStruct* GetType() const override;
	
	IKRIG_API virtual void CollectRetargetedBones(TSet<int32>& OutRetargetedBones) const override;

	IKRIG_API FName GetPelvisBoneName(ERetargetSourceOrTarget SourceOrTarget) const;
	
	IKRIG_API FVector GetGlobalScaleVector() const;

	IKRIG_API FVector GetAffectIKWeightAsVector() const;

	IKRIG_API FVector GetPelvisTranslationOffset() const;

	int32 GetPelvisBoneIndex(ERetargetSourceOrTarget SourceOrTarget) const;

	UPROPERTY()
	FIKRetargetPelvisMotionOpSettings Settings;

private:

	void Reset();
	
	bool InitializeSource(
		const FName SourcePelvisBoneName,
		const FRetargetSkeleton& SourceSkeleton,
		FIKRigLogger& Log);
	
	bool InitializeTarget(
		const FName TargetPelvisBoneName,
		const FTargetSkeleton& TargetSkeleton,
		FIKRigLogger& Log);
	
	void RetargetPelvisTransform(const TArray<FTransform>& SourceGlobalPose, FTransform& OutPelvisGlobalPose);

	// transient work data
	FPelvisSource Source;
	FPelvisTarget Target;
	FVector GlobalScaleFactor;
	
#if WITH_EDITOR
public:
	
	IKRIG_API virtual FText GetWarningMessage() const override;
	
	IKRIG_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual bool HasDebugDrawing() override { return true; };
	
	FTransform CurrentPelvisTransform;
	static IKRIG_API FCriticalSection DebugDataMutex;
#endif
};

/* The blueprint/python API for editing a Pelvis Motion Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetPelvisMotionController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetPelvisMotionOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	IKRIG_API FIKRetargetPelvisMotionOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetPelvisMotionOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	IKRIG_API void SetSettings(FIKRetargetPelvisMotionOpSettings InSettings);

	/* Set the pelvis bone for the source.
	 * @param InSourcePelvisBone the name of the pelvis bone on the source skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	IKRIG_API void SetSourcePelvisBone(const FName InSourcePelvisBone);

	/* Get the pelvis bone for the source.
	 * @return the name of the pelvis bone on the source skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	IKRIG_API FName GetSourcePelvisBone();

	/* Set the pelvis bone for the target.
	 * @param InTargetPelvisBone the name of the pelvis bone on the target skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	IKRIG_API void SetTargetPelvisBone(const FName InTargetPelvisBone);

	/* Get the pelvis bone for the target.
	 * @return the name of the pelvis bone on the target skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	IKRIG_API FName GetTargetPelvisBone();

private:

	FIKRetargetPelvisMotionOpSettings& GetPelvisOpSettings() const;
};

#if WITH_EDITOR
// select root control to edit root settings
struct HIKRetargetEditorRootProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( IKRIG_API );
	
	HIKRetargetEditorRootProxy()
		: HHitProxy(HPP_World){}

	virtual EMouseCursor::Type GetMouseCursor()
	{
		return EMouseCursor::Crosshairs;
	}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};
#endif

#undef LOCTEXT_NAMESPACE
