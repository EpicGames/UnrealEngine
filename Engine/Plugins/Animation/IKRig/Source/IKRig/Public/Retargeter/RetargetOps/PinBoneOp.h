// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetProcessor.h"

#include "PinBoneOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "PinBoneOp"

UENUM()
enum class EPinBoneType : uint8
{
	FullTransform,
	TranslateOnly,
	RotateOnly,
	ScaleOnly
};

USTRUCT(BlueprintType)
struct FPinBoneData
{
	GENERATED_BODY()
	
	void CachePinData(ERetargetSourceOrTarget InSkeletonToCopyFrom, const FIKRetargetProcessor& Processor);

	// The bone to copy FROM.
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ReinitializeOnEdit, NotOverrideable))
	FBoneReference BoneToCopyFrom;
	// The bone to copy TO. Will have its transform pinned to BoneToCopyFrom
	UPROPERTY(EditAnywhere, Category=Settings, meta=(ReinitializeOnEdit, NotOverrideable))
	FBoneReference BoneToCopyTo;

	// the scale factor used to calculate all the cached offset (lazy evaluated when scale factor is modified)
	FRetargetPoseScaleWithPivot LastUsedSourceScale;
	// the local transform of the BoneToCopyTo in the ref pose
	FTransform LocalRefPoseBoneToCopyTo;
	// the local transform of the BoneToCopyFrom in the ref pose
	FTransform LocalRefPoseBoneToCopyFrom;
	// the relative transform between the BoneToCopyFrom and BoneToCopyTo in the reference pose
	FTransform OffsetFromBoneToCopyFromInRefPose;

	// deprecated properties
	UPROPERTY(meta=(DeprecatedProperty))
	FName BoneToPin_DEPRECATED;
	UPROPERTY(meta=(DeprecatedProperty))
	FName BoneToPinTo_DEPRECATED;

	void PostSerialize(const FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FPinBoneData> : public TStructOpsTypeTraitsBase2<FPinBoneData>
{
	enum
	{
		WithPostSerialize = true,
	};
};

UENUM()
enum class EPinBoneTranslationMode : uint8
{
	// Copy the global position of BoneToCopyFrom
	CopyGlobalPosition,
	// Copy the local translation (relative to parent), in global space, from BoneToCopyFrom to BoneToCopyTo
	CopyLocalPosition,
	// Copy the local translation, in global space, from BoneToCopyFrom to BoneToCopyTo and add the difference in lengths in the reference pose
	CopyLocalPositionRelativeOffset,
	// Copy the local translation, in global space, from BoneToCopyFrom to BoneToCopyTo and scale by the relative lengths in the reference pose
	CopyLocalPositionRelativeScaled,
	// Copy the global position of BoneToCopyFrom, and add the local offset between BoneToCopyFrom and BoneToCopyTo in the reference pose
	CopyGlobalPositionAndMaintainOffset,
};

UENUM()
enum class EPinBoneRotationMode : uint8
{
	// no offset is maintained
	CopyGlobalRotation,
	// maintains rotation offset between BoneToCopyTo and BoneToCopyFrom in the reference pose
	MaintainOffsetFromBoneToCopyFrom
};

USTRUCT(BlueprintType, meta = (DisplayName = "Pin Bone Settings"))
struct FIKRetargetPinBoneOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	/** A list of bone-pairs to copy transforms between */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Setup, meta=(ReinitializeOnEdit, NotOverrideable))
	TArray<FPinBoneData> BonesToPin;

	/** Which skeleton to copy animation from. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Setup, meta=(ReinitializeOnEdit))
	ERetargetSourceOrTarget SkeletonToCopyFrom = ERetargetSourceOrTarget::Target;

	/** Copy the translation of the bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Translation)
	bool bCopyTranslation = true;
	
	/** The method used to calculate the translation of the bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Translation)
	EPinBoneTranslationMode TranslationMode = EPinBoneTranslationMode::CopyGlobalPosition;

	/** Copy the rotation of the bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	bool bCopyRotation = true;

	/** The method used to calculate the rotation of the bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotation)
	EPinBoneRotationMode RotationMode = EPinBoneRotationMode::CopyGlobalRotation;
	
	/** Copy the scale of the bone. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Scale)
	bool bCopyScale = true;

	/** Update children bone transforms (recursively). */
	UPROPERTY(EditAnywhere, Category=PropagateTransform)
	bool bPropagateToChildren = false;

	/** A manual offset to apply in global space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Offset)
	FTransform GlobalOffset;

	/** A manual offset to apply in local space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Offset)
	FTransform LocalOffset;

	UE_API virtual const UClass* GetControllerType() const override;
	
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

#if WITH_EDITOR
	UE_API virtual USkeleton* GetSkeleton(const FName InPropertyName) override;
#endif

	UE_API virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) override;
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FIKRetargetPinBoneOpSettings> : public TStructOpsTypeTraitsBase2<FIKRetargetPinBoneOpSettings>
{
	enum { WithSerializer = true };
};

USTRUCT(BlueprintType, meta = (DisplayName = "Pin Bones"))
struct FIKRetargetPinBoneOp : public FIKRetargetOpBase
{
	GENERATED_BODY()

	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& Processor,
		const FRetargetSkeleton& SourceSkeleton,
		const FTargetSkeleton& TargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& Processor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;
	
	UPROPERTY()
	FIKRetargetPinBoneOpSettings Settings;
	
#if WITH_EDITOR
	UE_API virtual FText GetWarningMessage() const override;
#endif

private:

	bool bFoundAllBonesToPin = false;

	FTransform GetNewBoneTransform(
		const FPinBoneData& PinData,
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) const;
};

/* The blueprint/python API for editing a Pin Bone Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetPinBoneController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetPinBoneOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetPinBoneOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetPinBoneOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetPinBoneOpSettings InSettings);

	/* Clear all the bone pairs */
	UFUNCTION(BlueprintCallable, Category = BonePairs)
	UE_API void ClearAllBonePairs();

	/* Add a pair of bones to copy animation between.
	 * @param InBoneToCopyFrom the name of the bone to copy animation from
	 * @param InBoneToCopyTo the name of the bone to apply animation to
	 * NOTE: if the bone to copy to is already present in the op, it will be updated with the new bone to copy from. */
	UFUNCTION(BlueprintCallable, Category = BonePairs)
	UE_API void SetBonePair(const FName InBoneToCopyFrom, const FName InBoneToCopyTo);

	/* Get all the bone pairs currently stored in the op.
	 * @return a map with target bones as keys and source bones as values. */
	UFUNCTION(BlueprintCallable, Category = BonePairs)
	UE_API TMap<FName,FName> GetAllBonePairs();
};

//
// BEGIN LEGACY UOBJECT-based OP
//

// NOTE: This type has been replaced with FIKRetargetPinBoneOp.
UCLASS(MinimalAPI)
class UPinBoneOp : public URetargetOpBase
{
	GENERATED_BODY()

public:
	
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) override
	{
		OutInstancedStruct.InitializeAs(FIKRetargetPinBoneOp::StaticStruct());
		FIKRetargetPinBoneOp& NewOp = OutInstancedStruct.GetMutable<FIKRetargetPinBoneOp>();
		NewOp.SetEnabled(bIsEnabled);
		NewOp.Settings.BonesToPin = BonesToPin;
		NewOp.Settings.SkeletonToCopyFrom = PinTo;
		NewOp.Settings.bCopyTranslation = bCopyTranslation;
		NewOp.Settings.bCopyRotation = bCopyRotation;
		NewOp.Settings.bCopyScale = bCopyScale;
		NewOp.Settings.TranslationMode = TranslationMode;
		NewOp.Settings.RotationMode = RotationMode;
		NewOp.Settings.GlobalOffset = GlobalOffset;
		NewOp.Settings.LocalOffset = LocalOffset;
	};

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	// ~END UObject interface
	
	UPROPERTY()
	TArray<FPinBoneData> BonesToPin;
	UPROPERTY()
	ERetargetSourceOrTarget PinTo = ERetargetSourceOrTarget::Target;
	UPROPERTY()
	bool bCopyTranslation = true;
	UPROPERTY()
	bool bCopyRotation = true;
	UPROPERTY()
	bool bCopyScale = true;
	UPROPERTY()
	EPinBoneTranslationMode TranslationMode;
	UPROPERTY()
	EPinBoneRotationMode RotationMode;
	UPROPERTY()
	FTransform GlobalOffset;
	UPROPERTY()
	FTransform LocalOffset;

private:

	UE_DEPRECATED(5.6, "Uses separate translation/rotation offset modes")
	UPROPERTY()
	bool bMaintainOffset_DEPRECATED = true;
	
	UE_DEPRECATED(5.6, "Uses separate translate/rotate/scale checkboxes")
	UPROPERTY()
	EPinBoneType PinType_DEPRECATED;
};

//
// END LEGACY UOBJECT-based OP
//

#undef LOCTEXT_NAMESPACE

#undef UE_API
