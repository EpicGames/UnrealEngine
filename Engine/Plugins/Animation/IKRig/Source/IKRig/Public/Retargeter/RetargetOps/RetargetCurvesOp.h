// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "StructUtils/InstancedStruct.h"
#include "Animation/AnimCurveTypes.h"

#include "RetargetCurvesOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "RetargetCurvesOp"

/**
 * This is the base class for defining all animation curves operations including curve remapping and rig mapping.
*/
USTRUCT(BlueprintType)
struct FIKRetargetCurvesOpBase : public FIKRetargetOpBase
{
	GENERATED_BODY()

public:
	// NOTE: this op does not do anything in Initialize() or Run().
	// In the future we may remove this coupling and make the op itself do the work via callbacks
	virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override
	{
		bIsInitialized = true;
		return true;
	};

	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override {
	};

	virtual const UScriptStruct* GetType() const override { return FIKRetargetCurvesOpBase::StaticStruct(); }

	bool CanHaveChildOps() const override { return true; }

	virtual bool IsSingleton() const override { return true; };

	// Designator of an "abstract" base struct, in this case used to classify curve related ops
	virtual bool IsAbstract() const override { return true; };

	/** (optional) a function which returns true if the operation performs curve processing, false otherwise (default is true) */
	virtual bool HasCurveProcessing() const { return true;  }

	using FFrameValues = TArray<TArray<TOptional<float>>>;

	struct FCurveData
	{
		TArray<FName> Names;
		TArray<int32> Flags;
		TArray<FLinearColor> Colors;
	};

	/** (optional) a function which can be used during batch processing to apply any operations to curve data for each frame of an anim sequence. By default, this just moves the input data to the output data*/
	virtual void ProcessAnimSequenceCurves(FIKRetargetCurvesOpBase::FCurveData InCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues InCurveFrameValues,
		FIKRetargetCurvesOpBase::FCurveData& OutCurveMetaData, FIKRetargetCurvesOpBase::FFrameValues& OutCurveFrameValues) const
	{
		OutCurveMetaData = MoveTemp(InCurveMetaData);
		OutCurveFrameValues = MoveTemp(InCurveFrameValues);
	}

	/** a function which allows the user to manually indicate whether to take source curves from the source anim instance, or from the current pose context, since only the first enabled curve processing node must take inputs from the source anim instance*/
	void SetTakeInputCurvesFromSourceAnimInstance(bool bInTakeInputCurvesFromSourceAnimInstance)
	{
		bTakeInputCurvesFromSourceAnimInstance = bInTakeInputCurvesFromSourceAnimInstance;
	}

	/** a function which allows the user to manually indicate whether to take source curves from the source anim instance, or from the current pose context, since only the first enabled curve processing node must take inputs from the source anim instance */
	bool GetTakeInputCurvesFromSourceAnimInstance() const
	{
		return bTakeInputCurvesFromSourceAnimInstance;
	}

	// cached curves, copied on the game thread in PreUpdate()
	FBlendedHeapCurve SourceCurves;

protected:
	/** (optional) determines whether input curves are taken from the source anim instance or from the current pose context (for example when we have multiple nodes which process curves). Can be set on the fly */
	UPROPERTY(Transient)
	bool bTakeInputCurvesFromSourceAnimInstance = true;
};

#undef LOCTEXT_NAMESPACE
#undef UE_API