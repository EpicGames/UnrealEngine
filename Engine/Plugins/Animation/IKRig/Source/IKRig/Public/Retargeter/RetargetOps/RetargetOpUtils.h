// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RetargetOpUtils.generated.h"

// These are settings for an adaptive first-order low-pass filter based on the "One-Euro" 2012 paper: https://gery.casiez.net/1euro/
USTRUCT(BlueprintType)
struct FOneEuroFilterSettings
{
	GENERATED_BODY()

	/* Raises the cutoff as speed grows.
	 * Larger values are snappier on fast turns.
	 * Smaller values are smoother but laggier during quick motion.
	 * Typical sweet spot: 0.3 – 0.8 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="One Euro Filter")
	double Responsiveness = 0.5;
	
	/* Hz. Sets the low-pass cutoff when motion is near zero.
	 * Higher = less smoothing at rest (more responsive but more jitter).
	 * Lower = more smoothing at rest (less jitter but more “stickiness”). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="One Euro Filter")
	double CutoffFrequency = 1.5;

	/* Hz. Low-passes the raw velocity before we use it to adapt the derivative cutoff.
	* If you see breathing/pumping of the smoothing during motion onsets or reversals, lower this value (e.g., 30 → 20 Hz).
	* If responsiveness to fast changes is sluggish, raise this a bit.
	* Good starting range: 15–30 Hz. Keep under Nyquist frequency (frame_rate/2). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="One Euro Filter")
	double VelocityCutoffFrequency = 20.0;
};

// filter a double over time using the "One Euro" method
struct FOneEuroScalarFilter
{
	double Update(const double InValue, const double InDeltaTime, const FOneEuroFilterSettings& InSettings);

	void Reset(const double InValue);
	void Reset() { bIsFirstRun = true; }

	static double AlphaFromHz(const double InDeltaTime, const double InFreqCutoffHz);

	static double ClampDeltaTime(const double InDeltaTime);

	static double ClampFreqToFramerate(const double InHz, const double InDeltaTime);

private:
	
	double PrevValue = 0.0;
	double PrevRawValue = 0.0;
	double PrevValueDerivative = 0.0;
	bool bIsFirstRun = true;
};

// filter a 3d vector over time using the "One Euro" method
struct FOneEuroVectorFilter
{
	FVector Update(const FVector& InValue, const double InDeltaTime, const FOneEuroFilterSettings& InSettings);

	void Reset(const FVector& StartingValue);
	void Reset();

private:
	
	FOneEuroScalarFilter X;
	FOneEuroScalarFilter Y;
	FOneEuroScalarFilter Z;
};

// filter a quaternion over time using the "One Euro" method
struct FOneEuroQuatFilter
{
	FQuat Update(const FQuat& InTargetRotation, double InDeltaTime, const FOneEuroFilterSettings& InSettings);

	void Reset() { bIsFirstRun = true; };

private:
	
	FQuat PrevFilteredRot = FQuat::Identity;
	FQuat PrevRawTarget = FQuat::Identity;
	FVector PrevHalfAngVel = FVector::ZeroVector;
	FVector PrevPhiFilt = FVector::ZeroVector;
	bool bIsFirstRun = true;
};