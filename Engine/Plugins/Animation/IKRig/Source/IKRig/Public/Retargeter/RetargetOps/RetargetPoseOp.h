// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"

#include "RetargetPoseOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "RetargetPoseOp"

USTRUCT(BlueprintType, meta = (DisplayName = "Retarget Pose Op Settings"))
struct FIKRetargetPoseOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	// When true, overrides the default retarget pose for the source skeleton.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Source Pose")
	bool bOverrideSourcePose = false;

	// A retarget pose that is applied to the source skeleton. Must match the name of a retarget pose stored in this asset.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Source Pose")
	FName SourcePoseToUse;

	// When true, overrides the default retarget pose for the target skeleton.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Target Pose")
	bool bOverrideTargetPose = false;

	// A retarget pose that is applied to the target skeleton. Must match the name of a retarget pose stored in this asset.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Target Pose")
	FName TargetPoseToUse;
	
	UE_API virtual const UClass* GetControllerType() const override;
	
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Retarget Pose"))
struct FIKRetargetPoseOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;

	virtual bool IsSingleton() const override { return true; }

private:
	
	UPROPERTY()
	FIKRetargetPoseOpSettings Settings;
};

/* The blueprint/python API for editing a Retarget Pose Op */
UCLASS(BlueprintType)
class UIKRetargetPoseController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetPoseOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetPoseOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetPoseOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetPoseOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
