// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Animation/BoneReference.h"

#include "RootMotionGeneratorOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "URootMotionGeneratorOp"

// where to copy the motion of the root from
UENUM()
enum class ERootMotionSource : uint8
{
	CopyFromSourceRoot,
	GenerateFromTargetPelvis,
};

// where to copy the height of the root from
UENUM()
enum class ERootMotionHeightSource : uint8
{
	CopyHeightFromSource,
	SnapToGround,
};

USTRUCT(BlueprintType, meta = (DisplayName = "Root Motion Settings"))
struct FIKRetargetRootMotionOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	// The root of the source skeleton.
	UPROPERTY(EditAnywhere, Category="Bones", meta=(ReinitializeOnEdit, NotOverrideable))
	FBoneReference SourceRoot;

	// The root of the target skeleton.
	UPROPERTY(EditAnywhere, Category="Bones", meta=(ReinitializeOnEdit, NotOverrideable))
	FBoneReference TargetRoot;
	
	// Where to copy the root motion from.
	// Copy From Source Root: copies the motion from the source root bone, but scales it according to relative height difference.
	// Generate From Target Pelvis: uses the retargeted Pelvis motion to generate root motion.
	// NOTE: Generating root motion from the Pelvis 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Motion Source")
	ERootMotionSource RootMotionSource = ERootMotionSource::CopyFromSourceRoot;

	// The pelvis of the target skeleton. Only used when "Root Motion Source" is set to target pelvis.
	UPROPERTY(EditAnywhere, Category="Motion Source", meta=(ReinitializeOnEdit, NotOverrideable))
	FBoneReference TargetPelvis;

	// Applies only when generating root motion from the Pelvis.
	// When true, the applied offset will be rotated by the Pelvis.
	// (NOTE: This may cause unwanted rotations, for example if Pelvis Yaw is animated.)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Motion Source")
	bool bRotateWithPelvis = false;
	
	// How to set the height of the root bone.
	// Copy Height From Source: copies the height of the root bone on the source skeleton's root bone.
	// Snap To Ground: sets the root bone height to the ground plane (Z=0).
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Height Source")
	ERootMotionHeightSource RootHeightSource = ERootMotionHeightSource::CopyHeightFromSource;
	
	// A manual offset to apply in global space to the root bone.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Offset")
	FTransform GlobalOffset;

	// Applies only when generating root motion from the Pelvis.
	// Maintains the offset between the root and pelvis as recorded in the target retarget pose.
	// If false, the root bone is placed directly under the Pelvis bone.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Offset")
	bool bMaintainOffsetFromPelvis = true;

	// Will transform all children of the target root that are not themselves part of a retarget chain.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Update Children")
	bool bPropagateToNonRetargetedChildren = true;

	UE_API virtual const UClass* GetControllerType() const override;
	
	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	/** FBoneReference needs a skeleton for the UI widget. */
#if WITH_EDITOR
	UE_API virtual USkeleton* GetSkeleton(const FName InPropertyName) override;
#endif
	
	// Changed to FBoneReference to support bone selector UI
	UPROPERTY(meta=(DeprecatedProperty))
	FName SourceRootBone_DEPRECATED;
	UPROPERTY(meta=(DeprecatedProperty))
	FName TargetRootBone_DEPRECATED;
	UPROPERTY(meta=(DeprecatedProperty))
	FName TargetPelvisBone_DEPRECATED;

	virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) override;
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FIKRetargetRootMotionOpSettings> : public TStructOpsTypeTraitsBase2<FIKRetargetRootMotionOpSettings>
{
	enum { WithSerializer = true };
};

USTRUCT(BlueprintType, meta = (DisplayName = "Root Motion"))
struct FIKRetargetRootMotionOp : public FIKRetargetOpBase
{
	GENERATED_BODY()

	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& Processor,
		const FRetargetSkeleton& SourceSkeleton,
		const FTargetSkeleton& TargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& Processor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void PostInitialize(
		const FIKRetargetProcessor& Processor,
		const FRetargetSkeleton& SourceSkeleton,
		const FTargetSkeleton& TargetSkeleton,
		FIKRigLogger& Log) override;
	
	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;
	
	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;
	
	UPROPERTY()
	FIKRetargetRootMotionOpSettings Settings;

#if WITH_EDITOR
	UE_API virtual FText GetWarningMessage() const override;
#endif

private:

	void Reset();

	void GenerateRootMotionFromTargetPelvis(
		FTransform& OutRootTransform,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& InTargetGlobalPose) const;
	
	void CopyRootMotionFromSourceRoot(
		FTransform& OutRootTransform,
		const FIKRetargetProcessor& Processor,
		const TArray<FTransform>& InSourceGlobalPose) const;
	
	int32 SourceRootIndex = INDEX_NONE;
	int32 TargetRootIndex = INDEX_NONE;
	int32 TargetPelvisIndex = INDEX_NONE;

	FTransform TargetPelvisRelativeToTargetRootRefPose;
	FTransform TargetPelvisInRefPose;
	FTransform SourceRootInRefPose;
	FTransform TargetRootInRefPose;
	
	TArray<int32> NonRetargetedChildrenOfRoot;
};

/* The blueprint/python API for editing a Root Motion Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetRootMotionController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetRootMotionOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetRootMotionOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetRootMotionOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetRootMotionOpSettings InSettings);

	/* Set the root bone for the source.
	 * @param InSourceRootBone the name of the root bone on the source skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSourceRootBone(const FName InSourceRootBone);

	/* Get the root bone for the source.
	 * @return the name of the root bone on the source skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FName GetSourceRootBone();

	/* Set the root bone for the target.
	 * @param InTargetRootBone the name of the root bone on the target skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetTargetRootBone(const FName InTargetRootBone);

	/* Get the root bone for the target.
	 * @return the name of the root bone on the target skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FName GetTargetRootBone();

	/* Set the pelvis bone for the target.
	 * @param InTargetPelvisBone the name of the pelvis bone on the target skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetTargetPelvisBone(const FName InTargetPelvisBone);

	/* Get the pelvis bone for the target.
	 * @return the name of the pelvis bone on the target skeleton. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FName GetTargetPelvisBone();

private:

	FIKRetargetRootMotionOpSettings* GetSettingsPtr() const;
};

//
// BEGIN DEPRECATED UOBJECT-based OP
//

// NOTE: This type has been replaced with FIKRetargetRootMotionOp.
UCLASS(MinimalAPI)
class URootMotionGeneratorOp : public URetargetOpBase
{
	GENERATED_BODY()

public:
	
	UE_API virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) override;

	UPROPERTY()
	FName SourceRootBone;
	UPROPERTY()
	FName TargetRootBone;
	UPROPERTY()
	FName TargetPelvisBone;
	UPROPERTY()
	ERootMotionSource RootMotionSource;
	UPROPERTY()
	ERootMotionHeightSource RootHeightSource;
	UPROPERTY()
	bool bPropagateToNonRetargetedChildren = true;
	UPROPERTY()
	bool bMaintainOffsetFromPelvis = true;
	UPROPERTY()
	bool bRotateWithPelvis = false;
	UPROPERTY()
	FTransform GlobalOffset;
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
