// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetChainMapping.h"
#include "Retargeter/IKRetargetOps.h"
#include "Rig/IKRigProcessor.h"

#include "RunIKRigOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "RunIKRigOp"

struct FRetargeterBoneChains;
struct FResolvedBoneChain;
struct FIKRetargetPelvisMotionOp;

USTRUCT(BlueprintType)
struct FIKRetargetRunIKRigChainSettings
{
	GENERATED_BODY()

	FIKRetargetRunIKRigChainSettings() = default;
	FIKRetargetRunIKRigChainSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {}
	bool operator==(const FIKRetargetRunIKRigChainSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };

	/** The name of the TARGET chain to run IK on. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;
	
	/** Enable/disable IK on this chain.
	 * NOTE: This actually removes the goal from the internal IK Rig solve and triggers a reinitialization of the solver stack when toggled.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta=(ReinitializeOnEdit))
	bool bEnableIK = true;

	/** Range 0 to 1. Default 1. Retargets the goal position based on the source chain length and direction.
	* At 0, the goal is left at the input pose bone location.
	* At 1, the goal is translated in the direction of the source chain, at a distance from the base of the chain proportional to how much it is compressed or stretched. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings",
		meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bEnableIK"))
	double ChainPositionAlpha = 1.0f;

	/** Range 0 to 1. Default 1. Retargets the goal rotation based on the source chain end bone rotation.
	* At 0, the goal is left at the input pose bone rotation.
	* At 1, the goal is rotated by the same world space rotation applied to the source chain end bone.
	* NOTE: if the target end bone rotation was already retargeted before this op, this setting may have no effect.
	* It will not double-rotate the goal. This setting is useful if no FK retargeting is being performed on this chain before the IK op. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings",
		meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bEnableIK"))
	double ChainRotationAlpha = 1.0f;

	/** Default false. If true, the alpha values here will override the goal alpha settings from the IK Rig asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta=(ReinitializeOnEdit, EditCondition = "bEnableIK"))
	bool bOverrideGoalAlpha = false;
	
	/** Range 0 to 1. Default 1. Blends all goal translation offsets on/off.
	 * At 0, the goal is left at the input pose bone location.
	 * At 1, the goal is moved to the location generated by the IK goal ops. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings",
		meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bEnableIK && bOverrideGoalAlpha"))
	double FinalPositionAlpha = 1.0f;

	/** Range 0 to 1. Default 1. Blends all goal rotation offsets on/off.
	 * At 0, the goal is left at the input pose bone rotation.
	 * At 1, the goal is rotated to the orientation generated by the IK goal ops. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings",
		meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "bEnableIK && bOverrideGoalAlpha"))
	double FinalRotationAlpha = 1.0f;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Solve IK Goal Settings"))
struct FIKRetargetRunIKRigOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FIKRetargetRunIKRigChainSettings> Chains;

	/** The IK Rig asset to run when this op is executed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "IK Rig Asset", meta=(ReinitializeOnEdit))
	TObjectPtr<const UIKRigDefinition> IKRigAsset = nullptr;
	
	// Draw IK goal locations. 
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDrawGoals = true;

	// Draw locations of the source bone (pre-solve)
	UPROPERTY(EditAnywhere, Category = Debug)
	bool bDrawGoalBoneLocations = true;

	// Adjust size of goal debug drawing in viewport
	UPROPERTY(EditAnywhere, Category = Debug)
	double GoalDrawSize = 5.0;
	
	// Adjust thickness of goal debug drawing in viewport
	UPROPERTY(EditAnywhere, Category = Debug)
	double GoalDrawThickness = 1.0;

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

struct FRunIKRigChain
{
	FRunIKRigChain(
		const FResolvedBoneChain& InSourceBoneChain,
		const FResolvedBoneChain& InTargetBoneChain,
		const FIKRetargetRunIKRigChainSettings& InSettings,
		FIKRigLogger& InLog);

	void Run(
		FIKRigGoalContainer& InGoalContainer,
		const TArray<FTransform>& InSourceGlobalPose,
		const TArray<FTransform>& InTargetGlobalPose,
		const FIKRetargetRunIKRigOpSettings& InOpSettings,
		const FIKRetargetPelvisMotionOp* InPelvisMotionOp) const;

	void SetGoalAlpha(FIKRigGoalContainer& InGoalContainer);
	
	// cached data
	bool bIsReady = false;
	const FResolvedBoneChain* SourceBoneChain = nullptr;
	const FResolvedBoneChain* TargetBoneChain = nullptr;
	const FIKRetargetRunIKRigChainSettings* Settings = nullptr;
	double InvInitialLengthSource = 1.0f;
	double InitialTargetChainLength = 1.0f;
};

#if WITH_EDITOR
struct FRunIKRigOpGoalDebugData
{
	FName GoalName;
	FTransform InitialGoal;
	FTransform CurrentGoal;
};
#endif

USTRUCT(BlueprintType, meta = (DisplayName = "Run IK Rig"))
struct FIKRetargetRunIKRigOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;

	UE_API virtual const UScriptStruct* GetType() const override;
	
	virtual bool CanHaveChildOps() const override { return true; };

	UE_API virtual void InitializeBeforeChildren(
		FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		FIKRigLogger& Log) override;
	
	UE_API virtual void RunBeforeChildren(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual const UIKRigDefinition* GetCustomTargetIKRig() const override;

	UE_API virtual FRetargetChainMapping* GetChainMapping() override;
	
	UE_API virtual void OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent) override;

	// a helper function that sub-ops can use to validate IK chains before use
	static bool ValidateIKChain(
		const FName& InChainToValidate,
		const FIKRigGoalContainer& InGoalContainer,
		const FRetargeterBoneChains& InBoneChains,
		const FIKRetargetOpBase* InOp,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog,
		const FResolvedBoneChain*& OutSourceBoneChain,
		const FResolvedBoneChain*& OutTargetBoneChain);

	UPROPERTY()
	FIKRetargetRunIKRigOpSettings Settings;
	
	/* This op maintains its own chain mapping table to allow per-op mapping */
	UPROPERTY()
	FRetargetChainMapping ChainMapping;

#if WITH_EDITOR
	
	UE_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	virtual uint8* GetChainSettingsMemory(const FName InChainName) override;

	virtual bool HasDebugDrawing() override { return true; };
	
	UE_API void SaveInitialGoalTransformsIntoDebugData(const FIKRetargetProcessor& InProcessor, const TArray<FTransform>& InTargetGlobalPose);
	
	UE_API void SaveCurrentGoalTransformsIntoDebugData();
	
	TArray<FRunIKRigOpGoalDebugData> GoalDebugData;
	static UE_API FCriticalSection DebugDataMutex;
#endif

	UE_API TArray<FName> GetRequiredTargetChains() const;
	void RegenerateChainSettings();
	
private:

	void ResetGoalContainer(const TArray<FTransform> &InTargetGlobalPose, FIKRigGoalContainer& InOutGoalContainer);

	TArray<FRunIKRigChain> Chains;
	
	/** The IK Rig processor for running IK on the target */
	FIKRigProcessor IKRigProcessor;

	TMap<FName, int32> GoalBoneIndicesMap;
};

/* The blueprint/python API for editing a Run IK Rig Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetRunIKRigController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetRunIKRigOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetRunIKRigOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetRunIKRigOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetRunIKRigOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
