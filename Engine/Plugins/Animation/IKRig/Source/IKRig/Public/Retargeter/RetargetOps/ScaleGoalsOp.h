// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "ScaleGoalsOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "ScaleGoalsOp"

struct FResolvedBoneChain;
struct FIKRigGoalContainer;

USTRUCT(BlueprintType)
struct FIKRetargetScaleGoalsChainSettings
{
	GENERATED_BODY()

	FIKRetargetScaleGoalsChainSettings() = default;
	FIKRetargetScaleGoalsChainSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {}
	bool operator==(const FIKRetargetScaleGoalsChainSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };

	/** The name of the TARGET chain for these settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;

	/** Range 0 to infinity. Default 1. Scales the vertical component of the IK goal's position. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "2.0"))
	double ScaleVertical = 1.0f;

	/** Range 0 to infinity. Default 1. Scales the horizontal component of the IK goal's position. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "2.0"))
	double ScaleHorizontal = 1.0f;
	
	/** Range 0 to infinity. Default 1. Brings IK goal closer (0) or further (1+) from origin of chain.
	*  At 0 the effector is placed at the origin of the chain (ie Shoulder, Hip etc).
	*  At 1 the effector is left at the end of the chain (ie Wrist, Foot etc)
	*  Values in-between 0-1 will slide the effector along the vector from the start to the end of the chain.
	*  Values greater than 1 will stretch the chain beyond the retargeted length. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "2.0"))
	double ScaleAlongChain = 1.0f;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Scale Goals Settings"))
struct FIKRetargetScaleGoalsOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FIKRetargetScaleGoalsChainSettings> Chains;

	/** Range 0 to 1. Default 1. Blend the effect on all chains on/off. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double Alpha = 1.0f;

	virtual const UClass* GetControllerType() const override;

	virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

struct FIKChainToScale
{
	FIKChainToScale() = delete;
	FIKChainToScale(
		const FResolvedBoneChain& InTargetBoneChain,
		const FIKRetargetScaleGoalsChainSettings& InSettings)
		: TargetBoneChain(&InTargetBoneChain)
		, Settings(&InSettings){}

	void Run(
		FIKRigGoalContainer& InGoalContainer,
		const TArray<FTransform>& InTargetGlobalPose,
		const FIKRetargetScaleGoalsOpSettings& InOpSettings) const;

private:
	
	// cached data
	const FResolvedBoneChain* TargetBoneChain;
	const FIKRetargetScaleGoalsChainSettings* Settings;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Scale Goals"))
struct FIKRetargetScaleGoalsOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	virtual const UScriptStruct* GetSettingsType() const override;

	virtual const UScriptStruct* GetType() const override;

	virtual const UScriptStruct* GetParentOpType() const override;

	virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;

	virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) override;
	
	UPROPERTY()
	FIKRetargetScaleGoalsOpSettings Settings;

#if WITH_EDITOR
public:
	virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;
	virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif

	void RegenerateChainSettings(const FIKRetargetOpBase* InParentOp);

private:

	TArray<FIKChainToScale> ChainsToScale;
};

/* The blueprint/python API for editing a Scale Goals Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetScaleGoalsController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetScaleGoalsOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	FIKRetargetScaleGoalsOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetScaleGoalsOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	void SetSettings(FIKRetargetScaleGoalsOpSettings InSettings);
};

#undef UE_API
#undef LOCTEXT_NAMESPACE
