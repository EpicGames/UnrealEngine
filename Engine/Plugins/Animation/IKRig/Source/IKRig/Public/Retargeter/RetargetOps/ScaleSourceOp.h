// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Animation/BoneReference.h"

#include "ScaleSourceOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "ScaleSourceOp"

UENUM(BlueprintType)
enum class EScaleSourcePivot : uint8
{
	ComponentOrigin		UMETA(DisplayName = "Component Origin"),
	Bone				UMETA(DisplayName = "Bone"),
};

USTRUCT(BlueprintType, meta = (DisplayName = "Scale Source Settings"))
struct FIKRetargetScaleSourceOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	/** Range 0.01 to +inf. Default 1. Scales the incoming source pose. Affects entire skeleton and all IK goals.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ReinitializeOnEdit, ClampMin = "0.01", UIMin = "0.01", UIMax = "10.0"))
	double SourceScaleFactor = 1.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings")
	EScaleSourcePivot ScalePivot = EScaleSourcePivot::ComponentOrigin;

	UPROPERTY(EditAnywhere, Category = "Op Settings", meta=(NotOverrideable))
	FBoneReference ScalePivotBone;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings")
	bool bProjectScalePivotToFloor = false;

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

#if WITH_EDITORONLY_DATA
	virtual USkeleton* GetSkeleton(const FName InPropertyName) override;
#endif
};

USTRUCT(BlueprintType, meta = (DisplayName = "Scale Source"))
struct FIKRetargetScaleSourceOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	// NOTE: this op does not do anything in Initialize() or Run().
	// It is a special case op that the retargeter reads from when it needs to scale the source pose.
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& Log) override;
	
	virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override {};

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;

	virtual bool IsSingleton() const override { return true; };

	int32 GetScalePivotBoneIndex(const FRetargetSkeleton& InSourceSkeleton) const;

	UPROPERTY()
	FIKRetargetScaleSourceOpSettings Settings;

private:
	
	// caching the bone index to scale the source from
	mutable int32 CachedScalePivotBoneIndex = INDEX_NONE;
	mutable FName CachedScalePivotBoneName = NAME_None;
};

/* The blueprint/python API for editing a Scale Source Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetScaleSourceController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetScaleSourceOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetScaleSourceOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetScaleSourceOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetScaleSourceOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
