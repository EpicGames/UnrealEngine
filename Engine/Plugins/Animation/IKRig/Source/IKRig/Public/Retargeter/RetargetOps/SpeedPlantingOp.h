// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Kismet/KismetMathLibrary.h"
#include "Retargeter/IKRetargetOps.h"

#include "SpeedPlantingOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "SpeedPlantingOp"

struct FIKRigGoal;

USTRUCT(BlueprintType)
struct FRetargetSpeedPlantingSettings
{
	GENERATED_BODY()
	
	FRetargetSpeedPlantingSettings() = default;
	FRetargetSpeedPlantingSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {}

	/** The name of the target chain to plant the IK on.
	 * NOTE: this chain must have an IK Goal assigned to it! */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speed Planting", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;

	/** Range 0 to 1. Default 1. Blends the effect of the speed planting on this chain only. At 0, there is no effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Alpha = 1.0;

	/** The name of the curve on the source animation that contains the speed of the end effector bone.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintReadWrite, Category = "Chain Settings", meta = (ClampMin = "0.0", ClampMax = "100.0", UIMin = "0.0", UIMax = "100.0"))
	FName SpeedCurveName;

	bool operator==(const FRetargetSpeedPlantingSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };
};

UENUM(BlueprintType)
enum class ESpeedPlantBlendMethod : uint8
{
	Spring,
	MoveTowards
};

USTRUCT(BlueprintType, meta = (DisplayName = "Speed Plant Settings"))
struct FIKRetargetSpeedPlantingOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FRetargetSpeedPlantingSettings> ChainsToSpeedPlant;

	/** Range 0 to 1. Default 1. Blends the effect of the speed planting. At 0, there is no effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Alpha = 1.0;
	
	/** Range 0 to 100. Default 15. The maximum speed a source bone can be moving while being considered 'planted'.
	*  The target IK goal will not be allowed to move whenever the source bone speed drops below this threshold speed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "100.0"))
	double SpeedThreshold = 15.0f;

	/** The method to use when blending the IK goal in/out of a planted position.
	 * The Spring method uses a critically damped spring equation to blend into and out of the planted states.
	 * The Move Towards method can more accurately track fast motion without looking mushy. It only blends out of planted states. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings")
	ESpeedPlantBlendMethod BlendMethod = ESpeedPlantBlendMethod::Spring;

	/** The maximum speed in cm/s that the goal will move to catch up to the animation after unplanting. This blended towards the animated velocity
	 * as the goal catches up to animated location after unplanting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "2000.0", 
		EditCondition = "BlendMethod == ESpeedPlantBlendMethod::MoveTowards", EditConditionHides))
	double MaxBlendSpeed = 900.0;

	/** Seconds to reach MaxBlendSpeed. Determines how rapidly the goal accelerates after unplanting. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.001", UIMin = "0.01", UIMax = "1.0", 
		EditCondition = "BlendMethod == ESpeedPlantBlendMethod::MoveTowards", EditConditionHides))
	double TimeToMaxSpeed = 0.2;

	/** Controls how aggressively the goal velocity matches the animated velocity as it arrives at the animated location after unplanting.
	 * This allows the goal speed to gradually blend towards the target speed as the goal location approaches the animated location.
	 * Reduce this to remove velocity discontinuities when catching up to the animation.
	 * 1.0 is very soft, 20.0 is very sharp. Default is 10.0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.001", UIMin = "0.01", UIMax = "20.0", 
		EditCondition = "BlendMethod == ESpeedPlantBlendMethod::MoveTowards", EditConditionHides))
	double ArrivalSpeedGain = 10.0f;
	
	/**
	* The spring strength determines how hard it will pull towards the target when planting and unplanting.
	* The value is the frequency (hz) at which it will oscillate when there is no damping. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta=(Input, ClampMin = "0.0", UIMin = "0.0", UIMax = "30.0",
		EditCondition = "BlendMethod == ESpeedPlantBlendMethod::Spring", EditConditionHides))
	double SpringStrength = 3.0f;
	// Deprecated in 5.8, replaced with higher-level "Strength" param
	UPROPERTY(meta=(DeprecatedProperty))
	double Stiffness = 250.0f;

	/* Adjust size of goal debug drawing in viewport. Red is planted, green is unplanted. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug", meta = (ClampMin = "0.0", UIMin = "0.0", UIMax = "10.0"))
	double DebugDrawSize = 5.0;

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	/** Custom override to load old "Stiffness" as new "Strength" */
	virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) override;
};

struct FSpeedPlantingGoalState
{
	FSpeedPlantingGoalState(
		const FName InGoalName,
		const FRetargetSpeedPlantingSettings* InSettings) :
		GoalName(InGoalName),
		Settings(InSettings) {};

	void Update(
		FIKRigGoal* IKRigGoal,
		const double InDeltaTime,
		const FIKRetargetSpeedPlantingOpSettings& InSettings);

	void Reset();
	
	FName GoalName;
	const FRetargetSpeedPlantingSettings* Settings;
	FVectorSpringState PositionSpringState;
	FVector CurrentPosition;
	FVector TargetPosition;
	FVector LastTargetPosition;
	double CurrentSpeedValue = -1.0f;
	bool bIsPlanted = false;
	bool bWasReset = true;
	FVector PlantLocation;
	double AccelerationTimer = 0.0;
	bool bIsTrackingTarget = true;

#if WITH_EDITOR
	bool bFoundCurveInSourceComponent = false;
	bool bFoundCurveInTargetComponent = false;
#endif
};

USTRUCT(BlueprintType, meta = (DisplayName = "Speed Plant IK Goals"))
struct FIKRetargetSpeedPlantingOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;

	UE_API virtual void OnPlaybackReset() override;

	UE_API virtual void AnimGraphPreUpdateMainThread(USkeletalMeshComponent& SourceMeshComponent, USkeletalMeshComponent& TargetMeshComponent) override;

	UE_API virtual void AnimGraphEvaluateAnyThread(FPoseContext& Output) override;

	UE_API virtual const UScriptStruct* GetParentOpType() const override;

	UE_API virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;

	UE_API virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) override;

	/** return array of names of all the speed curves referenced by this op. */
	UE_API TArray<FName> GetRequiredSpeedCurves() const;

	UPROPERTY()
	FIKRetargetSpeedPlantingOpSettings Settings;

#if WITH_EDITOR
public:
	UE_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;
	virtual bool HasDebugDrawing() override { return true; };
	UE_API virtual FText GetWarningMessage() const override;
	UE_API virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	UE_API virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	UE_API virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif

private:
	
	void RegenerateChainSettings(const FIKRetargetOpBase* InParentOp);
	
	TArray<FSpeedPlantingGoalState> GoalsToPlant;
};

/* The blueprint/python API for editing a Speed Planting Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetSpeedPlantingController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetSpeedPlantingOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetSpeedPlantingOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetSpeedPlantingOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetSpeedPlantingOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
