// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetChainMapping.h"
#include "Retargeter/IKRetargetOps.h"

#include "StretchChainOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "StretchChainOp"

struct FResolvedBoneChain;
struct FRetargetStretchChainSettings;

struct FChainStretcher
{
	FChainStretcher(
		const FRetargetStretchChainSettings* InSettings,
		const FResolvedBoneChain& InSourceBoneChain,
		const FResolvedBoneChain& InTargetBoneChain);
	
	void Initialize(const FRetargetSkeleton& InTargetSkeleton);

	void StretchChain(
		const TArray<FTransform> &InSourceGlobalPose,
		const FRetargetSkeleton& InTargetSkeleton,
		TArray<FTransform> &OutTargetGlobalPose);
	
private:

	bool bIsInitialized = false;
	TArray<int32> AllChildrenOfChain;
	const FRetargetStretchChainSettings* Settings;
	const FResolvedBoneChain* SourceBoneChain;
	const FResolvedBoneChain* TargetBoneChain;
};

USTRUCT(BlueprintType)
struct FRetargetStretchChainSettings
{
	GENERATED_BODY()

	FRetargetStretchChainSettings() = default;
	FRetargetStretchChainSettings(FName InTargetChainName) : TargetChainName(InTargetChainName) {}
	
	/** The name of the target chain to stretch. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;

	/** Enable/disable stretching. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings")
	bool bEnabled = false;

	/** Range 0 to 1. Default 0. Matches the total length of this chain with the mapped source chain.
	*  At 0, the chain's length will be left alone
	*  At 1, the chain will be stretched to match the length of the source.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (EditCondition = "bEnabled", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	double MatchSourceLength = 0.0f;

	/** Range 0 to inf. Default 1. A multiplier on the length of the chain, applied after matching the source length. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (EditCondition = "bEnabled", UIMin = "0.0", UIMax = "5.0", ClampMin = "0.0"))
	double ScaleChainLength = 1.0f;
	
	bool operator==(const FRetargetStretchChainSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };
};

USTRUCT(BlueprintType, meta = (DisplayName = "Stretch Chain Settings"))
struct FIKRetargetStretchChainOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	/** The target IK Rig asset that contains the bones chains this op will modify. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "IK Rig Asset", meta=(ReinitializeOnEdit))
	TObjectPtr<const UIKRigDefinition> IKRigAsset = nullptr;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(TitleProperty="TargetChainName"))
	TArray<FRetargetStretchChainSettings> ChainsToStretch;

	virtual const UClass* GetControllerType() const override;
	
	virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Stretch Chains"))
struct FIKRetargetStretchChainOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;

	UE_API virtual const UScriptStruct* GetType() const override;
	
	UE_API virtual const UIKRigDefinition* GetCustomTargetIKRig() const override;

	UE_API virtual FRetargetChainMapping* GetChainMapping() override;

	UE_API virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;
	
	UE_API virtual void OnReinitPropertyEdited(const FPropertyChangedEvent* InPropertyChangedEvent) override;

#if WITH_EDITOR
	UE_API virtual FText GetWarningMessage() const override;
	UE_API virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	UE_API virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	UE_API virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif

	UPROPERTY()
	FIKRetargetStretchChainOpSettings Settings;

	/* This op maintains its own chain mapping table to allow per-op mapping */
	UPROPERTY()
	FRetargetChainMapping ChainMapping;

private:
	
	void ApplyIKRigs(const UIKRigDefinition* InSourceIKRig, const UIKRigDefinition* InTargetIKRig);
	
	TArray<FChainStretcher> ChainStretchers;
};

/* The blueprint/python API for editing an Align Pole Vector Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetStretchChainController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetStretchChainOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetStretchChainOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetStretchChainOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetStretchChainOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
