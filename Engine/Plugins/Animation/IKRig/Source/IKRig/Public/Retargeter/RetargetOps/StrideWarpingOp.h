// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetSettings.h"
#include "StrideWarpingOp.generated.h"

#define UE_API IKRIG_API

#define LOCTEXT_NAMESPACE "StrideWarpingOp"

struct FIKRigGoal;

UENUM()
enum class EWarpingDirectionSource
{
	Goals,
	Chain,
	RootBone
};

USTRUCT(BlueprintType)
struct FRetargetStrideWarpChainSettings
{
	GENERATED_BODY()

	FRetargetStrideWarpChainSettings() = default;
	FRetargetStrideWarpChainSettings(const FName InTargetChainName) : TargetChainName(InTargetChainName) {}

	/** The name of the TARGET chain with an IK goal to warp. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Setup", meta=(ReinitializeOnEdit, NotOverrideable))
	FName TargetChainName;
	
	/** Whether to warp the location of the IK goal on this chain. Default is true. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta=(ReinitializeOnEdit))
	bool EnableStrideWarping = true;

	/** Range 0 to 1. Default 1. Blends the effect of the stride warping on this chain only. At 0, there is no effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chain Settings", meta = (EditCondition = "EnableStrideWarping", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Alpha = 1.0;

	UE_API bool operator==(const FRetargetStrideWarpChainSettings& Other) const;
	FName GetName() const { return TargetChainName; };
	void SetName(const FName InName) { TargetChainName = InName; };
};

USTRUCT(BlueprintType, meta = (DisplayName = "Stride Warp Settings"))
struct FIKRetargetStrideWarpingOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	/** The chains to apply stride warping to */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Chains, meta=(ReinitializeOnEdit, TitleProperty="TargetChainName"))
	TArray<FRetargetStrideWarpChainSettings> ChainSettings;

	/** Range 0 to 1. Default 1. Blends the effect of all stride warping on/off. At 0, there is no effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Alpha = 1.0;

	/** Defines the source used to determine the forward direction as the character animates in world space. Default is "Goals".
	 * This method uses various positions on the character to create a "best fit" global rotation that approximates the facing direction of the character over time.
	 * This global rotation is used to define the forward and sideways directions used when warping goals along those axes.
	 * The options are:
	 * Goals: uses the positions of the IK goals to approximate the facing direction. This is best used on characters with a vertical spine, like bipeds.
	 * Chain: uses the positions of the bones in a retarget chain to approximate the facing direction. This is best when used with the spine chain for characters with a horizontal spine, like quadrupeds.
	 * Root Bone: uses the rotation of the root bone of the skeleton. This is most robust, but character must have correct root motion with yaw rotation in movement direction.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings")
	EWarpingDirectionSource DirectionSource = EWarpingDirectionSource::Goals;
	
	/** The world space axis that represents the forward facing direction for your character. By default, in Unreal, this is +Y.  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings")
	EBasicAxis ForwardDirection = EBasicAxis::Y;

	/** When using the "Chain" option as a Direction Source, this defines the chain to use to determine the facing direction of the character.
	 * Typically this would be the chain that contains the Spine bones. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta=(EditCondition="DirectionSource == EWarpingDirectionSource::Chain"))
	FName DirectionChain;

	/** Range 0 to Inf. Default 1. Warps IK goal positions in the forward direction. Useful for stride warping.
	 * Values below 1 will create smaller, squashed strides. Values greater than 1 will create stretched, longer strides.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (UIMin = "0.0", UIMax = "5.0", ClampMin = "0.0"))
	double WarpForwards = 1.0f;

	/** Range -+Inf. Default is 0. A static offset in world units to move the IK goals perpendicular to the forward direction.
	 * Values less than zero will move the goals towards the center-line of the character. Values greater than zero push the goals outwards.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (UIMin = "-10.0", UIMax = "100.0"))
	double SidewaysOffset = 0.0f;

	/** Range 0 to +Inf. Default is 1.0f.
	 * Values below 1 pull all the goals towards the average of all the goals (towards each other).
	 * Values greater than 1 push the goals apart.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (UIMin = "0.0", UIMax = "2.0", ClampMin = "0.0"))
	double WarpSplay = 1.0f;

	/** Adjust the size of the debug drawing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	double DebugDrawSize = 5.0;
	
	/** Adjust the thickness of the debug drawing */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Debug")
	double DebugDrawThickness = 5.0;

	UE_API virtual const UClass* GetControllerType() const override;

	UE_API virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;

	static UE_API FVector GetAxisVector(const EBasicAxis& Axis);

	UE_API bool operator==(const FIKRetargetStrideWarpingOpSettings& Other) const;

	UPROPERTY(meta = (DeprecatedProperty))
	bool bEnableDebugDraw_DEPRECATED = true;

	virtual void PostLoad(const FIKRigObjectVersion::Type InVersion) override;
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FIKRetargetStrideWarpingOpSettings> : public TStructOpsTypeTraitsBase2<FIKRetargetStrideWarpingOpSettings>
{
	enum { WithSerializer = true };
};

struct FStrideWarpGoalData
{
	FStrideWarpGoalData(
		const FName InGoalName,
		const FTransform& InGlobalRefPoseOfGoalBone,
		const FRetargetStrideWarpChainSettings& InChainSettings) :
		IKRigGoalName(InGoalName),
		GlobalRefPoseOfGoalBone(InGlobalRefPoseOfGoalBone),
		Settings(&InChainSettings) {};
	
	FName IKRigGoalName;
	FTransform GlobalRefPoseOfGoalBone;
	const FRetargetStrideWarpChainSettings* Settings;
};

USTRUCT(BlueprintType, meta = (DisplayName = "Stride Warp IK Goals"))
struct FIKRetargetStrideWarpingOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;

	UE_API virtual void OnAddedToStack(const UIKRetargeter* InRetargetAsset, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual void OnAssignIKRig(const ERetargetSourceOrTarget SourceOrTarget, const UIKRigDefinition* InIKRig, const FIKRetargetOpBase* InParentOp) override;

	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;
	
	UE_API virtual const UScriptStruct* GetType() const override;
	
	UE_API virtual const UScriptStruct* GetParentOpType() const override;
	
	UE_API virtual void OnTargetChainRenamed(const FName InOldChainName, const FName InNewChainName) override;

	UE_API virtual void OnParentReinitPropertyEdited(const FIKRetargetOpBase& InParentOp, const FPropertyChangedEvent* InPropertyChangedEvent) override;
	
	UPROPERTY()
	FIKRetargetStrideWarpingOpSettings Settings;

#if WITH_EDITOR
	
	UE_API virtual void DebugDraw(
		FPrimitiveDrawInterface* InPDI,
		const FTransform& InSourceTransform,
		const FTransform& InComponentTransform,
		const double InComponentScale,
		const FIKRetargetDebugDrawState& InEditorState) const override;

	virtual bool HasDebugDrawing() override { return true; };
	
	FTransform DebugStrideWarpingFrame;
	static UE_API FCriticalSection DebugDataMutex;

	UE_API virtual void ResetChainSettingsToDefault(const FName& InChainName) override;
	UE_API virtual bool AreChainSettingsAtDefault(const FName& InChainName) override;
	UE_API virtual uint8* GetChainSettingsMemory(const FName InChainName) override;
#endif

	UE_API void RegenerateChainSettings(const FIKRetargetOpBase* InParentOp);
	
	TArray<FStrideWarpGoalData> GoalsToWarp;
	TObjectPtr<const UIKRigDefinition> TargetIKRig;
};

/* The blueprint/python API for editing a Stride Warping Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetStrideWarpingController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current op settings as a struct.
	 * @return FIKRetargetStrideWarpingOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetStrideWarpingOpSettings GetSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRetargetStrideWarpingOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetStrideWarpingOpSettings InSettings);
};

#undef LOCTEXT_NAMESPACE

#undef UE_API
