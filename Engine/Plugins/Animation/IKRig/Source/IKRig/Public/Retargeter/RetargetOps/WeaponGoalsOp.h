// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Animation/BoneReference.h"
#include "Retargeter/IKRetargetOps.h"
#include "WeaponGoalsOp.generated.h"

#define UE_API IKRIG_API

struct FResolvedBoneChain;
struct FIKRigGoalContainer;

USTRUCT(BlueprintType, meta = (DisplayName = "Weapon Op Settings"))
struct FIKRetargetWeaponGoalsOpSettings : public FIKRetargetOpSettingsBase
{
	GENERATED_BODY()

	FIKRetargetWeaponGoalsOpSettings();

	/* An ancillary weapon bone for the left hand.
	 * This will be scaled with Weapon Scale. Defaults to "weapon_l". */
	UPROPERTY(EditAnywhere, Category = "Bones", meta=(NotOverrideable))
	FBoneReference LeftWeaponBone;

	/* An ancillary weapon bone for the right hand.
	 * This will be scaled with Weapon Scale. Defaults to "weapon_r". */
	UPROPERTY(EditAnywhere, Category = "Bones", meta=(NotOverrideable))
	FBoneReference RightWeaponBone;

	/* An optional bone used to attach props to the left hand.
	 * This will be scaled with Weapon Scale. Defaults to "hand_attach_l". */
	UPROPERTY(EditAnywhere, Category = "Bones", meta=(NotOverrideable))
	FBoneReference LeftHandAttachBone;

	/* An optional bone used to attach props to the right hand.
	 * This will be scaled with Weapon Scale. Defaults to "hand_attach_r". */
	UPROPERTY(EditAnywhere, Category = "Bones", meta=(NotOverrideable))
	FBoneReference RightHandAttachBone;

	/* The bone skinned to the left hand at wrist location.
	 * This bone must have an associated IK Goal. Defaults to "hand_l". */
	UPROPERTY(EditAnywhere, Category = "Bones", meta=(NotOverrideable))
	FBoneReference LeftHandBone;

	/* The bone skinned to the right hand at wrist location.
	 * This bone must have an associated IK Goal. Defaults to "hand_r". */
	UPROPERTY(EditAnywhere, Category = "Bones", meta=(NotOverrideable))
	FBoneReference RightHandBone;

	/* A bone representing the location of the left hand IK position while holding a two-handed prop.
	 * Typically, a child of the IKHandGun. Defaults to "ik_hand_l". */
	UPROPERTY(EditAnywhere, Category = "Bones", DisplayName="IK Hand Left Bone", meta=(NotOverrideable))
	FBoneReference IKHandLeftBone;

	/* A bone representing the location of the right hand IK position while holding a two-handed prop.
	 * Typically, a child of the IKHandGun. Defaults to "ik_hand_r". */
	UPROPERTY(EditAnywhere, Category = "Bones", DisplayName="IK Hand Right Bone", meta=(NotOverrideable))
	FBoneReference IKHandRightBone;

	/* A bone located at the right hand, and parent of IKHandLeftBone and IKHandRightBone.
	 * This will be scaled with Weapon Scale. Defaults to "ik_hand_gun". */
	UPROPERTY(EditAnywhere, Category = "Bones", meta=(NotOverrideable))
	FBoneReference IKHandGun;

	/** Range 0 to 1. Default 1. Blends the effect of the op off. At 0, there is no effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Alpha = 1.0;
	
	/* When false, all weapon scaling is skipped and WeaponScale has no effect. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings")
	bool bEnableWeaponScale = true;
	
	/* Range 0 to inf. Default 1. Scales the IKHandGun bones, Left/Right Weapon bones and hand attach bones. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", meta = (ClampMin = "0.01", UIMin = "0.01", UIMax = "10.0"))
	double WeaponScale = 1.0f;

	/** When true, enables RetargetHandIKAlpha to blend the IK Gun hierarchy between left/right hands. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", DisplayName="Snap IK Hand Gun")
	bool bSnapIKHandGun = true;

	/* Blends the entire Hand Gun bone hierarchy location from the left hand (at 0.0) to the right hand (at 1.0). Default is 1.0 (right hand).
	 * NOTE: this propagates the translational offset to the children bones (IKHandLeftBone/IKHandRightBone). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", DisplayName="Retarget Hand IK Alpha", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double LeftRightHandIKAlpha = 1.0f;

	/* Blend the effect of the left hand IK Goal from it's input location (0.0) to the IKHandLeftBone location (1.0). Default is 1.0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", DisplayName="Left Hand IK Alpha", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double LeftHandIKAlpha = 1.0f;

	/* Blend the effect of the right hand IK Goal from it's input location (0.0) to the IKHandRightBone location (1.0). Default is 1.0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings", DisplayName="Right Hand IK Alpha", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	double RightHandIKAlpha = 1.0f;

	/* When true, will suppress warnings in the retarget editor output log about missing bones from this op. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Op Settings")
	bool bSuppressMissingBoneWarnings = false;

	virtual const UClass* GetControllerType() const override;

	virtual void CopySettingsAtRuntime(const FIKRetargetOpSettingsBase* InSettingsToCopyFrom) override;
};


UENUM()
enum class ERetargetWeaponBone : uint8
{
	LeftWeapon      UMETA(DisplayName = "Left Weapon"), // labels here used for warnings only, not exposed to UI
	RightWeapon     UMETA(DisplayName = "Right Weapon"),
	LeftHandAttach  UMETA(DisplayName = "Left Hand Attach"),
	RightHandAttach UMETA(DisplayName = "Right Hand Attach"),
	LeftHand        UMETA(DisplayName = "Left Hand"),
	RightHand       UMETA(DisplayName = "Right Hand"),
	IKHandLeft      UMETA(DisplayName = "IK Hand Left"),
	IKHandRight     UMETA(DisplayName = "IK Hand Right"),
	HandGun         UMETA(DisplayName = "Hand Gun"),
	Count           UMETA(Hidden)
};

USTRUCT(BlueprintType, meta = (DisplayName = "Weapon Goals"))
struct FIKRetargetWeaponGoalOp : public FIKRetargetOpBase
{
	GENERATED_BODY()
	
	UE_API virtual bool Initialize(
		const FIKRetargetProcessor& InProcessor,
		const FRetargetSkeleton& InSourceSkeleton,
		const FTargetSkeleton& InTargetSkeleton,
		const FIKRetargetOpBase* InParentOp,
		FIKRigLogger& InLog) override;
	
	UE_API virtual void Run(
		FIKRetargetProcessor& InProcessor,
		const double InDeltaTime,
		const TArray<FTransform>& InSourceGlobalPose,
		TArray<FTransform>& OutTargetGlobalPose) override;
	
	UE_API virtual FIKRetargetOpSettingsBase* GetSettings() override;
	
	UE_API virtual const UScriptStruct* GetSettingsType() const override;

	UE_API virtual const UScriptStruct* GetType() const override;

	UE_API virtual const UScriptStruct* GetParentOpType() const override;
	
	UPROPERTY()
	FIKRetargetWeaponGoalsOpSettings Settings;

private:
	int32 BoneIndices[(int32)ERetargetWeaponBone::Count];
	FName LeftHandGoalName = NAME_None;
	FName RightHandGoalName = NAME_None;
};

/* The blueprint/python API for editing a Weapon Goals Op */
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetWeaponGoalsOpController : public UIKRetargetOpControllerBase
{
	GENERATED_BODY()
	
public:
	/* Get the current op settings as a struct.
	 * @return FIKRetargetWeaponGoalsOpSettings struct with the current settings used by the op. */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API FIKRetargetWeaponGoalsOpSettings GetSettings();

	/* Set the op settings. Input is a custom struct type for this op.
	 * @param InSettings a FIKRetargetWeaponGoalsOpSettings struct containing all the settings to apply to this op */
	UFUNCTION(BlueprintCallable, Category = Settings)
	UE_API void SetSettings(FIKRetargetWeaponGoalsOpSettings InSettings);
};

#undef UE_API