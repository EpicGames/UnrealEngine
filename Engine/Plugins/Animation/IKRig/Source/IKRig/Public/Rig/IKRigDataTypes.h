// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneContainer.h"
#include "IKRigDefinition.h"

#include "IKRigDataTypes.generated.h"

#define UE_API IKRIG_API

struct FIKRigProcessor;

UENUM(BlueprintType)
enum class EIKRigGoalSpace : uint8
{
	Component		UMETA(DisplayName = "Component Space"),
	Additive		UMETA(DisplayName = "Additive"),
    World			UMETA(DisplayName = "World Space"),
};

UENUM(BlueprintType)
enum class EIKRigGoalTransformSource : uint8
{
	Manual			UMETA(DisplayName = "Manual Input"),
    Bone			UMETA(DisplayName = "Bone"),
    ActorComponent	UMETA(DisplayName = "Actor Component"),
};

USTRUCT(Blueprintable)
struct FIKRigGoal
{
	GENERATED_BODY()
	
	/** Name of the IK Goal. Must correspond to the name of a Goal in the target IKRig asset. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Goal)
	FName Name;
	
	/** Name of the bone associate with this goal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Goal)
	FName BoneName;

	/** Set the source of the transform data for the Goal.
	 * "Manual Input" uses the values provided by the blueprint node pin.
	 * "Bone" uses the transform of the bone provided by OptionalSourceBone.
	 * "Actor Component" uses the transform supplied by any Actor Components that implements the IIKGoalCreatorInterface*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Goal)
	EIKRigGoalTransformSource TransformSource = EIKRigGoalTransformSource::Manual;

	/** When TransformSource is set to "Bone" mode, the Position and Rotation will be driven by this Bone's input transform.
	 * When using a Bone as the transform source, the Position and Rotation Alpha values can still be set as desired.
	 * But the PositionSpace and RotationSpace are no longer relevant and will not be used.*/
	UPROPERTY(EditAnywhere, Category = Goal)
	FBoneReference SourceBone;
	
	// UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Goal, meta = (CustomWidget = "BoneName"))
	// FName OptionalSourceBone;
	
	/** Position of the IK goal in Component Space of target actor component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Position, meta=(EditCondition="Source == EIKRigGoalSource::Manual"))
	FVector Position = FVector::ZeroVector;

	/** Rotation of the IK goal in Component Space of target actor component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rotation, meta=(EditCondition="Source == EIKRigGoalSource::Manual"))
	FRotator Rotation = FRotator::ZeroRotator;

	/** Range 0-1, default is 1.0. Smoothly blends the Goal position from the input pose (0.0) to the Goal position (1.0). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Position, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositionAlpha = 1.0f;

	/** Range 0-1, default is 1.0. Smoothly blends the Goal rotation from the input pose (0.0) to the Goal rotation (1.0). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rotation, meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RotationAlpha = 1.0f;

	/** The space that the goal position is in.
	 *"Additive" treats the goal transform as an additive offset relative to the Bone at the effector.
	 *"Component" treats the goal transform as being in the space of the Skeletal Mesh Actor Component.
	 *"World" treats the goal transform as being in the space of the World. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Position)
	EIKRigGoalSpace PositionSpace = EIKRigGoalSpace::Additive;
	
	/** The space that the goal rotation is in.
	*"Additive" treats the goal transform as an additive offset relative to the Bone at the effector.
	*"Component" treats the goal transform as being in the space of the Skeletal Mesh Actor Component.
	*"World" treats the goal transform as being in the space of the World. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Rotation)
	EIKRigGoalSpace RotationSpace = EIKRigGoalSpace::Additive;
	
	UPROPERTY(Transient)
	FVector FinalBlendedPosition = FVector::ZeroVector;

	UPROPERTY(Transient)
	FQuat FinalBlendedRotation = FQuat::Identity;

	/** If true, the goal is active and can be used by solvers in the rig.
	 * NOTE: Toggling this on or off at runtime will trigger a re-initialization. Prefer Alpha if possible. */
	UPROPERTY(Transient)
	bool bEnabled = true;

	FIKRigGoal() = default;
	
	FIKRigGoal(const FName& InGoalName, const FName& InBoneName) : Name(InGoalName), BoneName(InBoneName) {}

	FIKRigGoal(
        const FName& InName,
        const FName& InBoneName,
        const FVector& InPosition,
        const FQuat& InRotation,
        const float InPositionAlpha,
        const float InRotationAlpha,
        EIKRigGoalSpace InPositionSpace,
        EIKRigGoalSpace InRotationSpace,
        bool bInEnabled)
        : Name(InName),
		BoneName(InBoneName),
        Position(InPosition),
        Rotation(InRotation.Rotator()),
        PositionAlpha(InPositionAlpha),
        RotationAlpha(InRotationAlpha),
		PositionSpace(InPositionSpace),
		RotationSpace(InRotationSpace),
		FinalBlendedPosition(InPosition),
		FinalBlendedRotation(InRotation),
		bEnabled(bInEnabled) {}

	FIKRigGoal(const UIKRigEffectorGoal* InGoal)
		: Name(InGoal->GoalName),
		BoneName(InGoal->BoneName),
		Position(InGoal->CurrentTransform.GetTranslation()),
		Rotation(InGoal->CurrentTransform.Rotator()),
		PositionAlpha(InGoal->PositionAlpha),
        RotationAlpha(InGoal->RotationAlpha),
		PositionSpace(EIKRigGoalSpace::Component),
        RotationSpace(EIKRigGoalSpace::Component),
        FinalBlendedPosition(ForceInitToZero),
        FinalBlendedRotation(FQuat::Identity) {}

	FString ToString() const
	{
		return FString::Printf(TEXT("Name=%s, Pos=(%s, Alpha=%3.3f), Rot=(%s, Alpha=%3.3f)"),
			*Name.ToString(), *FinalBlendedPosition.ToString(), PositionAlpha, *FinalBlendedRotation.ToString(), RotationAlpha);
	}
};

inline uint32 GetTypeHash(FIKRigGoal ObjectRef) { return GetTypeHash(ObjectRef.Name); }

USTRUCT()
struct FIKRigGoalContainer
{
	GENERATED_BODY()

public:

	/** Set an IK goal to go to a specific location and rotation (in the specified space) blended by alpha.
	 * Will ADD the goal if none exist with the input name. */
	UE_API void SetIKGoal(const FIKRigGoal& InGoal);

	/** Set an IK goal to go to a specific location and rotation (in the specified space) blended by alpha.
	* Will ADD the goal if none exist with the input name. */
	UE_API void SetIKGoal(const UIKRigEffectorGoal* InEffectorGoal);

	/** Get an IK goal with the given name. Returns nullptr if no Goal is found in the container with the name. */
	UE_API const FIKRigGoal* FindGoalByName(const FName& GoalName) const;
	
	/** Get an IK goal with the given name. Returns nullptr if no Goal is found in the container with the name. */
	UE_API FIKRigGoal* FindGoalByName(const FName& GoalName);

	/** Clear out all Goals in container. */
	void Empty() { Goals.Empty(); bRigNeedsInitialized = true; };

	/** Returns true if there are no goals in the container */
	bool IsEmpty() const { return Goals.Num() <= 0; };

	/** Returns true if the container has a goal in it that triggered a reinitialization */
	bool NeedsInitialized() const {return bRigNeedsInitialized; };

	/** Get read-only access to array of Goals. */
	const TArray<FIKRigGoal>& GetGoalArray() const { return Goals; };
	TArray<FIKRigGoal>& GetGoalArray() { return Goals; };

	/** Fill this contain with all the goals in the input array */
	UE_API const void FillWithGoalArray(const TArray<UIKRigEffectorGoal*>& InGoals);
	
private:

	/** Set to true when:
	 * 1. a new goal is added
	 * 2. a goal has it's enabled flag toggled */
	bool bRigNeedsInitialized = true;

	UE_API FIKRigGoal* FindGoalWriteable(const FName& GoalName) const;
	
	/** Array of Goals. Cannot contain duplicates (name is key). */
	TArray<FIKRigGoal> Goals;

	friend FIKRigProcessor;
};

#undef UE_API
