// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoneContainer.h"
#include "UObject/Object.h"
#include "StructUtils/InstancedStruct.h"
#include "Interfaces/Interface_PreviewMeshProvider.h"
#include "IKRigSkeleton.h"
#include "Rig/Solvers/IKRigSolverBase.h"
#if WITH_EDITOR
#include "SAdvancedTransformInputBox.h"
#endif

#include "IKRigDefinition.generated.h"

class IPropertyHandle;
class UIKRigSolver;

#if WITH_EDITORONLY_DATA
UENUM()
enum class EIKRigGoalPreviewMode : uint8
{
	Additive		UMETA(DisplayName = "Additive"),
	Absolute		UMETA(DisplayName = "Absolute"),
};

namespace EIKRigTransformType
{
	enum Type : int8
	{
		Current,
		Reference,
	};
}
#endif

UCLASS(MinimalAPI, BlueprintType)
class UIKRigEffectorGoal : public UObject
{
	GENERATED_BODY()

public:

	/** The name used to refer to this goal from outside systems.
	 * This is the name to use when referring to this Goal from Blueprint, Anim Graph, Control Rig or IK Retargeter.*/
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Goal Settings")
	FName GoalName;

	/** The name of the bone that this Goal is located at.*/
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Goal Settings")
	FName BoneName;

	/** Range 0-1, default is 1. Blend between the input bone position (0.0) and the current goal position (1.0).*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Goal Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositionAlpha = 1.0f;
	
	/** Range 0-1, default is 1. Blend between the input bone rotation (0.0) and the current goal rotation (1.0).*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Goal Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RotationAlpha = 1.0f;

	/** The current transform of this Goal, in the Global Space of the character.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Goal Settings")
	FTransform CurrentTransform;

	/** The initial transform of this Goal, as defined by the initial transform of the Goal's bone in the retarget pose.*/
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Goal Settings")
	FTransform InitialTransform;

	bool operator==(const UIKRigEffectorGoal& Other) const { return GoalName == Other.GoalName; }

#if WITH_EDITORONLY_DATA

	/** Effects how this Goal transform is previewed in the IK Rig editor.
	* "Additive" interprets the Goal transform as being relative to the input pose. Useful for previewing animations.
	* "Absolute" pins the Goal transform to the Gizmo in the viewport.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Goal Settings")
	EIKRigGoalPreviewMode PreviewMode = EIKRigGoalPreviewMode::Additive;
	
	/**The size of the Goal gizmo drawing in the editor viewport.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Viewport Goal Settings", meta = (ClampMin = "0.1", ClampMax = "1000.0", UIMin = "0.1", UIMax = "100.0"))
	float SizeMultiplier = 1.0f;

	/**The thickness of the Goal gizmo drawing in the editor viewport.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Viewport Goal Settings",  meta = (ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "5.0"))
	float ThicknessMultiplier = 0.7f;

	/** Should position data be exposed in Blueprint */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Exposure")
	bool bExposePosition = false;

	/** Should rotation data be exposed in Blueprint */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Exposure")
	bool bExposeRotation = false;
	
	virtual void PostLoad() override
	{
		Super::PostLoad();
		SetFlags(RF_Transactional);
	}
	
#endif

#if WITH_EDITOR

	IKRIG_API TOptional<FTransform::FReal> GetNumericValue(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		EIKRigTransformType::Type TransformType
	) const;

	IKRIG_API TTuple<FTransform, FTransform> PrepareNumericValueChanged(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FTransform::FReal Value,
		EIKRigTransformType::Type TransformType
	) const;

	IKRIG_API void SetTransform( const FTransform& InTransform, EIKRigTransformType::Type InTransformType);

	IKRIG_API void OnCopyToClipboard(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType) const;
	IKRIG_API void OnPasteFromClipboard(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType);

	IKRIG_API bool TransformDiffersFromDefault(ESlateTransformComponent::Type Component, TSharedPtr<IPropertyHandle> PropertyHandle) const;
	IKRIG_API void ResetTransformToDefault(ESlateTransformComponent::Type Component, TSharedPtr<IPropertyHandle> PropertyHandle);
	
#endif
};

USTRUCT(BlueprintType)
struct FBoneChain
{
	GENERATED_BODY()

	FBoneChain() = default;
	
	FBoneChain(
		FName InName, 
		const FName InStartBone,
		const FName InEndBone,
		const FName InGoalName = NAME_None)
		: ChainName(InName)
		, StartBone(InStartBone)
		, EndBone(InEndBone)
		, IKGoalName(InGoalName)
	{}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = BoneChain)
	FName ChainName;

	UPROPERTY(VisibleAnywhere, Category = BoneChain)
	FBoneReference StartBone;

	UPROPERTY(VisibleAnywhere, Category = BoneChain)
	FBoneReference EndBone;
	
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = BoneChain)
	FName IKGoalName;
};

USTRUCT(BlueprintType)
struct FRetargetDefinition
{
	GENERATED_BODY()

	// The bone that contains root motion, located at the origin
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=RetargetRoot)
	FName RootBone;

	// The bone located at the hips or pelvis area, that translates the whole character
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=RetargetPelvis)
	FName PelvisBone;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=Chains)
	TArray<FBoneChain> BoneChains;

	// add a bone chain from start bone to end bone and store it in this retarget definition
	IKRIG_API void AddBoneChain(
		const FName& ChainName,
		const FName& StartBone,
		const FName& EndBone,
		const FName& GoalName = NAME_None);
	
	IKRIG_API FBoneChain* GetEditableBoneChainByName(FName ChainName);

	// post load patch old data
	IKRIG_API bool Serialize(FArchive& Ar);
	IKRIG_API void PostSerialize(const FArchive& Ar);
};

// load-patch FRetargetDefinition
template<>
struct TStructOpsTypeTraits<FRetargetDefinition> : public TStructOpsTypeTraitsBase2<FRetargetDefinition>
{
	enum
	{
		WithPostSerialize = true,
		WithSerializer = true,
	};
};

UCLASS(MinimalAPI, BlueprintType)
class UIKRigDefinition : public UObject, public IInterface_PreviewMeshProvider
{
	GENERATED_BODY()

public:
	
	/** The skeletal mesh to run the IK solve on (loaded into viewport).
	* NOTE: you can assign ANY Skeletal Mesh to apply the IK Rig to. Compatibility is determined when a new mesh is assigned
	* by comparing it's hierarchy with the goals, solvers and bone settings required by the rig. See output log for details. */
	UPROPERTY(AssetRegistrySearchable, EditAnywhere, Category = PreviewMesh)
	TSoftObjectPtr<USkeletalMesh> PreviewSkeletalMesh;

	/** runtime, read-only access to skeleton data */
	const FIKRigSkeleton& GetSkeleton() const { return Skeleton; };
	
	/** runtime, read-only access to array of Goals, all modifications must go through UIKRigController */
	const TArray<UIKRigEffectorGoal*>& GetGoalArray() const { return Goals; };

	/** runtime, read-only access to array of Solvers structs, all modifications must go through UIKRigController */
	IKRIG_API const TArray<FInstancedStruct>& GetSolverStructs() const;

	/** runtime, read-only access to Retarget Definition, all modifications must go through UIKRigController */
	const TArray<FBoneChain>& GetRetargetChains() const { return RetargetDefinition.BoneChains; };

	/** runtime, read-only access to Pelvis, all modifications must go through UIKRigController */
	const FName& GetPelvis() const { return RetargetDefinition.PelvisBone; };

	/** runtime, read-only access to Root bone, all modifications must go through UIKRigController */
	const FName& GetRoot() const { return RetargetDefinition.RootBone; };

	/** runtime, read-only access to read a bone chain, all modifications must go through UIKRigController */
	IKRIG_API const FBoneChain* GetRetargetChainByName(FName ChainName) const;

	/** get a list of the retarget chains contained in this IK Rig */
	IKRIG_API TArray<FName> GetRetargetChainNames() const;

	/** UObject */
	IKRIG_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	IKRIG_API virtual void Serialize(FArchive& Ar) override;
	IKRIG_API virtual void PostLoad() override;
	/** END UObject */
	
	/** IInterface_PreviewMeshProvider interface */
	IKRIG_API virtual void SetPreviewMesh(USkeletalMesh* PreviewMesh, bool bMarkAsDirty = true) override;
	IKRIG_API virtual USkeletalMesh* GetPreviewMesh() const override;
	/** END IInterface_PreviewMeshProvider interface */

#if WITH_EDITOR
	/* Get name of Preview Mesh property */
	static IKRIG_API const FName GetPreviewMeshPropertyName();
#endif

#if WITH_EDITORONLY_DATA
	/**The size of the Bones in the editor viewport. This is set by callbacks from the viewport Character>Bones menu*/
	UPROPERTY()
	float BoneSize = 2.0f;

	/**Draw bones in the viewport.*/
	UPROPERTY(EditAnywhere, Category = "Viewport Goal Settings")
	bool DrawGoals = true;
	
	/**The size of the Goals in the editor viewport.*/
	UPROPERTY(EditAnywhere, Category = "Viewport Goal Settings", meta = (ClampMin = "0.01", UIMin = "0.1", UIMax = "100.0"))
	float GoalSize = 7.0f;

	/** The thickness of the Goals in the editor viewport.*/
	UPROPERTY(EditAnywhere, Category = "Viewport Goal Settings",  meta = (ClampMin = "0.01", UIMin = "0.0", UIMax = "10.0"))
	float GoalThickness = 0.7f;

	/** The controller responsible for managing this asset's data (all editor mutation goes through this) */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional )
	TObjectPtr<UObject> Controller;
#endif

private:
	
	/** hierarchy and bone-pose transforms */
	UPROPERTY()
	FIKRigSkeleton Skeleton;

	/** goals, used as effectors by solvers that support them */
	UPROPERTY()
	TArray<TObjectPtr<UIKRigEffectorGoal>> Goals;

	/** polymorphic stack of solvers, executed in serial fashion where output of prior solve is input to the next */
	UPROPERTY(meta = (ExcludeBaseStruct, BaseStruct = "/Script/IKRig.IKRigSolverBase"))
	TArray<FInstancedStruct> SolverStack;

	/** bone chains for IK Retargeter */
	UPROPERTY()
	FRetargetDefinition RetargetDefinition;

	friend class UIKRigController;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS 
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "The pre 5.6 stack of UObject based solvers. Use SolverStack instead."))
	TArray<TObjectPtr<UIKRigSolver>> Solvers_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS 
};
