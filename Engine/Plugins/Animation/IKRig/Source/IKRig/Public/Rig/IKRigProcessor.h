// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IKRigDataTypes.h"
#include "IKRigLogger.h"
#include "IKRigSkeleton.h"

#include "IKRigProcessor.generated.h"

#define UE_API IKRIG_API

class UIKRigDefinition;
struct FIKRigSolverBase;

USTRUCT()
struct FGoalBone
{
	GENERATED_BODY()
	
	FName BoneName;
	int32 BoneIndex;
	int32 OptSourceIndex = INDEX_NONE;
};

/** the runtime for an IKRig to convert an input pose into
*   a solved output pose given a set of IK Rig Goals:
*   
* 1. Create an FIKRigProcessor in your animation system (ie anim node)
* 2. Initialize() with an IKRigDefinition asset
* 3. each tick, call SetIKGoal() and SetInputPoseGlobal()
* 4. Call Solve()
* 5. Copy output transforms with GetOutputPoseGlobal()
* 
*/
struct FIKRigProcessor
{
	
public:
	
	/** setup a new processor to run the given IKRig asset
	 *  NOTE!! this function creates new UObjects and consequently MUST be called from the main thread!!
	 *  @param InRigAsset - the IK Rig defining the collection of solvers to execute and all the rig settings
	 *  @param SkeletalMesh - the skeletal mesh you want to solve the IK on
	 *  @param GoalContainer - an optional list of goals to initialize with, if empty it will use the goals from the IK Rig asset
	 */
	UE_API void Initialize(
		const UIKRigDefinition* InRigAsset,
		const USkeletalMesh* InSkeletalMesh,
		const FIKRigGoalContainer& InOptionalGoals);

	//
	// BEGIN UPDATE SEQUENCE FUNCTIONS
	//
	// This is the general sequence of function calls to run a typical IK solve:
	//
	
	/** Set all bone transforms in global space. This is the pose the IK solve will start from */
	UE_API void SetInputPoseGlobal(const TArray<FTransform>& InGlobalBoneTransforms);

	/** Set input pose bone transforms individually */
	UE_API void SetInputBoneGlobal(const int32& InBoneIndex, const FTransform& InGlobalBoneTransform);

	/** Optionally can be called before Solve() to use the reference pose as start pose */
	UE_API void SetInputPoseToRefPose();

	/** Assign an entire container of goals to the rig. */
	UE_API void ApplyGoalsFromOtherContainer(const FIKRigGoalContainer& InGoalContainer);

	/** Set a named IK goal to go to a specific location, rotation and space, blended by separate position/rotation alpha (0-1)*/
	UE_API void SetIKGoal(const FIKRigGoal& Goal);

	/** Set a named IK goal to go to a specific location, rotation and space, blended by separate position/rotation alpha (0-1)*/
	UE_API void SetIKGoal(const UIKRigEffectorGoal* Goal);

	/** Run entire stack of solvers.
	 * If any Goals were supplied in World Space, a valid WorldToComponent transform must be provided.  */
	UE_API void Solve(const FTransform& WorldToComponent = FTransform::Identity);

	/** Get the results after calling Solve() */
	UE_API void GetOutputPoseGlobal(TArray<FTransform>& OutputPoseGlobal) const;

	/** Reset all internal data structures. Will require re-initialization before solving again. */
	UE_API void Reset();

	//
	// END UPDATE SEQUENCE FUNCTIONS
	//

	/** Get access to the internal goal data (read only) */
	UE_API const FIKRigGoalContainer& GetGoalContainer() const;
	UE_API FIKRigGoalContainer& GetGoalContainer();
	/** Get the bone associated with a goal */
	UE_API const FGoalBone* GetGoalBone(const FName& GoalName) const;
	
	/** Get read/write access to the internal skeleton data */
	UE_API FIKRigSkeleton& GetSkeletonWriteable();
	/** Get read-only access to the internal skeleton data */
	UE_API const FIKRigSkeleton& GetSkeleton() const;

	/** Used to determine if the IK Rig asset is compatible with a given skeleton. */
	static UE_API bool IsIKRigCompatibleWithSkeleton(
		const UIKRigDefinition* InRigAsset,
		const FIKRigInputSkeleton& InputSkeleton,
		const FIKRigLogger* Log);

	/** return true if the processor is initialized with a rig and ready to run */
	bool IsInitialized() const { return bInitialized; };

	/** force the processor to reinitialize */
	UE_API void SetNeedsInitialized();

	/** Get read-only access to a currently running solver */
	UE_API const FIKRigSolverBase* GetSolver(const int32 InSolverIndex) const;
	
	/** Used to propagate setting values from the source asset at runtime (settings that do not require re-initialization)
	 * This allows the user to edit their IK Rig asset and see the effects in PIE. */
	UE_API void CopyAllSettingsFromAsset(const UIKRigDefinition* SourceAsset);

	/** logging system */
	FIKRigLogger Log;
	
private:

	/** Update the final pos/rot of all the goals based on their alpha values and their space settings. */
	UE_API void ResolveFinalGoalTransforms(const FTransform& WorldToComponent);

	/** the stack of solvers to run in order */
	TArray<FInstancedStruct> Solvers;

	/** the named transforms that solvers use as end effectors */
	FIKRigGoalContainer GoalContainer;

	/** map of goal names to bone names/indices */
	TMap<FName, FGoalBone> GoalBones;

	/** storage for hierarchy and bone transforms */
	FIKRigSkeleton Skeleton;

	/** store asset for dynamic reinitialization */
	TObjectPtr<const UIKRigDefinition> AssetInitializedWith;

	/** true if processor is currently initialized and ready to run */
	bool bInitialized = false;
	/** true if processor attempted an initialization but failed (prevents infinite reinit) */
	bool bTriedToInitialize = false;
};

//
// BEGIN DEPRECATED UOBJECT-BASED PROCESSOR
//
// NOTE: As of 5.6, IK Rig processors are no long UObjects.
// The UObject based processor is here for backwards compatibility.
// If you have a system built off this processor, please convert it to the FIKRigProcessor as this will be removed in a future update.
//

UCLASS()
class UE_DEPRECATED(5.6, "UIKRigProcessor has been replaced by FIKRigProcessor. UIKRigProcessor is now a thin wrapper around FIKRigProcessor and will be removed in a future version.")
IKRIG_API UIKRigProcessor : public UObject
{
	GENERATED_BODY()
	
	public:
	
	void Initialize(const UIKRigDefinition* InRigAsset, const USkeletalMesh* SkeletalMesh, const TArray<FName>& ExcludedGoals = TArray<FName>())
	{
		Processor.Initialize(InRigAsset, SkeletalMesh, FIKRigGoalContainer());
	};
	void SetInputPoseGlobal(const TArray<FTransform>& InGlobalBoneTransforms)
	{
		Processor.SetInputPoseGlobal(InGlobalBoneTransforms);
	};
	void SetInputPoseToRefPose()
	{
		Processor.SetInputPoseToRefPose();
	};
	void SetIKGoal(const FIKRigGoal& Goal)
	{
		Processor.SetIKGoal(Goal);
	};
	void SetIKGoal(const UIKRigEffectorGoal* Goal)
	{
		Processor.SetIKGoal(Goal);
	};
	void Solve(const FTransform& WorldToComponent = FTransform::Identity)
	{
		Processor.Solve(WorldToComponent);
	};
	void GetOutputPoseGlobal(TArray<FTransform>& OutputPoseGlobal) const
	{
		Processor.GetOutputPoseGlobal(OutputPoseGlobal);
	};
	void Reset()
	{
		Processor.Reset();
	};
	const FIKRigGoalContainer& GetGoalContainer() const
	{
		return Processor.GetGoalContainer();
	};
	const FGoalBone* GetGoalBone(const FName& GoalName) const
	{
		return Processor.GetGoalBone(GoalName);
	};
	FIKRigSkeleton& GetSkeletonWriteable()
	{
		return Processor.GetSkeletonWriteable();
	} ;
	const FIKRigSkeleton& GetSkeleton() const
	{
		return Processor.GetSkeleton();
	};
	static bool IsIKRigCompatibleWithSkeleton(const UIKRigDefinition* InRigAsset, const FIKRigInputSkeleton& InputSkeleton, const FIKRigLogger* Log)
	{
		return FIKRigProcessor::IsIKRigCompatibleWithSkeleton(InRigAsset, InputSkeleton, Log);
	};
	bool IsInitialized() const
	{
		return Processor.IsInitialized();
	};
	void SetNeedsInitialized()
	{
		Processor.SetNeedsInitialized();
	};
	void CopyAllSettingsFromAsset(const UIKRigDefinition* SourceAsset)
	{
		Processor.CopyAllSettingsFromAsset(SourceAsset);
	};

	/** NOTE: the deprecated logging system will no longer function. It's here to avoid compilation issues.*/
	FIKRigLogger Log;

private:
	FIKRigProcessor Processor;
};

#undef UE_API
