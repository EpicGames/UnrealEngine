// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IKRigSkeleton.generated.h"

#define UE_API IKRIG_API

struct FBoneChain;
class USkeletalMesh;
class UIKRigDefinition;

/** Data used just to initialize an IKRigSkeleton from outside systems
 *
 * The input skeleton may be different than the skeleton that the IK Rig asset was created for, within some limits.
 * 1. It must have all the Bones that the IK Rig asset referenced (must be a sub-set)
 * 2. All the bones must have the same parents (no change in hierarchy)
 *
 * You can however add additional bones, change the reference pose (including proportions) and the bone indices.
 * This allows you to run the same IK Rig asset on somewhat different skeletal meshes.
 *
 * To validate compatibility use UIKRigProcess::IsIKRigCompatibleWithSkeleton()
 */
USTRUCT()
struct FIKRigInputSkeleton
{
	GENERATED_BODY()
	
	TArray<FName> BoneNames;
	TArray<int32> ParentIndices;
	TArray<FTransform> LocalRefPose;
	const USkeletalMesh* SkeletalMesh;

	FIKRigInputSkeleton() = default;
	
	UE_API FIKRigInputSkeleton(const USkeletalMesh* InSkeletalMesh);

	UE_API void Initialize(const USkeletalMesh* InSkeletalMesh);

	UE_API void Reset();
};

USTRUCT()
struct FIKRigSkeleton
{
	GENERATED_BODY()

	/** The skeletal mesh this was initialized with. */
	UPROPERTY(VisibleAnywhere, Category = Skeleton)
	TObjectPtr<const USkeletalMesh> SkeletalMesh;
	
	/** Names of bones. Used to match hierarchy with runtime skeleton. */
	UPROPERTY(VisibleAnywhere, Category = Skeleton)
	TArray<FName> BoneNames;

	/** Same length as BoneNames, stores index of parent for each bone. */
	UPROPERTY(VisibleAnywhere, Category = Skeleton)
	TArray<int32> ParentIndices;

	/** Sparse array of bones that are to be excluded from any solvers (parented around, treated as FK children). */
	UPROPERTY(VisibleAnywhere, Category = Skeleton)
	TArray<FName> ExcludedBones;

	/** The current GLOBAL space pose of each bone. */
	UPROPERTY(VisibleAnywhere, Category = Skeleton)
	TArray<FTransform> CurrentPoseGlobal;

	/** The current LOCAL space pose of each bone. */
	UPROPERTY(VisibleAnywhere, Category = Skeleton)
	TArray<FTransform> CurrentPoseLocal;

	/** The initial/reference GLOBAL space pose of each bone. */
	UPROPERTY(VisibleAnywhere, Category = Skeleton)
	TArray<FTransform> RefPoseGlobal;

	UE_API void SetInputSkeleton(const USkeletalMesh* InSkeletalMesh, const TArray<FName>& InExcludedBones);

	UE_API void SetInputSkeleton(const FIKRigInputSkeleton& InputSkeleton, const TArray<FName>& InExcludedBones);

	UE_API void Reset();
	
	UE_API int32 GetBoneIndexFromName(const FName InName) const;

	UE_API const FName& GetBoneNameFromIndex(const int32 BoneIndex) const;
	
	UE_API int32 GetParentIndex(const int32 BoneIndex) const;

	UE_API int32 GetParentIndexThatIsNotExcluded(const int32 BoneIndex) const;

	UE_API int32 GetChildIndices(const int32 ParentBoneIndex, TArray<int32>& Children) const;

	UE_API void GetChildrenIndicesRecursive(const int32 BoneIndex, TArray<int32>& OutChildren) const;
	
	UE_API int32 GetCachedEndOfBranchIndex(const int32 InBoneIndex) const;

	static UE_API void ConvertLocalPoseToGlobal(
		const TArray<int32>& InParentIndices,
		const TArray<FTransform>& InLocalPose,
		TArray<FTransform>& OutGlobalPose);
	
	UE_API void UpdateAllGlobalTransformFromLocal();
	
	UE_API void UpdateAllLocalTransformFromGlobal();
	
	UE_API void UpdateGlobalTransformFromLocal(const int32 BoneIndex);
	
	UE_API void UpdateLocalTransformFromGlobal(const int32 BoneIndex);
	
	UE_API void PropagateGlobalPoseBelowBone(const int32 BoneIndex);

	UE_API bool IsBoneInDirectLineage(const FName& Child, const FName& PotentialParent) const;

	UE_API bool IsBoneExcluded(const int32 BoneIndex) const;

	static UE_API void NormalizeRotations(TArray<FTransform>& Transforms);

	UE_API void GetChainsInList(const TArray<int32>& SelectedBones, TArray<FBoneChain>& OutChains) const;

	// get indices of bones in this chain, ordered from tip to root. returns false if chain is invalid.
	UE_API bool ValidateChainAndGetBones(const FBoneChain& Chain, TArray<int32>& OutBoneIndices) const;

	// given a set of bones, returns a set of indices of bones that are mirrored across X axis (YZ plane)
	// return false if any of the bones do not have a mirrored pair within the MaxDistanceThreshold
	UE_API bool GetMirroredBoneIndices(const TArray<int32>& BoneIndices, TArray<int32>& OutMirroredIndices) const;

private:

	/** The branch bellow a specific bone. It stores the last element of the branch instead of the indices */
	mutable TArray<int32> CachedEndOfBranchIndices;
};

#undef UE_API
