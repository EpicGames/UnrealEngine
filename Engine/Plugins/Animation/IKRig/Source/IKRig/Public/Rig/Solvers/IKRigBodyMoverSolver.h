// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/IKRigDefinition.h"
#include "IKRigSolverBase.h"

#include "IKRigBodyMoverSolver.generated.h"

#define UE_API IKRIG_API

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.BodyMoverGoalSettingsWrapper"))
struct FIKRigBodyMoverGoalSettings : public FIKRigGoalSettingsBase
{
	GENERATED_BODY()

	// The bone associated with this goal.
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Goal Settings")
	FName BoneName;

	// Scale the influence this goal has on the body. Range is 0-10. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Goal Settings", meta = (ClampMin = "0", ClampMax = "10", UIMin = "0.0", UIMax = "10.0"))
	float InfluenceMultiplier = 1.0f;
};

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.BodyMoverSettingsWrapper"))
struct FIKRigBodyMoverSettings : public FIKRigSolverSettingsBase
{
	GENERATED_BODY()

	// The target bone to move with the effectors.
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Body Mover Settings")
	FName StartBone;

	// Blend the translational effect of this solver on/off. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float PositionAlpha = 1.0f;

	// Multiply the POSITIVE X translation. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float PositionPositiveX = 1.0f;

	// Multiply the NEGATIVE X translation. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float PositionNegativeX = 1.0f;

	// Multiply the POSITIVE Y translation. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float PositionPositiveY = 1.0f;

	// Multiply the NEGATIVE Y translation. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float PositionNegativeY = 1.0f;

	// Multiply the POSITIVE Z translation. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float PositionPositiveZ = 1.0f;

	//* Multiply the NEGATIVE Z translation. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float PositionNegativeZ = 1.0f;

	// Blend the total rotational effect on/off. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float RotationAlpha = 1.0f;

	// Blend the X-axis rotational effect on/off. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float RotateXAlpha = 1.0f;

	// Blend the Y-axis rotational effect on/off. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float RotateYAlpha = 1.0f;

	// Blend the Z-axis rotational effect on/off. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Body Mover Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	float RotateZAlpha = 1.0f;
};

USTRUCT(BlueprintType)
struct FIKRigBodyMoverSolver : public FIKRigSolverBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	FIKRigBodyMoverSettings Settings;
	
	UPROPERTY()
	TArray<FIKRigBodyMoverGoalSettings> AllGoalSettings;

	// FIKRigSolverBase interface
	UE_API virtual void Initialize(const FIKRigSkeleton& InIKRigSkeleton) override;
	UE_API virtual void Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals) override;
	UE_API virtual void GetRequiredBones(TSet<FName>& OutRequiredBones) const override;
	UE_API virtual void GetRequiredGoals(TSet<FName>& OutRequiredGoals) const override;

	// solver settings
	UE_API virtual FIKRigSolverSettingsBase* GetSolverSettings() override;
	UE_API virtual const UScriptStruct* GetSolverSettingsType() const override;
	
	// goals
	UE_API virtual void AddGoal(const UIKRigEffectorGoal& InNewGoal) override;
	UE_API virtual void OnGoalRemoved(const FName& InGoalName) override;
	UE_API virtual void OnGoalRenamed(const FName& InOldName, const FName& InNewName) override;
	UE_API virtual void OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName) override;

	// goal settings
	virtual bool UsesCustomGoalSettings() const override { return true; };
	UE_API virtual FIKRigGoalSettingsBase* GetGoalSettings(const FName& InGoalName) override;
	UE_API virtual const UScriptStruct* GetGoalSettingsType() const override;
	UE_API virtual void GetGoalsWithSettings(TSet<FName>& OutGoalsWithSettings) const override;
	
	// start bone can be set on this solver
	virtual bool UsesStartBone() const override { return true; };
	virtual FName GetStartBone() const override { return Settings.StartBone; };
	UE_API virtual void SetStartBone(const FName& InRootBoneName) override;

#if WITH_EDITOR
	// return custom controller for scripting this solver
	UE_API virtual UIKRigSolverControllerBase* GetSolverController(UObject* Outer) override;
	// UI stuff
	UE_API virtual FText GetNiceName() const override;
	UE_API virtual bool GetWarningMessage(FText& OutWarningMessage) const override;
	UE_API virtual bool IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const override;
#endif

private:

	UE_API int32 GetIndexOfGoal(const FName& InGoalName) const;

	int32 BodyBoneIndex;
};

/* The blueprint/python API for modifying a Body Mover solver's settings in an IK Rig.
 * Can adjust Solver and Goal settings. */
UCLASS(MinimalAPI, BlueprintType)
class UIKRigBodyMoverController : public UIKRigSolverControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current solver settings as a struct.
	 * @return FIKRigBodyMoverSettings struct with the current settings used by the solver. */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API FIKRigBodyMoverSettings GetSolverSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRigBodyMoverSettings struct containing all the settings to apply to this solver */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API void SetSolverSettings(FIKRigBodyMoverSettings InSettings);

	/* Get the settings for the specified goal.
	 * @param InGoalName the name of the goal to get settings for
	 * @return FIKRigBodyMoverGoalSettings struct (empty if the specified goal does not belong to this solver) */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API FIKRigBodyMoverGoalSettings GetGoalSettings(const FName InGoalName);

	/* Set the settings for the specified goal.
	 * @param InGoalName: the name of the goal to assign the settings to.
	 * @param InSettings: a custom struct type with all the settings for a goal */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API void SetGoalSettings(const FName InGoalName, FIKRigBodyMoverGoalSettings InSettings);
};

//
// BEGIN LEGACY TYPES
//

// NOTE: This type has been replaced with FBodyMoverGoalSettings.
UCLASS(MinimalAPI)
class UIKRig_BodyMoverEffector : public UObject
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	FName GoalName;
	UPROPERTY()
	FName BoneName;
	UPROPERTY()
	float InfluenceMultiplier = 1.0f;
};

// NOTE: This type has been replaced with FBodyMoverSolver.
UCLASS(MinimalAPI)
class UIKRig_BodyMover : public UIKRigSolver
{
	GENERATED_BODY()
	
public:

	// upgrade patch to new UStruct-based solver
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) override
	{
		OutInstancedStruct.InitializeAs(FIKRigBodyMoverSolver::StaticStruct());
		FIKRigBodyMoverSolver& NewSolver = OutInstancedStruct.GetMutable<FIKRigBodyMoverSolver>();

		// load solver settings
		NewSolver.SetEnabled(IsEnabled());
		NewSolver.Settings.StartBone = RootBone;
		NewSolver.Settings.PositionAlpha = PositionAlpha;
		NewSolver.Settings.PositionPositiveX = PositionPositiveX;
		NewSolver.Settings.PositionNegativeX = PositionNegativeX;
		NewSolver.Settings.PositionPositiveY = PositionPositiveY;
		NewSolver.Settings.PositionNegativeY = PositionNegativeY;
		NewSolver.Settings.PositionPositiveZ = PositionPositiveZ;
		NewSolver.Settings.PositionNegativeZ = PositionNegativeZ;
		NewSolver.Settings.RotationAlpha = RotationAlpha;
		NewSolver.Settings.RotateXAlpha = RotateXAlpha;
		NewSolver.Settings.RotateYAlpha = RotateYAlpha;
		NewSolver.Settings.RotateZAlpha = RotateZAlpha;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// load goal settings
		for (const TObjectPtr<UIKRig_BodyMoverEffector>& Effector : Effectors_DEPRECATED)
		{
			FIKRigBodyMoverGoalSettings NewSettings;
			NewSettings.BoneName = Effector->BoneName;
			NewSettings.Goal = Effector->GoalName;
			NewSettings.InfluenceMultiplier = Effector->InfluenceMultiplier;
			NewSolver.AllGoalSettings.Add(NewSettings);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
	
	UPROPERTY()
	FName RootBone;
	UPROPERTY()
	float PositionAlpha = 1.0f;
	UPROPERTY()
	float PositionPositiveX = 1.0f;
	UPROPERTY()
	float PositionNegativeX = 1.0f;
	UPROPERTY()
	float PositionPositiveY = 1.0f;
	UPROPERTY()
	float PositionNegativeY = 1.0f;
	UPROPERTY()
	float PositionPositiveZ = 1.0f;
	UPROPERTY()
	float PositionNegativeZ = 1.0f;
	UPROPERTY()
	float RotationAlpha = 1.0f;
	UPROPERTY()
	float RotateXAlpha = 1.0f;
	UPROPERTY()
	float RotateYAlpha = 1.0f;
	UPROPERTY()
	float RotateZAlpha = 1.0f;

	UE_DEPRECATED(5.6, "IK Rig refactored to not use UObjects")
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TObjectPtr<UIKRig_BodyMoverEffector>> Effectors_DEPRECATED;
};

//
// END LEGACY TYPES
//

#undef UE_API
