// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/IKRigDefinition.h"
#include "IKRigSolverBase.h"
#include "Core/PBIKSolver.h"
#include "PBIK_Shared.h"

#include "IKRigFullBodyIK.generated.h"

#define UE_API IKRIG_API

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.FBIKGoalSettingsWrapper"))
struct FIKRigFBIKGoalSettings : public FIKRigGoalSettingsBase
{
	GENERATED_BODY()

	FIKRigFBIKGoalSettings() = default;
	FIKRigFBIKGoalSettings(FName InGoalName, FName InBoneName) : BoneName(InBoneName) { Goal = InGoalName;}
	
	/** The bone that this effector will pull on. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Full Body IK Effector")
	FName BoneName;

	/** Range 0-inf (default is 0). Explicitly set the number of bones up the hierarchy to consider part of this effector's 'chain'.
	* The "chain" of bones is used to apply Preferred Angles, Pull Chain Alpha and Chain "Sub Solves".
	* If left at 0, the solver will attempt to determine the root of the chain by searching up the hierarchy until it finds a branch or another effector, whichever it finds first.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Full Body IK Effector", meta = (ClampMin = "0", UIMin = "0"))
	int32 ChainDepth = 0;

	/** Range 0-1 (default is 1.0). The strength of the effector when pulling the bone towards it's target location.
	* At 0.0, the effector does not pull at all, but the bones between the effector and the root will still slightly resist motion from other effectors.
	* This can thus act as a "stabilizer" for parts of the body that you do not want to behave in a pure FK fashion.
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Full Body IK Effector", meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float StrengthAlpha = 1.0f;

	/** Range 0-1 (default is 1.0). When enabled (greater than 0.0), the solver internally partitions the skeleton into 'chains' which extend
	 * from the effector up the hierarchy by "Chain Depth". If Chain Depth is 0, the chain root is set to the nearest fork in the skeleton.
	* These chains are pre-rotated and translated, as a whole, towards the effector targets.
	* This can improve the results for sparse bone chains, and significantly improve convergence on dense bone chains.
	* But it may cause undesirable results in highly constrained bone chains (like robot arms).
	*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Full Body IK Effector", meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float PullChainAlpha = 1.0f;

	/** Range 0-1 (default is 1.0).
	*Blends the effector bone rotation between the rotation of the effector transform (1.0) and the rotation of the input bone (0.0).*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Full Body IK Effector", meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float PinRotation = 1.0f;

	UPROPERTY(Transient)
	int32 IndexInSolver = -1;
};

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.FBIKBoneSettingsWrapper"))
struct FIKRigFBIKBoneSettings : public FIKRigBoneSettingsBase
{
	GENERATED_BODY()

	FIKRigFBIKBoneSettings() = default;
	FIKRigFBIKBoneSettings(FName InBoneName) { Bone = InBoneName; }
	
	/** Range is 0 to 1 (Default is 0). At higher values, the bone will resist rotating (forcing other bones to compensate). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Stiffness, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float RotationStiffness = 0.0f;

	/** Range is 0 to 1 (Default is 0). At higher values, the bone will resist translational motion (forcing other bones to compensate). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Stiffness, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float PositionStiffness = 0.0f;

	/** Limit the rotation angle of the bone on the X axis.
	 *Free: can rotate freely in this axis.
	 *Limited: rotation is clamped between the min/max angles relative to the Skeletal Mesh reference pose.
	 *Locked: no rotation is allowed in this axis (will remain at reference pose angle). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	EPBIKLimitType X = EPBIKLimitType::Free;
	/**Range is -180 to 0 (Default is 0). Degrees of rotation in the negative X direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "-180", ClampMax = "0", UIMin = "-180.0", UIMax = "0.0"))
	float MinX = 0.0f;
	/**Range is 0 to 180 (Default is 0). Degrees of rotation in the positive X direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "0", ClampMax = "180", UIMin = "0.0", UIMax = "180.0"))
	float MaxX = 0.0f;

	/** Limit the rotation angle of the bone on the Y axis.
	*Free: can rotate freely in this axis.
	*Limited: rotation is clamped between the min/max angles relative to the Skeletal Mesh reference pose.
	*Locked: no rotation is allowed in this axis (will remain at input pose angle). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	EPBIKLimitType Y = EPBIKLimitType::Free;
	/**Range is -180 to 0 (Default is 0). Degrees of rotation in the negative Y direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "-180", ClampMax = "0", UIMin = "-180.0", UIMax = "0.0"))
	float MinY = 0.0f;
	/**Range is 0 to 180 (Default is 0). Degrees of rotation in the positive Y direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "0", ClampMax = "180", UIMin = "0.0", UIMax = "180.0"))
	float MaxY = 0.0f;

	/** Limit the rotation angle of the bone on the Z axis.
	*Free: can rotate freely in this axis.
	*Limited: rotation is clamped between the min/max angles relative to the Skeletal Mesh reference pose.
	*Locked: no rotation is allowed in this axis (will remain at input pose angle). */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits)
	EPBIKLimitType Z = EPBIKLimitType::Free;
	/**Range is -180 to 0 (Default is 0). Degrees of rotation in the negative Z direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "-180", ClampMax = "0", UIMin = "-180.0", UIMax = "0.0"))
	float MinZ = 0.0f;
	/**Range is 0 to 180 (Default is 0). Degrees of rotation in the positive Z direction to allow when joint is in "Limited" mode. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = Limits, meta = (ClampMin = "0", ClampMax = "180", UIMin = "0.0", UIMax = "180.0"))
	float MaxZ = 0.0f;

	/**When true, this bone will "prefer" to rotate in the direction specified by the Preferred Angles when the chain it belongs to is compressed.
	 * Preferred Angles should be the first method used to fix bones that bend in the wrong direction (rather than limits).
	 * The resulting angles can be visualized on their own by temporarily setting the Solver iterations to 0 and moving the effectors.*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = PreferredAngles)
	bool bUsePreferredAngles = false;
	/**The local Euler angles (in degrees) used to rotate this bone when the chain it belongs to is squashed.
	 * This happens by moving the effector at the tip of the chain towards the root of the chain.
	 * This can be used to coerce knees and elbows to bend in the anatomically "correct" direction without resorting to limits (which may require more iterations to converge).*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = PreferredAngles)
	FVector PreferredAngles = FVector::ZeroVector;

	void CopyToCoreStruct(PBIK::FBoneSettings& Settings) const
	{
		Settings.RotationStiffness = RotationStiffness;
		Settings.PositionStiffness = PositionStiffness;
		Settings.X = static_cast<PBIK::ELimitType>(X);
		Settings.MinX = MinX;
		Settings.MaxX = MaxX;
		Settings.Y = static_cast<PBIK::ELimitType>(Y);
		Settings.MinY = MinY;
		Settings.MaxY = MaxY;
		Settings.Z = static_cast<PBIK::ELimitType>(Z);
		Settings.MinZ = MinZ;
		Settings.MaxZ = MaxZ;
		Settings.bUsePreferredAngles = bUsePreferredAngles;
		Settings.PreferredAngles.Pitch = PreferredAngles.Y;
		Settings.PreferredAngles.Yaw = PreferredAngles.Z;
		Settings.PreferredAngles.Roll = PreferredAngles.X;
	}
};

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.FBIKSettingsWrapper"))
struct FIKRigFBIKSettings : public FIKRigSolverSettingsBase
{
	GENERATED_BODY()

	/** All bones above this bone in the hierarchy will be completely ignored by the solver. Typically, this is set to
	 * the top-most skinned bone in the Skeletal Mesh (ie Pelvis, Hips etc), NOT the actual root of the entire skeleton.
	 *
	 * If you want to use the solver on a single chain of bones, and NOT translate the chain, ensure that "PinRoot" is
	 * checked on to disable the root from translating to reach the effector goals.*/
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Full Body IK Settings")
	FName RootBone;
	
	/** High iteration counts can help solve complex joint configurations with competing constraints, but will increase runtime cost. Default is 20. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0", ClampMax = "1000", UIMin = "0.0", UIMax = "200.0"))
	int32 Iterations = 20;

	/** Iterations used for sub-chains defined by the Chain Depth of the effectors. These are solved BEFORE the main iteration pass. Default is 0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0", ClampMax = "1000", UIMin = "0.0", UIMax = "200.0"))
	int32 SubIterations = 0;

	/** A global mass multiplier; higher values will make the joints more stiff, but require more iterations. Typical range is 0.0 to 10.0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0", UIMin = "0.0", UIMax = "10.0"))
	float MassMultiplier = 1.0f;

	/** If true, joints will translate to reach the effectors; causing bones to lengthen if necessary. Good for cartoon effects. Default is false. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = SolverSettings)
	bool bAllowStretch = false;

	/** (Default is PrePull) Set the behavior of the solver root.
	*Pre Pull: translates and rotates the root (and all children) by the average motion of the stretched effectors to help achieve faster convergence when reaching far.
	*Pin to Input: locks the translation and rotation of the root bone to the input pose. Overrides any bone settings applied to the root. Good for partial-body solves.
	*Free: treats the root bone like any other and allows it to move freely or according to any bone settings applied to it. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootBehavior)
	EPBIKRootBehavior RootBehavior = EPBIKRootBehavior::PrePull;

	/** Settings only applicable when Root Behavior is set to "PrePull". Use these values to adjust the gross movement and orientation of the entire skeleton. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = RootBehavior)
	FRootPrePullSettings PrePullRootSettings;

	/** A global multiplier for all Pull Chain Alpha values on all effectors. Range is 0.0 to 1.0. Default is 1.0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AdvancedSettings, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0.0", UIMax = "1.0"))
	float GlobalPullChainAlpha = 1.0f;

	/** Maximum angle that a joint can be rotated per constraint iteration. Lower this value if the solve is diverging. Range is 0.0 to 180.0. Default is 30. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AdvancedSettings, meta = (ClampMin = "0", ClampMax = "45", UIMin = "0.0", UIMax = "180.0"))
	float MaxAngle = 30.f;

	/** Pushes constraints beyond their normal amount to speed up convergence. Increasing this may speed up convergence, but at the cost of stability. Range is 1.0 - 2.0. Default is 1.3. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = AdvancedSettings, meta = (ClampMin = "1",  ClampMax = "2", UIMin = "1.0", UIMax = "2.0"))
	float OverRelaxation = 1.3f;
};


USTRUCT(BlueprintType)
struct FIKRigFullBodyIKSolver : public FIKRigSolverBase
{
	GENERATED_BODY()

	UPROPERTY()
	FIKRigFBIKSettings Settings;
	
	UPROPERTY()
	TArray<FIKRigFBIKGoalSettings> AllGoalSettings;

	UPROPERTY()
	TArray<FIKRigFBIKBoneSettings> AllBoneSettings;
	
	// FIKRigSolverBase runtime interface
	UE_API virtual void Initialize(const FIKRigSkeleton& InIKRigSkeleton) override;
	UE_API virtual void Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals) override;
	UE_API virtual void GetRequiredBones(TSet<FName>& OutRequiredBones) const override;
	UE_API virtual void GetRequiredGoals(TSet<FName>& OutRequiredGoals) const override;
	// END FIKRigSolverBase runtime interface

	// solver settings
	UE_API virtual FIKRigSolverSettingsBase* GetSolverSettings() override;
	UE_API virtual const UScriptStruct* GetSolverSettingsType() const override;
	// END solver settings
	
	// goals
	UE_API virtual void AddGoal(const UIKRigEffectorGoal& InNewGoal) override;
	UE_API virtual void OnGoalRenamed(const FName& InOldName, const FName& InNewName) override;
	UE_API virtual void OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName) override;
	UE_API virtual void OnGoalRemoved(const FName& InGoalName) override;
	// END goals

	// goal settings
	UE_API virtual bool UsesCustomGoalSettings() const override;
	UE_API virtual FIKRigGoalSettingsBase* GetGoalSettings(const FName& InGoalName) override;
	UE_API virtual const UScriptStruct* GetGoalSettingsType() const override;
	UE_API virtual void GetGoalsWithSettings(TSet<FName>& OutGoalsWithSettings) const override;
	// END goal settings

	// start bone
	UE_API virtual bool UsesStartBone() const override;
	UE_API virtual void SetStartBone(const FName& InStartBoneName) override;
	UE_API virtual FName GetStartBone() const override;
	// END start bone
	
	// bone settings
	UE_API virtual bool UsesCustomBoneSettings() const override;
	UE_API virtual void AddSettingsToBone(const FName& InBoneName) override;
	UE_API virtual void RemoveSettingsOnBone(const FName& InBoneName) override;
	UE_API virtual FIKRigBoneSettingsBase* GetBoneSettings(const FName& InBoneName) override;
	UE_API virtual const UScriptStruct* GetBoneSettingsType() const override;
	UE_API virtual bool HasSettingsOnBone(const FName& InBoneName) const override;
	UE_API virtual void GetBonesWithSettings(TSet<FName>& OutBonesWithSettings) const override;
	// END bone settings
	
#if WITH_EDITOR
	// return custom controller for scripting this solver
	UE_API virtual UIKRigSolverControllerBase* GetSolverController(UObject* Outer) override;
	// ui
	UE_API virtual FText GetNiceName() const override;
	UE_API virtual bool GetWarningMessage(FText& OutWarningMessage) const override;
	UE_API virtual bool IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const override;
#endif

private:

	FPBIKSolver Solver;

	int32 GetIndexOfGoal(const FName& InGoalName) const;
};


/* The blueprint/python API for modifying an Full-Body IK solver's settings in an IK Rig.
 * Can adjust Solver, Goal and Bone settings. */
UCLASS(MinimalAPI, BlueprintType)
class UIKRigFBIKController : public UIKRigSolverControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current solver settings as a struct.
	 * @return FIKRigFBIKSettings struct with the current settings used by the solver. */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API FIKRigFBIKSettings GetSolverSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRigFBIKSettings struct containing all the settings to apply to this solver */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API void SetSolverSettings(FIKRigFBIKSettings InSettings);

	/* Get the settings for the specified goal.
	 * @param InGoalName the name of the goal to get settings for
	 * @return FIKRigFBIKGoalSettings struct (empty if the specified goal does not belong to this solver) */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API FIKRigFBIKGoalSettings GetGoalSettings(const FName InGoalName);

	/* Set the settings for the specified goal.
	 * @param InGoalName: the name of the goal to assign the settings to.
	 * @param InSettings: a custom struct type with all the settings for an FBIK goal */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API void SetGoalSettings(const FName InGoalName, FIKRigFBIKGoalSettings InSettings);

	/* Get the settings associated with a particular bone.
	 * Note that you must AddBoneSettings() using the IK Rig controller before a bone will have settings on it.
	 * @param InBoneName the name of the bone to get settings for
	 * @return FIKRigFBIKBoneSettings struct holding all the settings for the specified bone (or empty if the bone did not have settings) */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API FIKRigFBIKBoneSettings GetBoneSettings(const FName InBoneName);

	/* Apply settings to a given bone
	 * @param InBoneName the name of the bone to apply the settings to
	 * @param InSettings a FIKRigFBIKBoneSettings struct containing the settings */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API void SetBoneSettings(const FName InBoneName, FIKRigFBIKBoneSettings InSettings);
};

//
// BEGIN LEGACY TYPES
//

// NOTE: This type has been replaced with FFBIKGoalSettings.
UCLASS(MinimalAPI)
class UIKRig_FBIKEffector : public UObject
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY()
	FName GoalName;
	UPROPERTY()
	FName BoneName;
	UPROPERTY()
	int32 ChainDepth = 0;
	UPROPERTY()
	float StrengthAlpha = 1.0f;
	UPROPERTY()
	float PullChainAlpha = 1.0f;
	UPROPERTY()
	float PinRotation = 1.0f;
};

// NOTE: This type has been replaced with FFBIKBoneSettings.
UCLASS(MinimalAPI)
class UIKRig_FBIKBoneSettings : public UObject
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	FName Bone;
	UPROPERTY()
	float RotationStiffness = 0.0f;
	UPROPERTY()
	float PositionStiffness = 0.0f;
	UPROPERTY()
	EPBIKLimitType X;
	UPROPERTY()
	float MinX = 0.0f;
	UPROPERTY()
	float MaxX = 0.0f;
	UPROPERTY()
	EPBIKLimitType Y;
	UPROPERTY()
	float MinY = 0.0f;
	UPROPERTY()
	float MaxY = 0.0f;
	UPROPERTY()
	EPBIKLimitType Z;
	UPROPERTY()
	float MinZ = 0.0f;
	UPROPERTY()
	float MaxZ = 0.0f;
	UPROPERTY()
	bool bUsePreferredAngles = false;
	UPROPERTY()
	FVector PreferredAngles;
};

// NOTE: This type has been replaced with FFBIKSolver.
UCLASS(MinimalAPI)
class UIKRigFBIKSolver : public UIKRigSolver
{
	GENERATED_BODY()

public:

	// upgrade patch to new UStruct-based solver
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) override
	{
		OutInstancedStruct.InitializeAs(FIKRigFullBodyIKSolver::StaticStruct());
		
		// load solver settings
		FIKRigFullBodyIKSolver& NewSolver = OutInstancedStruct.GetMutable<FIKRigFullBodyIKSolver>();
		NewSolver.SetEnabled(IsEnabled());
		NewSolver.Settings.RootBone = RootBone;
		NewSolver.Settings.Iterations = Iterations;
		NewSolver.Settings.SubIterations = SubIterations;
		NewSolver.Settings.MassMultiplier = MassMultiplier;
		NewSolver.Settings.bAllowStretch = bAllowStretch;
		NewSolver.Settings.RootBehavior = RootBehavior;
		NewSolver.Settings.PrePullRootSettings = PrePullRootSettings;
		NewSolver.Settings.GlobalPullChainAlpha = PullChainAlpha;
		NewSolver.Settings.MaxAngle = MaxAngle;
		NewSolver.Settings.OverRelaxation = OverRelaxation;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// load effectors into solver
		for (UIKRig_FBIKEffector* OldEffector : Effectors_DEPRECATED)
		{
			if (!OldEffector)
			{
				continue;
			}
			FIKRigFBIKGoalSettings NewEffector;
			NewEffector.Goal = OldEffector->GoalName;
			NewEffector.BoneName = OldEffector->BoneName;
			NewEffector.ChainDepth = OldEffector->ChainDepth;
			NewEffector.StrengthAlpha = OldEffector->StrengthAlpha;
			NewEffector.PullChainAlpha = OldEffector->PullChainAlpha;
			NewEffector.PinRotation = OldEffector->PinRotation;
			NewSolver.AllGoalSettings.Add(NewEffector);
		}
		
		// load bone settings into solver
		for (UIKRig_FBIKBoneSettings* OldBoneSetting : BoneSettings_DEPRECATED)
		{
			if (!OldBoneSetting)
			{
				continue;
			}
			FIKRigFBIKBoneSettings NewBoneSetting;
			NewBoneSetting.Bone = OldBoneSetting->Bone;
			NewBoneSetting.RotationStiffness = OldBoneSetting->RotationStiffness;
			NewBoneSetting.PositionStiffness = OldBoneSetting->PositionStiffness;
			NewBoneSetting.X = OldBoneSetting->X;
			NewBoneSetting.MinX = OldBoneSetting->MinX;
			NewBoneSetting.MaxX = OldBoneSetting->MaxX;
			NewBoneSetting.Y = OldBoneSetting->Y;
			NewBoneSetting.MinY = OldBoneSetting->MinY;
			NewBoneSetting.MaxY = OldBoneSetting->MaxY;
			NewBoneSetting.Z = OldBoneSetting->Z;
			NewBoneSetting.MinZ = OldBoneSetting->MinZ;
			NewBoneSetting.MaxZ = OldBoneSetting->MaxZ;
			NewBoneSetting.bUsePreferredAngles = OldBoneSetting->bUsePreferredAngles;
			NewBoneSetting.PreferredAngles = OldBoneSetting->PreferredAngles;
			NewSolver.AllBoneSettings.Add(NewBoneSetting);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
	
	UPROPERTY()
	FName RootBone;
	UPROPERTY()
	int32 Iterations = 20;
	UPROPERTY()
	int32 SubIterations = 0;
	UPROPERTY()
	float MassMultiplier = 1.0f;
	UPROPERTY()
	bool bAllowStretch = false;
	UPROPERTY()
	EPBIKRootBehavior RootBehavior = EPBIKRootBehavior::PrePull;
	UPROPERTY()
	FRootPrePullSettings PrePullRootSettings;
	UPROPERTY()
	float PullChainAlpha = 1.0f;
	UPROPERTY()
	float MaxAngle = 30.f;
	UPROPERTY()
	float OverRelaxation = 1.3f;

	UE_DEPRECATED(5.6, "Deprecated UObject based FBIK solver.")
	UFUNCTION(meta = (DeprecatedFunction))
	TArray<UIKRig_FBIKEffector*> GetEffectors()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Effectors_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.6, "Deprecated UObject based FBIK solver.")
	UFUNCTION(meta = (DeprecatedFunction))
	TArray<UIKRig_FBIKBoneSettings*> GetBoneSettings()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return BoneSettings_DEPRECATED;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.6, "IK Rig refactored to not use UObjects")
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TObjectPtr<UIKRig_FBIKEffector>> Effectors_DEPRECATED;

	UE_DEPRECATED(5.6, "IK Rig refactored to not use UObjects")
	UPROPERTY(meta = (DeprecatedProperty))
	TArray<TObjectPtr<UIKRig_FBIKBoneSettings>> BoneSettings_DEPRECATED;
};

//
// END LEGACY TYPES
//

#undef UE_API
