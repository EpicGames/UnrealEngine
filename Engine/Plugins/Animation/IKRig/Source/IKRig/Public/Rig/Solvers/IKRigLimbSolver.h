// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/IKRigDefinition.h"
#include "IKRigSolverBase.h"

#include "Rig/Solvers/LimbSolver.h"

#include "IKRigLimbSolver.generated.h"

#define UE_API IKRIG_API


USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.LimbSolverSettingsWrapper"))
struct FIKRigLimbSolverSettings : public FLimbSolverSettings
{
	GENERATED_BODY()
	
	/** The first bone in the IK chain, for example the "hip" in a leg, or the "shoulder" in an arm */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, DisplayName = "Root Bone", Category = "Limb IK Settings")
	FName StartBone = NAME_None;

	/** The name of the IK goal to drive the end bone towards */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Limb IK Effector")
	FName GoalName = NAME_None;

	/** The name of the last bone in the IK chain. This is the bone you want to reach the goal. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Limb IK Effector")
	FName EndBone = NAME_None;
};

USTRUCT(BlueprintType)
struct FIKRigLimbSolver : public FIKRigSolverBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	FIKRigLimbSolverSettings Settings;
	
	// runtime
	UE_API virtual void Initialize(const FIKRigSkeleton& InIKRigSkeleton) override;
	UE_API virtual void Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals) override;
	UE_API virtual void GetRequiredBones(TSet<FName>& OutRequiredBones) const override;
	UE_API virtual void GetRequiredGoals(TSet<FName>& OutRequiredGoals) const override;
	
	// solver settings
	UE_API virtual FIKRigSolverSettingsBase* GetSolverSettings() override;
	UE_API virtual const UScriptStruct* GetSolverSettingsType() const override;

	// goals
	UE_API virtual void AddGoal(const UIKRigEffectorGoal& InNewGoal) override;
	UE_API virtual void OnGoalRenamed(const FName& InOldName, const FName& InNewName) override;
	UE_API virtual void OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName) override;
	UE_API virtual void OnGoalRemoved(const FName& InGoalName) override;
	
	// start bone
	UE_API virtual bool UsesStartBone() const override;
	UE_API virtual void SetStartBone(const FName& InStartBoneName) override;
	UE_API virtual FName GetStartBone() const override;

#if WITH_EDITOR
	// return custom controller for scripting this solver
	UE_API virtual UIKRigSolverControllerBase* GetSolverController(UObject* Outer) override;
	
	// general UI
	UE_API virtual FText GetNiceName() const override;
	UE_API virtual bool GetWarningMessage(FText& OutWarningMessage) const override;
	UE_API virtual bool IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const override;
#endif
	
private:

	static void GatherChildren(const int32 BoneIndex, const FIKRigSkeleton& InSkeleton, TArray<int32>& OutChildren);
	
	FLimbSolver Solver;
	TArray<int32> ChildrenToUpdate;
	
};

/* The blueprint/python API for modifying an Full-Body IK solver's settings in an IK Rig.
 * Can adjust Solver, Goal and Bone settings. */
UCLASS(MinimalAPI, BlueprintType)
class UIKRigLimbSolverController : public UIKRigSolverControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current solver settings as a struct.
	 * @return FIKRigLimbSolverSettings struct with the current settings used by the solver. */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API FIKRigLimbSolverSettings GetSolverSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRigLimbSolverSettings struct containing all the settings to apply to this solver */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API void SetSolverSettings(FIKRigLimbSolverSettings InSettings);
};

//
// BEGIN LEGACY TYPES
//

// NOTE: This type has been replaced with FIKRigLimbSolver.
UCLASS(MinimalAPI)
class UIKRig_LimbEffector : public UObject
{
	GENERATED_BODY()

public:
	UIKRig_LimbEffector() { SetFlags(RF_Transactional); }

	UPROPERTY()
	FName GoalName = NAME_None;
	UPROPERTY()
	FName BoneName = NAME_None;
};

// NOTE: This type has been replaced with FIKRigLimbSolver.
UCLASS(MinimalAPI)
class UIKRig_LimbSolver : public UIKRigSolver
{
	GENERATED_BODY()

public:

	UIKRig_LimbSolver()
	{
		Effector_DEPRECATED = CreateDefaultSubobject<UIKRig_LimbEffector>(TEXT("Effector"));
	}

	// upgrade patch to new UStruct-based solver
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) override
	{
		OutInstancedStruct.InitializeAs(FIKRigLimbSolver::StaticStruct());
		FIKRigLimbSolver& NewSolver = OutInstancedStruct.GetMutable<FIKRigLimbSolver>();
		NewSolver.SetEnabled(IsEnabled());
		NewSolver.Settings.StartBone = RootName;
		NewSolver.Settings.GoalName = Effector_DEPRECATED->GoalName;
		NewSolver.Settings.EndBone = Effector_DEPRECATED->BoneName;
		NewSolver.Settings.ReachPrecision = ReachPrecision;
		NewSolver.Settings.HingeRotationAxis = HingeRotationAxis;
		NewSolver.Settings.MaxIterations = MaxIterations;
		NewSolver.Settings.bEnableLimit = bEnableLimit;
		NewSolver.Settings.MinRotationAngle = MinRotationAngle;
		NewSolver.Settings.bAveragePull = bAveragePull;
		NewSolver.Settings.PullDistribution = PullDistribution;
		NewSolver.Settings.ReachStepAlpha = ReachStepAlpha;
		NewSolver.Settings.bEnableTwistCorrection = bEnableTwistCorrection;
		NewSolver.Settings.EndBoneForwardAxis = EndBoneForwardAxis;
	}
	
	UPROPERTY()
	FName RootName = NAME_None;
	UPROPERTY()
	float ReachPrecision = 0.01f;
	UPROPERTY()
	TEnumAsByte<EAxis::Type> HingeRotationAxis = EAxis::None;
	UPROPERTY()
	int32 MaxIterations = 12;
	UPROPERTY()
	bool bEnableLimit = false;
	UPROPERTY()
	float MinRotationAngle = 15.f;
	UPROPERTY()
	bool bAveragePull = true;
	UPROPERTY()
	float PullDistribution = 0.5f;
	UPROPERTY()
	float ReachStepAlpha = 0.7f;
	UPROPERTY()
	bool bEnableTwistCorrection = false;
	UPROPERTY()
	TEnumAsByte<EAxis::Type> EndBoneForwardAxis = EAxis::Y;
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UIKRig_LimbEffector> Effector_DEPRECATED;
};

//
// END LEGACY TYPES
//

#undef UE_API
