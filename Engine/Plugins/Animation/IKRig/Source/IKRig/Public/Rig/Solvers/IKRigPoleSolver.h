// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/IKRigDefinition.h"
#include "Rig/Solvers/IKRigSolverBase.h"

#include "IKRigPoleSolver.generated.h"

#define UE_API IKRIG_API

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.PoleSolverSettingsWrapper"))
struct FIKRigPoleSolverSettings : public FIKRigSolverSettingsBase
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Pole Solver Settings")
	FName StartBone = NAME_None;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Pole Solver Settings")
	FName EndBone = NAME_None;

	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Pole Solver Settings")
	FName AimAtGoal = NAME_None;

	// Blend the effect on/off. Range is 0-1. Default is 1.0. 
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Pole Solver Settings", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Alpha = 1.0f;
};

USTRUCT(BlueprintType)
struct FIKRigPoleSolver : public FIKRigSolverBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	FIKRigPoleSolverSettings Settings;

	// runtime FIKRigSolverBase interface
	UE_API virtual void Initialize(const FIKRigSkeleton& InIKRigSkeleton) override;
	UE_API virtual void Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals) override;
	UE_API virtual void GetRequiredBones(TSet<FName>& OutRequiredBones) const override;
	UE_API virtual void GetRequiredGoals(TSet<FName>& OutRequiredGoals) const override;

	// solver settings
	UE_API virtual FIKRigSolverSettingsBase* GetSolverSettings() override;
	UE_API virtual const UScriptStruct* GetSolverSettingsType() const override;
	
	// goals
	UE_API virtual void AddGoal(const UIKRigEffectorGoal& NewGoal) override;
	UE_API virtual void OnGoalRenamed(const FName& InOldName, const FName& InNewName) override;
	UE_API virtual void OnGoalRemoved(const FName& InName) override;
	UE_API virtual void OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName) override;
	
	// start bone can be set on this solver
	virtual bool UsesStartBone() const override { return true; };
	UE_API virtual void SetStartBone(const FName& InRootBoneName) override;
	UE_API virtual FName GetStartBone() const override;
	
	// end bone can be set on this solver
	virtual bool UsesEndBone() const override { return true; };
	UE_API virtual void SetEndBone(const FName& InEndBoneName) override;
	UE_API virtual FName GetEndBone() const override;

#if WITH_EDITOR
	// return custom controller for scripting this solver
	UE_API virtual UIKRigSolverControllerBase* GetSolverController(UObject* Outer) override;
	
	// UI
	UE_API virtual FText GetNiceName() const override;
	UE_API virtual bool GetWarningMessage(FText& OutWarningMessage) const override;
	UE_API virtual bool IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const override;
#endif

private:
	
	TArray<int32> Chain;
	TArray<int32> ChildrenToUpdate;

	static UE_API void GatherChildren(const int32 BoneIndex, const FIKRigSkeleton& InSkeleton, TArray<int32>& OutChildren);
};

/* The blueprint/python API for modifying a Pole solver's settings in an IK Rig. */
UCLASS(MinimalAPI, BlueprintType)
class UIKRigPoleSolverController : public UIKRigSolverControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current solver settings as a struct.
	 * @return FIKRigPoleSolverSettings struct with the current settings used by the solver. */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API FIKRigPoleSolverSettings GetSolverSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRigPoleSolverSettings struct containing all the settings to apply to this solver */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API void SetSolverSettings(FIKRigPoleSolverSettings InSettings);
};

//
// BEGIN LEGACY TYPES
//

// NOTE: This type has been replaced with FPoleSolver.
UCLASS(MinimalAPI)
class UIKRig_PoleSolverEffector : public UObject
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	FName GoalName = NAME_None;
	UPROPERTY()
	FName BoneName = NAME_None;
	UPROPERTY()
	float Alpha = 1.0f;
};

// NOTE: This type has been replaced with FPoleSolver.
UCLASS(MinimalAPI)
class UIKRig_PoleSolver : public UIKRigSolver
{
	GENERATED_BODY()

public:

	UIKRig_PoleSolver()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Effector_DEPRECATED = CreateDefaultSubobject<UIKRig_PoleSolverEffector>(TEXT("Effector"));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
	
	UPROPERTY()
	FName RootName = NAME_None;
	UPROPERTY()
	FName EndName = NAME_None;
	UE_DEPRECATED(5.6, "IK Rig refactored to not use UObjects")
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UIKRig_PoleSolverEffector> Effector_DEPRECATED;

	// upgrade patch to new UStruct-based solver
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) override
	{
		OutInstancedStruct.InitializeAs(FIKRigPoleSolver::StaticStruct());
		FIKRigPoleSolver& NewSolver = OutInstancedStruct.GetMutable<FIKRigPoleSolver>();

		// load solver and effector settings
		NewSolver.SetEnabled(IsEnabled());
		NewSolver.Settings.StartBone = RootName;
		NewSolver.Settings.EndBone = EndName;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NewSolver.Settings.AimAtGoal = Effector_DEPRECATED->GoalName;
		NewSolver.Settings.Alpha = Effector_DEPRECATED->Alpha;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
};

//
// END LEGACY TYPES
//

#undef UE_API
