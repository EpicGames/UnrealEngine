// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/IKRigDefinition.h"
#include "IKRigSolverBase.h"

#include "IKRigSetTransform.generated.h"

#define UE_API IKRIG_API

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.SetTransformSettingsWrapper"))
struct FIKRigSetTransformSettings : public FIKRigSolverSettingsBase
{
	GENERATED_BODY()
	
	// The goal used to affect the bone transform
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Set Transform Settings")
	FName Goal;

	// The bone to affect
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Set Transform Settings")
	FName BoneToAffect;

	// Blend the translation on/off. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Set Transform Effector", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float PositionAlpha = 1.0;

	// Blend the rotation on/off. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Set Transform Effector", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float RotationAlpha = 1.0f;

	// Blend the total effect on/off. Range is 0-1. Default is 1.0.
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Set Transform Effector", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0"))
	float Alpha = 1.0f;

	// If true, the transform modification will propagate to the hierarchy below the target bone.
	UPROPERTY(EditAnywhere, Category = "Set Transform Settings")
	bool bPropagateToChildren = true;
};

USTRUCT(BlueprintType)
struct FIKRigSetTransform : public FIKRigSolverBase
{
	GENERATED_BODY()
	
	UPROPERTY()
	FIKRigSetTransformSettings Settings;

	// runtime
	UE_API virtual void Initialize(const FIKRigSkeleton& IKRigSkeleton) override;
	UE_API virtual void Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals) override;
	UE_API virtual void GetRequiredBones(TSet<FName>& OutRequiredBones) const override;
	UE_API virtual void GetRequiredGoals(TSet<FName>& OutRequiredGoals) const override;

	// settings
	UE_API virtual FIKRigSolverSettingsBase* GetSolverSettings() override;
	UE_API virtual const UScriptStruct* GetSolverSettingsType() const override;
	
	// goals
	UE_API virtual void AddGoal(const UIKRigEffectorGoal& InNewGoal) override;
	UE_API virtual void OnGoalRemoved(const FName& InGoalName) override;
	UE_API virtual void OnGoalRenamed(const FName& InOldName, const FName& InNewName) override;
	UE_API virtual void OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName) override;
	
#if WITH_EDITOR
	// return custom controller for scripting this solver
	UE_API virtual UIKRigSolverControllerBase* GetSolverController(UObject* Outer) override;
	// ui
	UE_API virtual FText GetNiceName() const override;
	UE_API virtual bool GetWarningMessage(FText& OutWarningMessage) const override;
	UE_API virtual bool IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const override;
#endif

private:

	int32 BoneIndex = INDEX_NONE;
};

/* The blueprint/python API for modifying Set Transform settings in an IK Rig. */
UCLASS(MinimalAPI, BlueprintType)
class UIKRigSetTransformController : public UIKRigSolverControllerBase
{
	GENERATED_BODY()
	
public:

	/* Get the current solver settings as a struct.
	 * @return FIKRigSetTransformSettings struct with the current settings used by the solver. */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API FIKRigSetTransformSettings GetSolverSettings();

	/* Set the solver settings. Input is a custom struct type for this solver.
	 * @param InSettings a FIKRigSetTransformSettings struct containing all the settings to apply to this solver */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	UE_API void SetSolverSettings(FIKRigSetTransformSettings InSettings);
};

//
// BEGIN LEGACY TYPES
//

// NOTE: This is a legacy type that is converted into an FSetTransformGoalSettings on PostLoad()
UCLASS(MinimalAPI)
class UIKRig_SetTransformEffector : public UObject
{
	GENERATED_BODY()

public:
	
	UPROPERTY()
	bool bEnablePosition = true;
	UPROPERTY()
	bool bEnableRotation = true;
	UPROPERTY()
	float Alpha = 1.0f;
};

// NOTE: This is a legacy type that is converted into an FIKRigSetTransform on PostLoad()
UCLASS(MinimalAPI)
class UIKRig_SetTransform : public UIKRigSolver
{
	GENERATED_BODY()

public:
	
	UIKRig_SetTransform()
	{
		Effector_DEPRECATED = CreateDefaultSubobject<UIKRig_SetTransformEffector>(TEXT("Effector"));
	}

	// upgrade patch to new UStruct-based solver
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct) override
	{
		OutInstancedStruct.InitializeAs(FIKRigSetTransform::StaticStruct());
		FIKRigSetTransform& NewSolver = OutInstancedStruct.GetMutable<FIKRigSetTransform>();
		NewSolver.SetEnabled(IsEnabled());
		NewSolver.Settings.Goal = Goal;
		NewSolver.Settings.BoneToAffect = RootBone;
		NewSolver.Settings.Alpha = Effector_DEPRECATED->Alpha;
		NewSolver.Settings.PositionAlpha = Effector_DEPRECATED->bEnablePosition ? 1.0f : 0.0f;
		NewSolver.Settings.RotationAlpha = Effector_DEPRECATED->bEnableRotation ? 1.0f : 0.0f;
	};

	UPROPERTY()
	FName Goal;
	UPROPERTY()
	FName RootBone;
	UPROPERTY(meta = (DeprecatedProperty))
	TObjectPtr<UIKRig_SetTransformEffector> Effector_DEPRECATED;
};

//
// END LEGACY TYPES
//

#undef UE_API
