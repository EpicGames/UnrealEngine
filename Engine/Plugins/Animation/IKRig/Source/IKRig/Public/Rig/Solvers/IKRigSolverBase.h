// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"

#include "IKRigSolverBase.generated.h"

#define UE_API IKRIG_API

class UIKRigEffectorGoal;
class FPrimitiveDrawInterface;
struct FIKRigGoalContainer;
struct FIKRigSkeleton;
struct FInstancedStruct;

// This is the base class for all types of settings used by IKRig
USTRUCT(BlueprintType)
struct FIKRigSettingsBase
{
	GENERATED_BODY()
	
	virtual ~FIKRigSettingsBase() = default;
};

// This is the base class for defining editable per-goal settings for your custom IKRig solver.
// For example, your solver could have a "Strength" value for goals, which can go here.
// NOTE: the derived type must be returned by the solver's GetGoalSettingsType() and GetGoalSettings()
USTRUCT(BlueprintType)
struct FIKRigGoalSettingsBase : public FIKRigSettingsBase
{
	GENERATED_BODY()

	/** The IKRig Goal that these settings are applied to. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Goal Settings")
	FName Goal;
	
	virtual ~FIKRigGoalSettingsBase() = default;
};

// This is the base class for defining editable per-bone settings for your custom IKRig solver.
// For example, your solver may have rotation limits per-bone which can be stored here.
// NOTE: the derived type must be returned by the solver's GetBoneSettingsType() and GetBoneSettings()
USTRUCT(BlueprintType)
struct FIKRigBoneSettingsBase : public FIKRigSettingsBase
{
	GENERATED_BODY()

	/** The bone these settings are applied to. */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Bone Settings")
	FName Bone;
	
	virtual ~FIKRigBoneSettingsBase() = default;
};

// This is the base class for defining editable settings for your custom IKRig solver.
// All user-configurable properties for your solver should be stored in a subclass of this
// NOTE: the derived type must be returned by the solver's GetSolverSettingsType() and GetSolverSettings()
USTRUCT(BlueprintType)
struct FIKRigSolverSettingsBase : public FIKRigSettingsBase
{
	GENERATED_BODY()
	
	virtual ~FIKRigSolverSettingsBase() = default;
};

// this is the base class for creating your own solver type that integrates into the IK Rig framework/editor.
USTRUCT(BlueprintType)
struct FIKRigSolverBase
{
	GENERATED_BODY()

	virtual ~FIKRigSolverBase() = default;

	// get if this solver is enabled
	bool IsEnabled() const { return bIsEnabled; };
	// turn solver on/off (will be skipped during execution if disabled)
	void SetEnabled(const bool bEnabled){ bIsEnabled = bEnabled; };
	
	// RUNTIME
	// (required) override to setup internal data based on ref pose
	virtual void Initialize(const FIKRigSkeleton& InIKRigSkeleton) { checkNoEntry(); };
	// (required) override Solve() to evaluate new output pose (InOutGlobalTransform)
	virtual void Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals) { checkNoEntry(); };
	// (required) override to add any bones which this solver requires (helps validate skeleton compatibility)
	virtual void GetRequiredBones(TSet<FName>& OutRequiredBones) const  { checkNoEntry(); };
	// (required) override to add any goals that have been added to this solver
	virtual void GetRequiredGoals(TSet<FName>& OutRequiredGoals) const  { checkNoEntry(); };
	// END RUNTIME

	// (optional) override to add custom logic when updating settings from the source asset at runtime
	// NOTE: you can safely cast this to your own solver type and copy any relevant settings at runtime
	// This is necessary because at runtime, the IKRigProcessor creates a copy of your solver struct
	// and the copy must be notified of changes made to the editor instance so that users can see their edits update in PIE.
	// NOTE: the default implementation will copy solver, goal and bone settings if your solver has them
	UE_API virtual void UpdateSettingsFromAsset(const FIKRigSolverBase& InAssetSolver);
	
	// SOLVER SETTINGS
	// (required) override this to support custom settings for your solver type
	virtual FIKRigSolverSettingsBase* GetSolverSettings()   { checkNoEntry(); return nullptr; };
	// (required) override this and return the type your solver uses to store its settings
	virtual const UScriptStruct* GetSolverSettingsType() const { return nullptr; };
	// (optional) override to add custom logic if needed when outside systems set solver settings.
	UE_API virtual void SetSolverSettings(const FIKRigSolverSettingsBase* InSettings);
	// END SOLVER SETTINGS

	// GOALS
	// (required) override to support ADDING a new goal to custom solver
	virtual void AddGoal(const UIKRigEffectorGoal& InNewGoal)  { checkNoEntry(); };
	// (required) override to support RENAMING an existing goal
	virtual void OnGoalRenamed(const FName& InOldName, const FName& InNewName)  { checkNoEntry(); };
	// (required) override to support CHANGING BONE for an existing goal
	virtual void OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName)  { checkNoEntry(); };
	// (required) override to support REMOVING a goal from custom solver
	virtual void OnGoalRemoved(const FName& InGoalName)  { checkNoEntry(); };
	// END Goals

	// GOAL SETTINGS
	// (optional) override and return true to tell systems if this solver supports custom per-goal settings
	virtual bool UsesCustomGoalSettings() const { return false; };
	// (required if UsesCustomGoalSettings())
	// override to support supplying goals settings specific to this solver to outside systems for editing/UI
	// returns a pointer to the settings stored for the given bone (null if bone does not have settings)
	virtual FIKRigGoalSettingsBase* GetGoalSettings(const FName& InGoalName) { ensure(!UsesCustomGoalSettings()); return nullptr; };
	// (optional) override if you need custom logic when setting goal settings from outside systems.
	UE_API virtual void SetGoalSettings(const FName& InGoalName, FIKRigGoalSettingsBase* InSettings);
	// (required if UsesCustomGoalSettings()) returns the type used for the goal settings in this solver
	virtual const UScriptStruct* GetGoalSettingsType() const { ensure(!UsesCustomGoalSettings()); return nullptr; };
	// (required if UsesCustomGoalSettings()) override to support telling outside systems which goals this solver has settings for.
	virtual void GetGoalsWithSettings(TSet<FName>& OutGoalsWithSettings) const { ensure(!UsesCustomGoalSettings()); };
	// END GOAL SETTINGS

	// START BONE
	// (optional) determines if your solver requires a start bone to be specified (most do)
	// override and return true if your solver requires the user to specify a start bone
	virtual bool UsesStartBone() const { return false; };
	// (required if UsesStartBone()) if solver requires a start bone, then override this to return it.
	virtual FName GetStartBone() const { ensure(!UsesStartBone()); return NAME_None; };
	// (required if UsesStartBone()) override to support SETTING ROOT BONE for the solver
	virtual void SetStartBone(const FName& InRootBoneName){ ensure(!UsesStartBone()); };
	// END START BONE

	// END BONE
	// (optional) implement if your solver requires an end bone
	// override and return true if this solver requires an end bone to be specified
	virtual bool UsesEndBone() const { return false; };
	// (required if UsesEndBone()) override to support getting end bone of the solver
	virtual FName GetEndBone() const { ensure(!UsesEndBone()); return NAME_None; };
	// (required if UsesEndBone()) override to support setting end bone for the solver
	virtual void SetEndBone(const FName& InEndBoneName){ ensure(!UsesEndBone()); };
	// END ROOT BONE

	// BONE SETTINGS
	// (optional) implement if your solver supports per-bone settings
	// override and return true to tell systems if this solver supports per-bone settings
	virtual bool UsesCustomBoneSettings() const { return false; };
	// (required if UsesCustomBoneSettings()) override to support ADDING settings to a particular bone
	virtual void AddSettingsToBone(const FName& InBoneName){ ensure(!UsesCustomBoneSettings()); };
	// (required if UsesCustomBoneSettings()) override to support REMOVING settings on a particular bone
	virtual void RemoveSettingsOnBone(const FName& InBoneName){ ensure(!UsesCustomBoneSettings()); };
	// (required if UsesCustomBoneSettings()) override to support supplying per-bone settings to outside systems for editing/UI
	// NOTE: This must be overriden on solvers that use bone setting and return nullptr if no settings for the given bone
	virtual FIKRigBoneSettingsBase* GetBoneSettings(const FName& InBoneName) { ensure(!UsesCustomBoneSettings()); return nullptr; };
	// (optional) override if you need custom logic when per-bone settings are applied from outside systems
	UE_API virtual void SetBoneSettings(const FName& InBoneName, FIKRigBoneSettingsBase* InSettings);
	// (required if UsesCustomBoneSettings()) returns the type used for the bone settings in this solver
	virtual const UScriptStruct* GetBoneSettingsType() const {  ensure(!UsesCustomBoneSettings()); return nullptr; };
	// (required if UsesCustomBoneSettings()) override to inform outside systems if the given bone has custom settings on it
	virtual bool HasSettingsOnBone(const FName& InBoneName) const { ensure(!UsesCustomBoneSettings()); return false; };
	// (required if UsesCustomBoneSettings()) override to support telling outside systems which bones this solver has settings for.
	// NOTE: Only ADD to the incoming set, do not remove from it.
	virtual void GetBonesWithSettings(TSet<FName>& OutBonesWithSettings) const { ensure(!UsesCustomBoneSettings()); };
	// END BONE SETTINGS

#if WITH_EDITOR

	// SCRIPTING API
	// (optional, but recommended) provide a scripting object to edit your custom solver type in the editor via BP/Python
	UE_API virtual UIKRigSolverControllerBase* GetSolverController(UObject* Outer);
	// END SCRIPTING API

	// GENERAL UI
	// (required) override to give your solver a nice name to display in the UI
	virtual FText GetNiceName() const { checkNoEntry() return FText::GetEmpty(); };
	// (optional, but recommended) override to provide warning to user during setup of any missing components. return false if no warnings.
	virtual bool GetWarningMessage(FText& OutWarningMessage) const { return false; };
	// (optional, but recommended) return true if the supplied Bone is affected by this solver - this provides UI feedback for user
	virtual bool IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const { return false; };
	// END GENERAL UI

	// TODO: This will be supported in a future version of Unreal
	// (optional) override to do custom debug drawing in the editor viewport
	virtual void DrawBoneSettings(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton, FPrimitiveDrawInterface* PDI) const {};

protected:

	// helper function for solvers to instantiate their own controller
	UE_API UIKRigSolverControllerBase* CreateControllerIfNeeded(UObject* Outer, const UClass* ClassType);

	// an optional custom editor controller used to edit this solver by script/blueprint
	TStrongObjectPtr<UIKRigSolverControllerBase> Controller = nullptr;
	
#endif

private:
	
	// generic function to set all types of solver settings
	static UE_API void CopyAllEditableProperties(const UScriptStruct* SettingsType, const FIKRigSettingsBase* CopyFrom, FIKRigSettingsBase* CopyTo);

	// all solvers can be enabled/disabled in the UI or API
	UPROPERTY()
	bool bIsEnabled = true;
};

UCLASS(MinimalAPI, BlueprintType)
class UIKRigSolverControllerBase : public UObject
{
	GENERATED_BODY()
	
public:

	/* Turn the solver on/off in the IK Rig solver stack.
	 * @param bIsEnabled if true, the solver will execute, false otherwise. */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	void SetEnabled(bool bIsEnabled) { SolverToControl->SetEnabled(bIsEnabled); };

	/* Get if the solver is on or off in the IK Rig solver stack.
	 * @param bIsEnabled returns true if the solver is enabled, false otherwise. */
	UFUNCTION(BlueprintCallable, Category = IKRig)
	bool GetEnabled() { return SolverToControl->IsEnabled(); };

	// the solver this controller controls
	FIKRigSolverBase* SolverToControl = nullptr;
};

//
// BEGIN LEGACY CODE
//
// NOTE: As of Unreal 5.6, the old UObject based IK Rig system has been replaced.
// These are left here to load old UObject data into the new format.
//

// NOTE on UIKRigSolver upgrade path:
// In 5.6 the UObject's in IK Rig have been replaced with runtime-friendly UStruct based data structures
// As part of this change, solvers based on UIKRigSolver will no longer function and must be upgraded to the new base FIKRigSolverBase.
// Old assets using UIKRigSolver-based solvers can be loaded and patched using the ConvertToInstancedStruct() function (see below)
// All solvers that ship with the IKRig plugin have been ported to the new system and should work exactly as before.
UCLASS(MinimalAPI)
class UIKRigSolver : public UObject
{
	GENERATED_BODY()
	
public:

	// this is the legacy upgrade path for solvers that inherit from UIKRigSolver to convert them to FIKRigSolverBase
	// override this and supply a struct into OutInstancedStruct that derives from FIKRigSolverBase that implements your custom solver.
	virtual void ConvertToInstancedStruct(FInstancedStruct& OutInstancedStruct){};

	// LEGACY UObject-based solver interface
	virtual void Initialize(const FIKRigSkeleton& IKRigSkeleton) {};
	virtual void Solve(FIKRigSkeleton& IKRigSkeleton, const FIKRigGoalContainer& Goals) {};
	virtual FName GetRootBone() const { return NAME_None; };
	virtual void GetBonesWithSettings(TSet<FName>& OutBonesWithSettings) const {};
	bool IsEnabled() const { return bIsEnabled; };
	void SetEnabled(const bool bEnabled){ bIsEnabled = bEnabled; };
	virtual void UpdateSolverSettings(UIKRigSolver* InSettings){};
	virtual void RemoveGoal(const FName& GoalName){};
	virtual bool IsGoalConnected(const FName& GoalName) const {return false;};
#if WITH_EDITORONLY_DATA
	virtual FText GetNiceName() const { return FText::GetEmpty(); };
	virtual bool GetWarningMessage(FText& OutWarningMessage) const { return false; };
	virtual void AddGoal(const UIKRigEffectorGoal& NewGoal){};
	virtual void RenameGoal(const FName& OldName, const FName& NewName){};
	virtual void SetGoalBone(const FName& GoalName, const FName& NewBoneName){};
	virtual UObject* GetGoalSettings(const FName& GoalName) const {return nullptr;};
	virtual void SetRootBone(const FName& RootBoneName){};
	virtual bool RequiresRootBone() const { return false; };
	virtual void SetEndBone(const FName& EndBoneName){};
	virtual FName GetEndBone() const { return NAME_None; };
	virtual bool RequiresEndBone() const { return false; };
	virtual void AddBoneSetting(const FName& BoneName){};
	virtual void RemoveBoneSetting(const FName& BoneName){};
	virtual UObject* GetBoneSetting(const FName& BoneName) const { ensure(!UsesBoneSettings()); return nullptr; };
	virtual bool UsesBoneSettings() const { return false;};
	virtual void DrawBoneSettings(const FName& BoneName, const FIKRigSkeleton& IKRigSkeleton, FPrimitiveDrawInterface* PDI) const {};
	virtual bool IsBoneAffectedBySolver(const FName& BoneName, const FIKRigSkeleton& IKRigSkeleton) const { return false; };
	// END LEGACY UObject-based solver interface
#endif

private:
	
	UPROPERTY()
	bool bIsEnabled = true;
};

#undef UE_API
