// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/IKRigDefinition.h"
#include "IKRigSolverBase.h"

#include "IKRigStretchLimb.generated.h"

struct FIKRigGoal;

#define UE_API IKRIG_API

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.StretchLimbBoneSettingsWrapper"))
struct FIKRigStretchLimbBoneSettings : public FIKRigBoneSettingsBase
{
	GENERATED_BODY()

	FIKRigStretchLimbBoneSettings() = default;
	FIKRigStretchLimbBoneSettings(FName InBoneName) { Bone = InBoneName; }
	
	/** The direction to push this bone when the limb is squashed. This is relative to the local bone axes.
	 * NOTE: This direction is used by the "Squash Strength" feature of the stretch limb solver.
	 * If no custom Squash direction is specified, the solver will push the bone in the direction away from its projection onto the pole vector.
	 * If the bone is lying directly on the pole vector, the default squash direction is undefined and the bone may mot squash in a deterministic manner.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Preferred Squash Direction")
	FVector SquashDirection = FVector::ZAxisVector;

	int32 CachedChainIndex;
};

UENUM(BlueprintType)
enum class EStretchLimbSquashMode : uint8
{
	None,
	Uniform,
	Bulge
};

UENUM(BlueprintType)
enum class EStretchLimbRotationMode : uint8
{
	None,
	OrientToGoal,
};

USTRUCT(BlueprintType, meta=(UIWrapper="/Script/IKRigEditor.StretchLimbSettingsWrapper"))
struct FIKRigStretchLimbSettings : public FIKRigSolverSettingsBase
{
	GENERATED_BODY()

	/** The start bone assigned to this solver (usually the thigh or shoulder). */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Limb Setup")
	FName StartBone;

	/** The end bone assigned to this solver (usually the foot or ball). */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Limb Setup")
	FName EndBone;

	/** The end bone assigned to this solver (usually the foot or ball). */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "Limb Setup")
	FName Goal;
	
	/** Determines whether to squash or stretch the bones to reach the goal. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Stretch Settings")
	bool bEnableStretching = true;

	/** The maximum distance the limb is allowed to stretch (beyond its rest length). Default is -1.0 (no limit).*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Stretch Settings")
	double MaximumStretchDistance = -1.0;

	/** The percentage of the bone chain length to straighten before stretching is applied (can prevent over extension).
	 * NOTE: At 1.0, the limb will not stretch until the goal is 100% of the length of the limb away from the root of the chain. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Stretch Settings", meta = (ClampMin = "0.1", ClampMax = "1.0", UIMin = "0.1", UIMax = "1.0"))
	double StretchStartPercent = 0.95;

	/** Determines how to affect the orientation of the bones in the chain. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Rotation Settings")
	EStretchLimbRotationMode RotationMode = EStretchLimbRotationMode::OrientToGoal;

	/** Allow the end bone to be reoriented (0) or match the orientation of the goal (1). Range is 0-1. Default is 1.0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Rotation Settings", meta = (UIMin = "0.0", UIMax = "1.0"))
	double RotateEndBoneWithGoal = 1.0f;

	/** Number of FABRIK iterations to correct bones lengths. Adjust until the end bone converges on the goal. Default is 8. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Constraint Solving", meta = (UIMin = "0", UIMax = "20.0"))
	int32 Iterations = 8;

	/** Control the falloff shape of the squash effect applied to the bones when the goal compresses the limb. Default is Uniform.
	 * NOTE: this has no effect if the Squash Strength is 0. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Constraint Solving")
	EStretchLimbSquashMode SquashMode = EStretchLimbSquashMode::Uniform;
	
	/** The distance to squash the bones perpendicular to the pole vector when the limb is fully compressed.
	 * NOTE: tune this to help dense chains achieve a reasonable pose. Must be used with constraint iterations to fix bones lengths. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Constraint Solving", meta = (UIMin = "0", UIMax = "200.0"))
	double SquashStrength = 50;
};

USTRUCT(BlueprintType)
struct FIKRigStretchLimbSolver : public FIKRigSolverBase
{
	GENERATED_BODY()
	
	// FIKRigSolverBase interface
	UE_API virtual void Initialize(const FIKRigSkeleton& InIKRigSkeleton) override;
	UE_API virtual void Solve(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoalContainer& InGoals) override;
	UE_API virtual void GetRequiredBones(TSet<FName>& OutRequiredBones) const override;
	UE_API virtual void GetRequiredGoals(TSet<FName>& OutRequiredGoals) const override;
	
	// solver settings
	UE_API virtual FIKRigSolverSettingsBase* GetSolverSettings() override;
	UE_API virtual const UScriptStruct* GetSolverSettingsType() const override;
	
	// goals
	UE_API virtual void AddGoal(const UIKRigEffectorGoal& InNewGoal) override;
	UE_API virtual void OnGoalRemoved(const FName& InGoalName) override;
	UE_API virtual void OnGoalRenamed(const FName& InOldName, const FName& InNewName) override;
	UE_API virtual void OnGoalMovedToDifferentBone(const FName& InGoalName, const FName& InNewBoneName) override;

	// goal settings
	virtual bool UsesCustomGoalSettings() const override { return false; };

	// bone settings
	UE_API virtual bool UsesCustomBoneSettings() const override;
	UE_API virtual void AddSettingsToBone(const FName& InBoneName) override;
	UE_API virtual void RemoveSettingsOnBone(const FName& InBoneName) override;
	UE_API virtual FIKRigBoneSettingsBase* GetBoneSettings(const FName& InBoneName) override;
	UE_API virtual const UScriptStruct* GetBoneSettingsType() const override;
	UE_API virtual bool HasSettingsOnBone(const FName& InBoneName) const override;
	UE_API virtual void GetBonesWithSettings(TSet<FName>& OutBonesWithSettings) const override;
	// END bone settings
	
	// start bone can be set on this solver
	virtual bool UsesStartBone() const override { return true; };
	virtual FName GetStartBone() const override { return Settings.StartBone; };
	UE_API virtual void SetStartBone(const FName& InRootBoneName) override;

#if WITH_EDITOR
	// return custom controller for scripting this solver
	virtual UIKRigSolverControllerBase* GetSolverController(UObject* Outer) override;
	// UI stuff
	virtual FText GetNiceName() const override;
	virtual bool GetWarningMessage(FText& OutWarningMessage) const override;
	virtual bool IsBoneAffectedBySolver(const FName& InBoneName, const FIKRigSkeleton& InIKRigSkeleton) const override;
#endif

	UPROPERTY()
	FIKRigStretchLimbSettings Settings;

	UPROPERTY()
	TArray<FIKRigStretchLimbBoneSettings> AllBoneSettings;

private:

	void AimChainAtGoal(FIKRigSkeleton& InIKRigSkeleton, const FVector& GoalLocation) const;

	void StoreBonePositions(FIKRigSkeleton& InIKRigSkeleton);
	void UpdateBoneOrientations(FIKRigSkeleton& InIKRigSkeleton);
	
	void SquashOrStretchChainParallel(FIKRigSkeleton& InIKRigSkeleton, const FVector& GoalLocation) const;
	void SquashChainPerpendicular(FIKRigSkeleton& InIKRigSkeleton);
	void StretchChainFinal(FIKRigSkeleton& InIKRigSkeleton, const FVector& GoalLocation) const;
	void ApplyParallelOffsetToChain(FIKRigSkeleton& InIKRigSkeleton, const FVector& ChainStart, const FVector& NewPoleVector, const FVector& CurrentPoleVector) const;

	void RunFABRIK(FIKRigSkeleton& InIKRigSkeleton, const FVector& GoalLocation);
	void RunFABRIKBackwardPass(const FVector& Goal, TArray<FVector>& BonePositions);
	void RunFABRIKForwardPass(const FVector& Start, TArray<FVector>& BonePositions);

	void UpdateFKChildren(FIKRigSkeleton& InIKRigSkeleton);

	void RotateEndBoneWithGoal(FIKRigSkeleton& InIKRigSkeleton, const FIKRigGoal* Goal);
	
	double CalculateCurrentChainLength(FIKRigSkeleton& InIKRigSkeleton) const;
	
	void UpdateRestLengths(FIKRigSkeleton& InIKRigSkeleton);

	bool bIsInitialized = false;
	TArray<int32> BoneIndices;
	TArray<int32> ChildrenToUpdate;

	TArray<FVector> BonePositionsPreOffset;
	TArray<double> RestLengths;
	double TotalRestLength;
	TArray<double> PoleVectorParams;
	double InitialPoleVectorLenSq;
};

#undef UE_API
