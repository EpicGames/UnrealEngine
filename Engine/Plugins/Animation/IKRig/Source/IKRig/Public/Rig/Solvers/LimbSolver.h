// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IKRigSolverBase.h"
#include "LimbSolver.generated.h"

USTRUCT(BlueprintType)
struct FLimbSolverSettings : public FIKRigSolverSettingsBase
{
	GENERATED_BODY()

	// make this polymorphic
	virtual ~FLimbSolverSettings() = default;

	/** Precision (distance to the target) */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings", meta = (ClampMin = "0.0"))
	float ReachPrecision = 0.01f;
	
	// TWO BONES SETTINGS
	
	/** Hinge Bones Rotation Axis. This is essentially the plane normal for (hip - knee - foot). */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|Two Bones")
	TEnumAsByte<EAxis::Type> HingeRotationAxis = EAxis::None;

	// FABRIK SETTINGS

	/** Number of Max Iterations to reach the target */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|FABRIK", meta = (ClampMin = "0", UIMin = "0", UIMax = "100"))
	int32 MaxIterations = 12;

	/** Enable/Disable rotational limits */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|FABRIK|Limits")
	bool bEnableLimit = false;

	/** Only used if bEnableRotationLimit is enabled. Prevents the leg from folding onto itself,
	* and forces at least this angle between Parent and Child bone. */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|FABRIK|Limits", meta = (EditCondition="bEnableLimit", ClampMin = "0.0", ClampMax = "90.0", UIMin = "0.0", UIMax = "90.0"))
	float MinRotationAngle = 15.f;

	/** Pull averaging only has a visual impact when we have more than 2 bones (3 links). */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|FABRIK|Pull Averaging")
	bool bAveragePull = true;

	/** Re-position limb to distribute pull: 0 = foot, 0.5 = balanced, 1.f = hip */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|FABRIK|Pull Averaging", meta = (ClampMin = "0.0", ClampMax = "1.0",UIMin = "0.0", UIMax = "1.0"))
	float PullDistribution = 0.5f;
	
	/** Move end effector towards target. If we are compressing the chain, limit displacement.*/
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|FABRIK|Pull Averaging", meta = (ClampMin = "0.0", ClampMax = "1.0",UIMin = "0.0", UIMax = "1.0"))
	float ReachStepAlpha = 0.7f;

	// TWIST SETTINGS
	
	/** Enable Knee Twist correction, by comparing Foot FK with Foot IK orientation. */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|Twist")
	bool bEnableTwistCorrection = false;
	
	/** Forward Axis for Foot bone. */
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings|Twist", meta = (EditCondition="bEnableTwistCorrection"))
	TEnumAsByte<EAxis::Type> EndBoneForwardAxis = EAxis::Y;
};

USTRUCT()
struct FLimbLink
{
	GENERATED_USTRUCT_BODY()

	// Location of bone in component space.
	FVector Location;

	// Distance to its parent
	float Length;

	// Bone Index in SkeletalMesh
	int32 BoneIndex;

	// Axis utilites
	FVector LinkAxisZ;
	FVector RealBendDir;
	FVector BaseBendDir;

	FLimbLink()
		: Location(FVector::ZeroVector)
		, Length(0.f)
		, BoneIndex( INDEX_NONE )
		, LinkAxisZ(FVector::ZeroVector)
		, RealBendDir(FVector::ZeroVector)
		, BaseBendDir(FVector::ZeroVector)
	{}

	FLimbLink(const FVector& InLocation, int32 InBoneIndex)
		: Location(InLocation)
		, Length(0.f)
		, BoneIndex(InBoneIndex)
		, LinkAxisZ(FVector::ZeroVector)
		, RealBendDir(FVector::ZeroVector)
		, BaseBendDir(FVector::ZeroVector)
	{}
};

USTRUCT()
struct FLimbSolver
{
	GENERATED_USTRUCT_BODY()

	void Reset();
	bool Initialize();
	bool Solve(
		TArray<FTransform>& InOutTransforms,
		const FVector& InGoalLocation,
		const FQuat& InGoalRotation,
		const FLimbSolverSettings& InSettings);

	int32 AddLink(const FVector& InLocation, int32 InBoneIndex);
	int32 GetBoneIndex(const int32 Index) const;
	int32 NumLinks() const;
	
private:
	
	void UpdateTransforms(TArray<FTransform>& InOutTransforms);
	bool OrientTransformsTowardsGoal(TArray<FTransform>& InOutTransforms, const FVector& InGoalLocation) const;
	bool AdjustKneeTwist(TArray<FTransform>& InOutTransforms, const FTransform& InGoalTransform, const EAxis::Type FootBoneForwardAxis );

	float Length = 0.f;
	
	TArray<FLimbLink> Links;
};

namespace AnimationCore
{
	/**
	* Limb solver
	*
	* This solves a hybrid TwoBoneIK / FABRIK based on the number of links within the chain
	*
	* @param	InOutLinks			Array of chain links
	* @param	LimbLength			The full length on the limb (sum of all links' length)
	* @param	GoalLocation		The location of the IK target
	* @param	Precision			Precision 
	* @param	InMaxIterations		Number of Max Iteration (used for FABRIK only)
	* @param	HingeRotationAxis	Hinge Bones Rotation Axis. This is essentially the plane normal for (hip - knee - foot). (used for TwoBones only)
	* @param	bUseAngleLimit		Enable/Disable rotational limits (used for FABRIK only)
	* @param	MinRotationAngle	Prevents the leg from folding onto itself, and forces at least this angle between Parent and Child bone (used for FABRIK only)
	* @param	ReachStepAlpha		Move end effector towards target. If we are compressing the chain, limit displacement. (used for FABRIK only)
	* * @param	bAveragePull		Pull averaging only has a visual impact when we have more than 2 bones (3 links). (used for FABRIK only)
	* @param	PullDistribution	Re-position limb to distribute pull: 0 = foot, 0.5 = balanced, 1.f = hip. (used for FABRIK only)
	* @return  true if modified. False if not.
	*/
	bool SolveLimb(
		TArray<FLimbLink>& InOutLinks,
		float LimbLength,
		const FVector& GoalLocation,
		float Precision,
		int32 InMaxIterations,
		const FVector& HingeRotationAxis,
		bool bUseAngleLimit,
		float MinRotationAngle,
		float ReachStepAlpha,
		bool bAveragePull,
		float PullDistribution);
}
