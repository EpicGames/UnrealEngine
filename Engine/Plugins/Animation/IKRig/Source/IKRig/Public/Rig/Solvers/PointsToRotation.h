// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Quat.h"
#include "Math/RotationMatrix.h"

// given a set of points in an initial configuration and the same set of points in a deformed configuration,
// this function outputs a quaternion that represents the "best fit" rotation that rotates the initial points to the
// current points.
static FQuat GetRotationFromDeformedPoints(
	const TArray<FVector>& InInitialPoints,
	const TArray<FVector>& InCurrentPoints,
	FVector& OutInitialCentroid,
	FVector& OutCurrentCentroid)
{
	check(InInitialPoints.Num() == InCurrentPoints.Num());

	// must have more than 1 point to generate a gradient
	if (InInitialPoints.Num() <= 1)
	{
		OutInitialCentroid = InInitialPoints.Num() > 0 ? InInitialPoints[0] : FVector::ZeroVector;
		OutCurrentCentroid = InCurrentPoints.Num() > 0 ? InCurrentPoints[0] : FVector::ZeroVector;
		return FQuat::Identity;
	}
	
	// calculate initial and current centroids
	OutInitialCentroid = FVector::ZeroVector;
	OutCurrentCentroid = FVector::ZeroVector;
	for (int32 PointIndex=0; PointIndex<InInitialPoints.Num(); ++PointIndex)
	{
		OutInitialCentroid += InInitialPoints[PointIndex];
		OutCurrentCentroid += InCurrentPoints[PointIndex];
	}

	// average centroids
	const double InvNumEffectors = 1.0f / static_cast<double>(InInitialPoints.Num());
	OutInitialCentroid *= InvNumEffectors;
	OutCurrentCentroid *= InvNumEffectors;

	// accumulate deformation gradient to extract a rotation from
	FVector DX = FVector::ZeroVector; // DX, DY, DZ are rows of 3x3 deformation gradient tensor
	FVector DY = FVector::ZeroVector;
	FVector DZ = FVector::ZeroVector;
	for (int32 PointIndex=0; PointIndex<InInitialPoints.Num(); ++PointIndex)
	{
		// accumulate the deformation gradient tensor for all points
		// "Meshless Deformations Based on Shape Matching"
		// Equation 7 describes accumulation of deformation gradient from points
		//
		// P is normalized vector from INITIAL centroid to INITIAL point
		// Q is normalized vector from CURRENT centroid to CURRENT point
		FVector P = (InInitialPoints[PointIndex] - OutInitialCentroid).GetSafeNormal();
		FVector Q = (InCurrentPoints[PointIndex] - OutCurrentCentroid).GetSafeNormal();
		// PQ^T is the outer product of P and Q which is a 3x3 matrix
		// https://en.m.wikipedia.org/wiki/Outer_product
		DX += FVector(P[0]*Q[0], P[0]*Q[1], P[0]*Q[2]);
		DY += FVector(P[1]*Q[0], P[1]*Q[1], P[1]*Q[2]);
		DZ += FVector(P[2]*Q[0], P[2]*Q[1], P[2]*Q[2]);
	}

	// extract "best fit" rotation from deformation gradient
	FQuat Q = FQuat::Identity;
	constexpr int32 MaxIter = 50;
	// "A Robust Method to Extract the Rotational Part of Deformations" equation 7
	// https://matthias-research.github.io/pages/publications/stablePolarDecomp.pdf
	for (unsigned int Iter = 0; Iter < MaxIter; Iter++)
	{
		FMatrix R = FRotationMatrix::Make(Q);
		FVector RCol0(R.M[0][0], R.M[0][1], R.M[0][2]);
		FVector RCol1(R.M[1][0], R.M[1][1], R.M[1][2]);
		FVector RCol2(R.M[2][0], R.M[2][1], R.M[2][2]);
		FVector Omega = RCol0.Cross(DX) + RCol1.Cross(DY) + RCol2.Cross(DZ);
		Omega *= 1.0 / (fabs(RCol0.Dot(DX) + RCol1.Dot(DY) + RCol2.Dot(DZ)) + UE_SMALL_NUMBER);
		const double W = Omega.Size();
		if (W < UE_SMALL_NUMBER)
		{
			break;
		}
		Q = FQuat(FQuat((1.0 / W) * Omega, W)) * Q;
		Q.Normalize();
	}

	return Q;
}