// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class IKRigDeveloper : ModuleRules
    {
        public IKRigDeveloper(ReadOnlyTargetRules Target) : base(Target)
        {
	        PrivateDependencyModuleNames.AddRange(
                new string[]
                {
					"Core",
					"CoreUObject",
					"Engine",
					"IKRig",
                }
            );

            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                }
            );

            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[]
                    {
                        "UnrealEd",
                        "AnimGraph",
                        "BlueprintGraph",
                        "Slate",
                        "SlateCore",
                        "Persona",
                        "StructUtilsEditor",
                    }
                );
            }
        }
    }
}
