// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_IKRig.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Rig/IKRigDefinition.h"
#include "Rig/Solvers/IKRigSolverBase.h"
#include "Kismet2/CompilerResultsLog.h"
#include "AnimationGraphSchema.h"
#include "ScopedTransaction.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include "BoneSelectionWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_IKRig)

/////////////////////////////////////////////////////
// UAnimGraphNode_IKRig 

#define LOCTEXT_NAMESPACE "AnimGraphNode_IKRig"

/////////////////////////////////////////////////////
// FIKRigGoalLayout

TSharedRef<SWidget> FIKRigGoalLayout::CreatePropertyWidget() const
{
	const TSharedPtr<IPropertyHandle> TransformSourceHandle = GoalPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRigGoal, TransformSource));

	// transform source combo box
	return SNew(SHorizontalBox)
	
	+ SHorizontalBox::Slot()
	.FillWidth(1.0f)
	.HAlign(HAlign_Left)
	[
		TransformSourceHandle->CreatePropertyValueWidget()
	];
}

TSharedRef<SWidget> FIKRigGoalLayout::CreateBoneValueWidget() const
{
	const TSharedPtr<IPropertyHandle> TransformSourceHandle = GoalPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRigGoal, TransformSource));
	
	return SNew(SHorizontalBox)

	// transform source combo box
	+ SHorizontalBox::Slot()
	.FillWidth(1.0f)
	.HAlign(HAlign_Left)
	[
		TransformSourceHandle->CreatePropertyValueWidget()
	]

	// bone selector
	+ SHorizontalBox::Slot()
	.AutoWidth()
	.HAlign(HAlign_Left)
	.Padding(3.f, 0.f)
	[
		SNew(SBoneSelectionWidget)
		.OnBoneSelectionChanged(this, &FIKRigGoalLayout::OnBoneSelectionChanged)
		.OnGetSelectedBone(this, &FIKRigGoalLayout::GetSelectedBone)
		.OnGetReferenceSkeleton(this, &FIKRigGoalLayout::GetReferenceSkeleton)
	];
}

TSharedRef<SWidget> FIKRigGoalLayout::CreateValueWidget() const
{
	const EIKRigGoalTransformSource TransformSource = GetTransformSource();
	if (TransformSource == EIKRigGoalTransformSource::Bone)
	{
		return CreateBoneValueWidget();
	}

	return CreatePropertyWidget();
}

void FIKRigGoalLayout::GenerateHeaderRowContent(FDetailWidgetRow& InOutGoalRow)
{
	InOutGoalRow
	.NameContent()
	[
		SNew(STextBlock)
		.Text(FText::FromName(GetName()))
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		CreateValueWidget()
	];
}

void FIKRigGoalLayout::GenerateChildContent(IDetailChildrenBuilder& InOutChildrenBuilder)
{
	const EIKRigGoalTransformSource TransformSource = GetTransformSource();
	if (TransformSource == EIKRigGoalTransformSource::Manual)
	{
		if (bExposePosition)
		{
			const TSharedPtr<IPropertyHandle> PosSpaceHandle = GoalPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRigGoal, PositionSpace));
			IDetailPropertyRow& PropertyRow = InOutChildrenBuilder.AddProperty(PosSpaceHandle.ToSharedRef());
			
			// Hide the reset to default button since it provides little value
			const FResetToDefaultOverride	ResetDefaultOverride = FResetToDefaultOverride::Create(TAttribute<bool>(false));
			PropertyRow.OverrideResetToDefault(ResetDefaultOverride);
		}

		if (bExposeRotation)
		{
			const TSharedPtr<IPropertyHandle> RotSpaceHandle = GoalPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRigGoal, RotationSpace));
			IDetailPropertyRow& PropertyRow = InOutChildrenBuilder.AddProperty(RotSpaceHandle.ToSharedRef());

			// Hide the reset to default button since it provides little value
			const FResetToDefaultOverride	ResetDefaultOverride = FResetToDefaultOverride::Create(TAttribute<bool>(false));
			PropertyRow.OverrideResetToDefault(ResetDefaultOverride);
		}
	}
}

FName FIKRigGoalLayout::GetGoalName(TSharedPtr<IPropertyHandle> InGoalHandle)
{
	if (!InGoalHandle.IsValid() || !InGoalHandle->IsValidHandle())
	{
		return  NAME_None;
	}
	
	const TSharedPtr<IPropertyHandle> NameHandle = InGoalHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRigGoal, Name));
	if (!NameHandle.IsValid() || !NameHandle->IsValidHandle())
	{
		return  NAME_None;
	}

	FName GoalName;
	NameHandle->GetValue(GoalName);
	return GoalName;
}

FName FIKRigGoalLayout::GetName() const
{
	return GetGoalName(GoalPropHandle);
}

EIKRigGoalTransformSource FIKRigGoalLayout::GetTransformSource() const
{
	if (!GoalPropHandle->IsValidHandle())
	{
		return EIKRigGoalTransformSource::Manual;
	}
	
	const TSharedPtr<IPropertyHandle> TransformSourceHandle = GoalPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRigGoal, TransformSource));
	if (!TransformSourceHandle->IsValidHandle())
	{
		return EIKRigGoalTransformSource::Manual;
	}
	
	uint8 Source;
	TransformSourceHandle->GetValue(Source);
	return static_cast<EIKRigGoalTransformSource>(Source);
}

TSharedPtr<IPropertyHandle> FIKRigGoalLayout::GetBoneNameHandle() const
{
	if (!GoalPropHandle->IsValidHandle())
	{
		return nullptr;
	}
	
	const TSharedPtr<IPropertyHandle> SourceBoneHandle = GoalPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRigGoal, SourceBone));
	if (!SourceBoneHandle->IsValidHandle())
	{
		return nullptr;
	}

	return SourceBoneHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FBoneReference, BoneName));
}

void FIKRigGoalLayout::OnBoneSelectionChanged(FName Name) const
{
	const TSharedPtr<IPropertyHandle> BoneNameProperty = GetBoneNameHandle();
	if (BoneNameProperty.IsValid() && BoneNameProperty->IsValidHandle())
	{
		BoneNameProperty->SetValue(Name);
	}
}

FName FIKRigGoalLayout::GetSelectedBone(bool& bMultipleValues) const
{
	const TSharedPtr<IPropertyHandle> BoneNameProperty = GetBoneNameHandle();
	if (!BoneNameProperty.IsValid() || !BoneNameProperty->IsValidHandle())
	{
		return NAME_None;
	}
	
	FString OutName;
	const FPropertyAccess::Result Result = BoneNameProperty->GetValueAsFormattedString(OutName);
	bMultipleValues = (Result == FPropertyAccess::MultipleValues);

	return FName(*OutName);
}

const struct FReferenceSkeleton& FIKRigGoalLayout::GetReferenceSkeleton() const
{
	static const FReferenceSkeleton DummySkeleton;
	
	if (!GoalPropHandle->IsValidHandle())
	{
		return DummySkeleton;
	}
	
	const TSharedPtr<IPropertyHandle> SourceBoneHandle = GoalPropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRigGoal, SourceBone));
	if (!SourceBoneHandle->IsValidHandle())
	{
		return DummySkeleton;
	}
	
	TArray<UObject*> Objects;
	SourceBoneHandle->GetOuterObjects(Objects);

	USkeleton* TargetSkeleton = nullptr;

	auto FindSkeletonForObject = [&TargetSkeleton](UObject* InObject)
	{
		for( ; InObject; InObject = InObject->GetOuter())
		{
			if (UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(InObject))
			{
				TargetSkeleton = AnimGraphNode->GetAnimBlueprint()->TargetSkeleton;
				break;
			}
		}

		return TargetSkeleton != nullptr;
	};
	
	for (UObject* Object : Objects)
	{
		if(FindSkeletonForObject(Object))
		{
			break;
		}
	}
	
	return TargetSkeleton ? TargetSkeleton->GetReferenceSkeleton() : DummySkeleton;
}

/////////////////////////////////////////////////////
// FIKRigGoalArrayLayout

void FIKRigGoalArrayLayout::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (NodePropHandle.IsValid())
	{
		const UObject* Object = nullptr;
		const TSharedPtr<IPropertyHandle> RigDefAssetHandle = NodePropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, RigDefinitionAsset));
		if (RigDefAssetHandle->GetValue(Object) == FPropertyAccess::Fail || Object == nullptr)
		{
			return;
		}

		const UIKRigDefinition* IKRigDefinition = CastChecked<const UIKRigDefinition>(Object);
		if (!IKRigDefinition)
		{
			return;
		}

		const TSharedPtr<IPropertyHandle> GoalsHandle = NodePropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, Goals));
		const TArray<UIKRigEffectorGoal*>& AssetGoals = IKRigDefinition->GetGoalArray();

		// add customization for each goal
		uint32 NumGoals = 0;
		GoalsHandle->GetNumChildren(NumGoals);
		for (uint32 Index = 0; Index < NumGoals; Index++)
		{
			TSharedPtr<IPropertyHandle> GoalHandle = GoalsHandle->GetChildHandle(Index);
			if (GoalHandle.IsValid())
			{
				const int32 AssetGoalIndex = AssetGoals.IndexOfByPredicate([&GoalHandle](const UIKRigEffectorGoal* InAssetGoal)
				{
					return FIKRigGoalLayout::GetGoalName(GoalHandle) == InAssetGoal->GoalName;
				});

				if (AssetGoalIndex != INDEX_NONE)
				{
					const UIKRigEffectorGoal* AssetGoal = AssetGoals[AssetGoalIndex];
					if (AssetGoal->bExposePosition || AssetGoal->bExposeRotation)
					{
						TSharedRef<FIKRigGoalLayout> ControlRigArgumentLayout = MakeShareable(
						   new FIKRigGoalLayout(GoalHandle, AssetGoal->bExposePosition, AssetGoal->bExposeRotation));
						ChildrenBuilder.AddCustomBuilder(ControlRigArgumentLayout);
					}
				}
			}
		}
	}
}

UAnimGraphNode_IKRig::~UAnimGraphNode_IKRig()
{
	if (OnAssetPropertyChangedHandle.IsValid())
	{
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnAssetPropertyChangedHandle);
		OnAssetPropertyChangedHandle.Reset();
	}
}

void UAnimGraphNode_IKRig::Serialize(FArchive& Ar)
{
	// Handle change of default blend type
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID);
		if (CustomVersion < FFortniteMainBranchObjectVersion::ChangeDefaultAlphaBlendType)
		{
			// Switch the default back to Linear so old data remains the same
			// Important: this is done before loading so if data has changed from default it still works
			Node.AlphaBoolBlend.BlendOption = EAlphaBlendOption::Linear;
		}
	}
	
	Super::Serialize(Ar);
}

void UAnimGraphNode_IKRig::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const
{
	if(PreviewSkelMeshComp)
	{
		if(FAnimNode_IKRig* ActiveNode = GetActiveInstanceNode<FAnimNode_IKRig>(PreviewSkelMeshComp->GetAnimInstance()))
		{
			ActiveNode->ConditionalDebugDraw(PDI, PreviewSkelMeshComp);
		}
	}
}

FText UAnimGraphNode_IKRig::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("AnimGraphNode_IKRig_Title", "IK Rig");
}

void UAnimGraphNode_IKRig::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_IKRig* IKRigNode = static_cast<FAnimNode_IKRig*>(InPreviewNode);
}

void UAnimGraphNode_IKRig::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	if (!IsValid(Node.RigDefinitionAsset))
	{
		MessageLog.Warning(*LOCTEXT("NoRigDefinitionAsset", "@@ - Please select a Rig Definition Asset.").ToString(), this);
	}
}

UObject* UAnimGraphNode_IKRig::GetJumpTargetForDoubleClick() const
{
	return Node.RigDefinitionAsset;
}

void UAnimGraphNode_IKRig::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{	
	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, RigDefinitionAsset))
	{
		Node.Goals.Empty();
		if (IsValid(Node.RigDefinitionAsset))
		{
			// create new goals based on the rig definition
			const TArray<UIKRigEffectorGoal*>& AssetGoals = Node.RigDefinitionAsset->GetGoalArray();
			for (const UIKRigEffectorGoal* AssetGoal: AssetGoals)
			{
				const int32 GoalIndex = Node.Goals.Emplace(AssetGoal->GoalName, AssetGoal->BoneName);
				SetupGoal(AssetGoal, Node.Goals[GoalIndex]);
			}

			BindPropertyChanges();
		}
		ReconstructNode();
		return;
	}
	
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, Goals))
	{
		ReconstructNode();
		return;
	}
	
	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, AlphaInputType))
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeAlphaInputType", "Change Alpha Input Type"));
		Modify();

		// Break links to pins going away
		for (int32 PinIndex = 0; PinIndex < Pins.Num(); ++PinIndex)
		{
			UEdGraphPin* Pin = Pins[PinIndex];
			if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, Alpha))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Float)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, bAlphaBoolEnabled))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
			else if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, AlphaCurveName))
			{
				if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
				{
					Pin->BreakAllPinLinks();
					RemoveBindings(Pin->PinName);
				}
			}
		}

		ReconstructNode();
	}

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FIKRigGoal, TransformSource))
	{
		ReconstructNode();
		return;
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FName GetGoalSubPropertyPinPrettyName(const FName& InGoalName, const FName& InPropertyName)
{
	return *FString::Printf(TEXT("%s_%s"), *InGoalName.ToString(), *InPropertyName.ToString());
}

FName GetGoalSubPropertyPinName(const FName& InGoalName, const FName& InPropertyName)
{
	return *FString::Printf(TEXT("%s_%s"), *InPropertyName.ToString(), *InGoalName.ToString());
}

void UAnimGraphNode_IKRig::CreateCustomPins(TArray<UEdGraphPin*>* InOldPins)
{
	if (!IsValid(Node.RigDefinitionAsset))
	{
		return;
	}
	
	// the asset is not completely loaded so we'll use the old pins to sustain the current set of custom pins
	if (Node.RigDefinitionAsset->HasAllFlags(RF_NeedPostLoad)) 
	{
		CreateCustomPinsFromUnloadedAsset(InOldPins);
		return;
	}

	// generate pins based on the current asset
	CreateCustomPinsFromValidAsset();
}

void UAnimGraphNode_IKRig::SetPinDefaultValue(UEdGraphPin* InPin, const FName& InPropertyName)
{
	LLM_SCOPE_BYNAME(TEXT("Animation/IKRig"));
	
	// default FIKRigGoal structure 
	static FIKRigGoal DefaultGoal;
	static const TSharedPtr<FStructOnScope> StructOnScope =
							MakeShareable(new FStructOnScope(FIKRigGoal::StaticStruct(), (uint8*)&DefaultGoal));

	// NOTE: we bypass the ExportTextItem for FVector and FRotator structures resulting in invalid pin's default value
	// and use custom functions. We can not use ToString() neither here since the vector/rotator pin control
	// doesn't use the standard 'X=0,Y=0,Z=0' syntax. This is seen in several other places so this should factorized.
	// We may not notice it's actually not working as the default value we set is generally a structure formed by 0s
	// but it becomes obvious if trying with a Vector set to (1.0, 2.0, 3.0) for instance.
	auto GetDefaultValue = [](const FProperty* InProperty)
	{
		const uint8* Memory = InProperty->ContainerPtrToValuePtr<uint8>(StructOnScope->GetStructMemory());
		if (Memory)
		{
			if (const FStructProperty* StructProp = CastField<FStructProperty>(InProperty))
			{
				if (StructProp->Struct == TBaseStructure<FVector>::Get())
				{
					const FVector& Vector = *(FVector*)Memory;
					return FString::Printf(TEXT("%3.3f,%3.3f,%3.3f"), Vector.X, Vector.Y, Vector.Z);
				}
			
				if (StructProp->Struct == TBaseStructure<FRotator>::Get())
				{
					const FRotator& Rotator = *(FRotator*)Memory;
					return FString::Printf(TEXT("%3.3f,%3.3f,%3.3f"), Rotator.Pitch, Rotator.Yaw, Rotator.Roll);
				}
			}
		}

		// fallback to default behaviour 
		FString DefaultValue;
		InProperty->ExportTextItem_Direct(DefaultValue, Memory, nullptr, nullptr, PPF_None);
		return DefaultValue;
	};
	
	// set pin's initial default value
	const UAnimationGraphSchema* AnimGraphDefaultSchema = GetDefault<UAnimationGraphSchema>();
	AnimGraphDefaultSchema->SetPinAutogeneratedDefaultValueBasedOnType(InPin);
	if (const FProperty* Property = FIKRigGoal::StaticStruct()->FindPropertyByName(InPropertyName))
	{
		const FString DefaultValue = GetDefaultValue(Property);
		if (!DefaultValue.IsEmpty())
		{
			AnimGraphDefaultSchema->TrySetDefaultValue(*InPin, DefaultValue);
		}
	}
}

void UAnimGraphNode_IKRig::CreateCustomPinsFromUnloadedAsset(TArray<UEdGraphPin*>* InOldPins)
{
	// recreate pin from old pin
	auto RecreateGoalPin = [this](const UEdGraphPin* InOldPin)
	{
		// pin's name is based on the property name within the FIKRigGoal structure + the index within the Goals array
		const FName PropertyName = InOldPin->GetFName();
		
		UEdGraphPin* NewPin = CreatePin(EEdGraphPinDirection::EGPD_Input, InOldPin->PinType, PropertyName);

		// pin's pretty name is the "GoalName_InPropertyName"
		NewPin->PinFriendlyName = InOldPin->PinFriendlyName;

		// default value
		SetPinDefaultValue(NewPin, PropertyName);
	};

	// ensure that this is a goal related pin
	auto bNeedsCreation = [&](const UEdGraphPin* InOldPin)
	{
		// in our case, custom pins are all inputs
		if (InOldPin->Direction != EEdGraphPinDirection::EGPD_Input)
		{
			return false;
		}

		// do not rebuild sub pins as they will be treated after in UK2Node::RestoreSplitPins
		if (InOldPin->ParentPin && InOldPin->ParentPin->bHidden)
		{
		 	return false;
		}
		
		// look for old pin's name-type into current pins. At this stage, the default pins have already been created
		const int32 PinIndex = Pins.IndexOfByPredicate([&](const UEdGraphPin* Pin)
		{
			return Pin->GetFName() == InOldPin->GetFName() && Pin->PinType == InOldPin->PinType;
		});
		
		return PinIndex == INDEX_NONE;
	};

	// recreate pins if needed
	if (InOldPins)
	{
		for (const UEdGraphPin* OldPin : *InOldPins)
		{
			if (bNeedsCreation(OldPin))
			{
				RecreateGoalPin(OldPin);
			}
		}
	}
}

void UAnimGraphNode_IKRig::CreateCustomPinsFromValidAsset()
{
	// pin's creation function
	auto CreateGoalPin = [this]( const uint32 InGoalIndex,
								 const FName& InPropertyName,
								 const FEdGraphPinType InPinType)
	{
		const FName& GoalName = Node.Goals[InGoalIndex].Name;
				
		UEdGraphPin* NewPin = CreatePin(EEdGraphPinDirection::EGPD_Input, InPinType, GetGoalSubPropertyPinName(GoalName, InPropertyName));
		
		// pin's pretty name is the "GoalName_InPropertyName"
		NewPin->PinFriendlyName = FText::FromName(GetGoalSubPropertyPinPrettyName(GoalName, InPropertyName));

		// default value
		SetPinDefaultValue(NewPin, InPropertyName);
	};

	static const FName PC_Struct(TEXT("struct"));
	
	// position property
	static FEdGraphPinType PositionPinType;
	PositionPinType.PinCategory = PC_Struct;
	PositionPinType.PinSubCategoryObject = TBaseStructure<FVector>::Get();

	// rotation property
	static FEdGraphPinType RotationPinType;
	RotationPinType.PinCategory = PC_Struct;
	RotationPinType.PinSubCategoryObject = TBaseStructure<FRotator>::Get();
	
	// alpha property
	static const FName PC_Real(TEXT("real"));
	static const FName PC_Float(TEXT("float"));
	static const FName PC_Double(TEXT("double"));
	static FEdGraphPinType AlphaPinType;
	AlphaPinType.PinCategory = PC_Real;
	AlphaPinType.PinSubCategory = PC_Double;

	// create pins
	const TArray<UIKRigEffectorGoal*>& AssetGoals = Node.RigDefinitionAsset->GetGoalArray();
	const TArray<FIKRigGoal>& Goals = Node.Goals;
	for (int32 GoalIndex = 0; GoalIndex < Goals.Num(); GoalIndex++)
	{
		const FIKRigGoal& Goal = Node.Goals[GoalIndex];
		
		const int32 AssetGoalIndex = AssetGoals.IndexOfByPredicate([&Goal](const UIKRigEffectorGoal* InAssetGoal)
		{
			return Goal.Name == InAssetGoal->GoalName;
		});

		if (AssetGoalIndex == INDEX_NONE)
		{
			continue;
		}

		const UIKRigEffectorGoal* AssetGoal = AssetGoals[AssetGoalIndex];
		if (Goal.TransformSource == EIKRigGoalTransformSource::Manual)
		{
			// position
			if (AssetGoal->bExposePosition)
			{
				CreateGoalPin(GoalIndex, GET_MEMBER_NAME_CHECKED(FIKRigGoal, Position), PositionPinType);
				CreateGoalPin(GoalIndex, GET_MEMBER_NAME_CHECKED(FIKRigGoal, PositionAlpha), AlphaPinType);
			}

			// rotation
			if (AssetGoal->bExposeRotation)
			{
				CreateGoalPin(GoalIndex, GET_MEMBER_NAME_CHECKED(FIKRigGoal, Rotation), RotationPinType);
				CreateGoalPin(GoalIndex, GET_MEMBER_NAME_CHECKED(FIKRigGoal, RotationAlpha), AlphaPinType);
			}
		}
		else if (Goal.TransformSource == EIKRigGoalTransformSource::Bone)
		{
			// position
			if (AssetGoal->bExposePosition)
			{
				CreateGoalPin(GoalIndex, GET_MEMBER_NAME_CHECKED(FIKRigGoal, PositionAlpha), AlphaPinType);
			}

			// rotation
			if (AssetGoal->bExposeRotation)
			{
				CreateGoalPin(GoalIndex, GET_MEMBER_NAME_CHECKED(FIKRigGoal, RotationAlpha), AlphaPinType);
			}
		}
	}
}

void UAnimGraphNode_IKRig::PostLoad()
{
	Super::PostLoad();

	// update goals name if needed
	if (IsValid(Node.RigDefinitionAsset))
	{
		//NOTE needed?
		const TArray<UIKRigEffectorGoal*>& AssetGoals = Node.RigDefinitionAsset->GetGoalArray();
	
		const int32 NumAssetGoals = AssetGoals.Num();
		const int32 NumNodeGoals = Node.Goals.Num();
		if (NumAssetGoals == NumNodeGoals)
		{
			for (int32 Index = 0; Index < NumNodeGoals; Index++)
			{
				FName& GoalName = Node.Goals[Index].Name;
				if (GoalName.IsNone())
				{
					GoalName = AssetGoals[Index]->GoalName;
				}
				SetupGoal(AssetGoals[Index], Node.Goals[Index]);
			}
		}

		// listened to changes within the asset / goals
		BindPropertyChanges();
	}
}

void UAnimGraphNode_IKRig::PostEditUndo()
{
	Super::PostEditUndo();

	UpdateGoalsFromAsset();
}

void UAnimGraphNode_IKRig::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	// Do not allow multi-selection
	if (DetailBuilder.GetSelectedObjects().Num() > 1)
	{
		return;
	}

	// Add goals customization
	const TSharedRef<IPropertyHandle> NodePropHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UAnimGraphNode_IKRig, Node), GetClass());
	if (NodePropHandle->IsValidHandle())
	{
		TSharedRef<FIKRigGoalArrayLayout> InputArgumentGroup = MakeShareable(new FIKRigGoalArrayLayout(NodePropHandle));

		IDetailCategoryBuilder& GoalsCategoryBuilder = DetailBuilder.EditCategory(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, Goals));
		GoalsCategoryBuilder.AddCustomBuilder(InputArgumentGroup);
	}
	// Hide normal goals properties
	DetailBuilder.HideCategory("Goal");

	// Handle property changed notification
	const FSimpleDelegate OnValueChanged = FSimpleDelegate::CreateLambda([&DetailBuilder]()
	{
		DetailBuilder.ForceRefreshDetails();
	});
	
	const TSharedRef<IPropertyHandle> AssetHandle = DetailBuilder.GetProperty(TEXT("Node.RigDefinitionAsset"), GetClass());
	if (AssetHandle->IsValidHandle())
	{
		AssetHandle->SetOnPropertyValueChanged(OnValueChanged);
	}
	
	const TSharedRef<IPropertyHandle> GoalHandle = DetailBuilder.GetProperty(TEXT("Node.Goals"), GetClass());
	if (AssetHandle->IsValidHandle())
	{
		GoalHandle->SetOnChildPropertyValueChanged(OnValueChanged);
	}

	// alpha customization
	if (Node.AlphaInputType != EAnimAlphaInputType::Bool)
	{
		DetailBuilder.HideProperty(NodePropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, bAlphaBoolEnabled)));
		DetailBuilder.HideProperty(NodePropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, AlphaBoolBlend)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Float)
	{
		DetailBuilder.HideProperty(NodePropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, Alpha)));
		DetailBuilder.HideProperty(NodePropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, AlphaScaleBias)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Curve)
	{
		DetailBuilder.HideProperty(NodePropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, AlphaCurveName)));
	}

	if (Node.AlphaInputType != EAnimAlphaInputType::Float && (Node.AlphaInputType != EAnimAlphaInputType::Curve))
	{
		DetailBuilder.HideProperty(NodePropHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FAnimNode_IKRig, AlphaScaleBiasClamp)));
	}
}

void UAnimGraphNode_IKRig::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, Alpha))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Float);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBias.GetFriendlyName(Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName));
		}
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, bAlphaBoolEnabled))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Bool);
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_IKRig, AlphaCurveName))
	{
		Pin->bHidden = (Node.AlphaInputType != EAnimAlphaInputType::Curve);

		if (!Pin->bHidden)
		{
			Pin->PinFriendlyName = Node.AlphaScaleBiasClamp.GetFriendlyName(Pin->PinFriendlyName);
		}
	}
}

void UAnimGraphNode_IKRig::BindPropertyChanges()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/IKRig"));
	
	// already bound
	if (OnAssetPropertyChangedHandle.IsValid())
	{
		return;	
	}

	// listen to the rig definition asset
	using FPropertyChangedDelegate = FCoreUObjectDelegates::FOnObjectPropertyChanged::FDelegate;
	const FPropertyChangedDelegate OnPropertyChangedDelegate =
				FPropertyChangedDelegate::CreateUObject(this, &UAnimGraphNode_IKRig::OnPropertyChanged);
	
	OnAssetPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.Add(OnPropertyChangedDelegate);
}

void UAnimGraphNode_IKRig::OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent)
{
	if (NeedsUpdate(ObjectBeingModified, PropertyChangedEvent))
	{
		UpdateGoalsFromAsset();
	}
}

bool UAnimGraphNode_IKRig::NeedsUpdate(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent) const
{
	if (!ObjectBeingModified)
	{
		return false;
	}
	
	if (!IsValid(Node.RigDefinitionAsset))
	{
		return false;
	}

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == NAME_None)
	{
		return false;
	}

	// something has changed within the asset
	if (ObjectBeingModified == Node.RigDefinitionAsset)
	{
		// we can't use GET_MEMBER_NAME_CHECKED as Goals is a private property
		static const TArray<FName> AssetWatchedProperties({ TEXT("Goals") } );

		const bool bNeedsUpdate = AssetWatchedProperties.Contains(PropertyName);
		return bNeedsUpdate;
	}

	// check whether this is a goal and if it belongs to the current asset
	if (UIKRigEffectorGoal* GoalBeingModified = Cast<UIKRigEffectorGoal>(ObjectBeingModified))
	{
		static const TArray<FName> GoalWatchedProperties({	GET_MEMBER_NAME_CHECKED(UIKRigEffectorGoal, GoalName),
															GET_MEMBER_NAME_CHECKED(UIKRigEffectorGoal, bExposePosition),
															GET_MEMBER_NAME_CHECKED(UIKRigEffectorGoal, bExposeRotation)} );
		
		const TArray<UIKRigEffectorGoal*>& AssetGoals = Node.RigDefinitionAsset->GetGoalArray();
		const bool bNeedsUpdate = AssetGoals.Contains(GoalBeingModified) && GoalWatchedProperties.Contains(PropertyName);
		return bNeedsUpdate;
	}
	
	return false;
}

void UAnimGraphNode_IKRig::UpdateGoalsFromAsset()
{
	TArray<FIKRigGoal> OldGoals = Node.Goals;
	Node.Goals.Empty();

	if (IsValid(Node.RigDefinitionAsset))
	{
		const TArray<UIKRigEffectorGoal*>& AssetGoals = Node.RigDefinitionAsset->GetGoalArray();
		for (const UIKRigEffectorGoal* AssetGoal: AssetGoals)
		{
			const int32 OldGoalIndex = OldGoals.IndexOfByPredicate([&AssetGoal](const FIKRigGoal& OldGoal)
			{
				return AssetGoal->GoalName == OldGoal.Name;
			});
			
			if (OldGoalIndex != INDEX_NONE)
			{
				Node.Goals.Add(OldGoals[OldGoalIndex]);
			}
			else
			{
				Node.Goals.Emplace(AssetGoal->GoalName, AssetGoal->BoneName);
			}
			
			SetupGoal(AssetGoal, Node.Goals.Last());
		}
	}
	
	ReconstructNode();
}

void UAnimGraphNode_IKRig::SetupGoal(const UIKRigEffectorGoal* InAssetGoal, FIKRigGoal& OutGoal)
{
	// setup hidden goals so that they keep the bone's translation / rotation
	if (InAssetGoal->GoalName == OutGoal.Name)
	{
		if (!InAssetGoal->bExposePosition)
		{
			OutGoal.Position = FVector::Zero();
			OutGoal.PositionAlpha = 0.f;
			OutGoal.PositionSpace = EIKRigGoalSpace::Additive;
		}

		if (!InAssetGoal->bExposeRotation)
		{
			OutGoal.Rotation = FQuat::Identity.Rotator();
			OutGoal.RotationAlpha = 0.f;
			OutGoal.RotationSpace = EIKRigGoalSpace::Additive;
		}
	}
}

#undef LOCTEXT_NAMESPACE

