// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RetargetPoseFromMesh.h"

#include "PropertyBagDetails.h"
#include "Animation/AnimInstance.h"
#include "Kismet2/CompilerResultsLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RetargetPoseFromMesh)

#define LOCTEXT_NAMESPACE "AnimGraphNode_IKRig"
const FName UAnimGraphNode_RetargetPoseFromMesh::AnimModeName(TEXT("IKRig.IKRigEditor.IKRigEditMode"));

void UAnimGraphNode_RetargetPoseFromMesh::Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const
{
}

FText UAnimGraphNode_RetargetPoseFromMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	if (Node.RetargetFrom == ERetargetSourceMode::SourcePosePin)
	{
		return LOCTEXT("AnimGraphNode_IKRetargeter_Title_FromInput", "Retarget Input Pose");
	}
	
	return LOCTEXT("AnimGraphNode_IKRetargeter_Title", "Retarget Pose From Mesh");
}

void UAnimGraphNode_RetargetPoseFromMesh::CopyNodeDataToPreviewNode(FAnimNode_Base* InPreviewNode)
{
	FAnimNode_RetargetPoseFromMesh* IKRetargeterNode = static_cast<FAnimNode_RetargetPoseFromMesh*>(InPreviewNode);
}

FEditorModeID UAnimGraphNode_RetargetPoseFromMesh::GetEditorMode() const
{
	return AnimModeName;
}

void UAnimGraphNode_RetargetPoseFromMesh::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RetargetPoseFromMesh, SourceMeshComponent))
	{
		Pin->bHidden = Node.RetargetFrom != ERetargetSourceMode::CustomSkeletalMeshComponent;
	}

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RetargetPoseFromMesh, Source))
	{
		const bool bCopyingFromOtherComponent = Node.RetargetFrom != ERetargetSourceMode::SourcePosePin;
		Pin->bHidden = bCopyingFromOtherComponent;
		Pin->bNotConnectable = bCopyingFromOtherComponent;
	}
}

void UAnimGraphNode_RetargetPoseFromMesh::CreateCustomPins(TArray<UEdGraphPin*>* InOldPins)
{
	Super::CreateCustomPins(InOldPins);

	// if we don't have an asset, we can't have custom variable pins
	if (!IsValid(Node.IKRetargeterAsset))
	{
		return;
	}

	// handle the 'PostLoad' race condition (if ABP is loaded before the asset)
	if (Node.IKRetargeterAsset->HasAllFlags(RF_NeedPostLoad)) 
	{
		CreateCustomPinsFromUnloadedAsset(InOldPins);
		return;
	}

	// asset is valid and loaded, generate current pins
	CreateCustomPinsFromValidAsset();
}

void UAnimGraphNode_RetargetPoseFromMesh::AddVariablesToDetailsPanel(IDetailLayoutBuilder& DetailBuilder)
{
	// NOTE: this lists all the variables in the asset and adds/removes from the TArray<FName> ExposedVariableNames
	// Variables stored in ExposedVariableNames will become pins on the anim node.

	
	UIKRetargeter* RetargeterAsset = Node.IKRetargeterAsset;
	if (!RetargeterAsset)
	{
		return;
	}

	const FInstancedPropertyBag& Bag = RetargeterAsset->GetVariables().Bag;
	const UPropertyBag* BagStruct = Bag.GetPropertyBagStruct();
	if (!BagStruct)
	{
		return;
	}

	// create a dedicated category for pin exposure
	IDetailCategoryBuilder& PinCategory = DetailBuilder.EditCategory("Variable Pins", LOCTEXT("VariablePins", "Variable Pins"));

	for (const FPropertyBagPropertyDesc& Desc : BagStruct->GetPropertyDescs())
	{
		const FName VarName = Desc.Name;

		PinCategory.AddCustomRow(FText::FromName(VarName))
		.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromName(VarName))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([this, VarName]()
			{
				return ExposedVariableNames.Contains(VarName) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([this, VarName](ECheckBoxState NewState)
			{
				if (NewState == ECheckBoxState::Checked)
				{
					ExposedVariableNames.Add(VarName);
				}
				else
				{
					ExposedVariableNames.Remove(VarName);
				}
				
				// force the node to recreate pins to show/hide the change immediately
				ReconstructNode();
			})
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ExposePinLabel", "Expose as Pin"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];
	}
}

void UAnimGraphNode_RetargetPoseFromMesh::CreateCustomPinsFromValidAsset()
{
	const FInstancedPropertyBag& Bag = Node.IKRetargeterAsset->GetVariables().Bag;
	const UPropertyBag* BagStruct = Bag.GetPropertyBagStruct();
	const uint8* BagMemory = Bag.GetValue().GetMemory();

	if (!BagStruct || !BagMemory)
	{
		return;
	}
	
	for (const FName& VarName : ExposedVariableNames)
	{
		const FPropertyBagPropertyDesc* Desc = BagStruct->FindPropertyDescByName(VarName);
		if (!Desc)
		{
			continue;
		}
		
		FEdGraphPinType PinType = UE::StructUtils::GetPropertyDescAsPin(*Desc);
		UEdGraphPin* NewPin = CreatePin(EGPD_Input, PinType, VarName);
		NewPin->PinFriendlyName = FText::FromName(VarName);
		
		// extract the default value from the asset and assign it to the pin
		// NOTE: assigning a default value is necessary to avoid BP compile errors
		if (FProperty* BagProp = BagStruct->FindPropertyByName(VarName))
		{
			const void* ValuePtr = BagProp->ContainerPtrToValuePtr<void>(BagMemory);
			
			FString DefaultString;
			BagProp->ExportTextItem_Direct(DefaultString, ValuePtr, nullptr, nullptr, PPF_None);
			NewPin->DefaultValue = DefaultString;
		}
	}
}

void UAnimGraphNode_RetargetPoseFromMesh::CreateCustomPinsFromUnloadedAsset(TArray<UEdGraphPin*>* InOldPins)
{
	if (!InOldPins)
	{
		return;
	}

	for (UEdGraphPin* OldPin : *InOldPins)
	{
		// only recreate pins that belonged to our dynamic variable set
		if (OldPin->Direction == EGPD_Input && ExposedVariableNames.Contains(OldPin->GetFName()))
		{
			// check if this pin already exists (avoid duplicates during reconstruction)
			if (!FindPin(OldPin->GetFName(), EGPD_Input))
			{
				CreatePin(EGPD_Input, OldPin->PinType, OldPin->GetFName());
			}
		}
	}
}

UObject* UAnimGraphNode_RetargetPoseFromMesh::GetJumpTargetForDoubleClick() const
{
	return Node.IKRetargeterAsset;
}

void UAnimGraphNode_RetargetPoseFromMesh::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent); // required for UAnimGraphNode_CustomProperty base

	const FName PropertyName = (PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None);
	if ((PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RetargetPoseFromMesh, RetargetFrom)))
	{
		ReconstructNode();
	}
	else if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RetargetPoseFromMesh, IKRetargeterAsset))
	{
		ReconstructNode();
	}
}

void UAnimGraphNode_RetargetPoseFromMesh::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);

	// validate source mesh component is not null
	if (Node.RetargetFrom == ERetargetSourceMode::CustomSkeletalMeshComponent)
	{
		const bool bIsLinked = IsPinExposedAndLinked(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RetargetPoseFromMesh, SourceMeshComponent));
		const bool bIsBound = IsPinExposedAndBound(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RetargetPoseFromMesh, SourceMeshComponent));
		if (!(bIsLinked || bIsBound))
		{
			MessageLog.Error(TEXT("@@ requires a source Skeletal Mesh Component to be plugged in."), this);
			return;
		}
	}

	// validate IK Rig asset has been assigned
	if (!Node.IKRetargeterAsset)
	{
		UEdGraphPin* Pin = FindPin(GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RetargetPoseFromMesh, IKRetargeterAsset));
		if (Pin == nullptr)
		{
			// retarget asset unassigned
			MessageLog.Error(TEXT("@@ does not have an IK Retargeter asset assigned."), this);
		}
		return;
	}

	// validate SOURCE IK Rig asset has been assigned
	const UIKRigDefinition* SourceIKRig = Node.IKRetargeterAsset->GetIKRig(ERetargetSourceOrTarget::Source);
	if (!SourceIKRig)
	{
		MessageLog.Warning(TEXT("@@ has an IK Retargeter that is missing a source IK Rig asset."), this);
	}

	// validate TARGET IK Rig asset has been assigned
	const UIKRigDefinition* TargetIKRig = Node.IKRetargeterAsset->GetIKRig(ERetargetSourceOrTarget::Target);
	if (!TargetIKRig)
	{
		MessageLog.Warning(TEXT("@@ has an IK Retargeter that is missing a target IK Rig asset."), this);
	}

	if (!(SourceIKRig && TargetIKRig))
	{
		return;
	}

	// pull messages out of the processor's log
	if (!Node.bSuppressWarnings)
	{
		if (const FIKRetargetProcessor* Processor = Node.GetRetargetProcessor())
		{
			const TArray<FText>& Warnings = Processor->Log.GetWarnings();
			for (const FText& Warning : Warnings)
			{
				MessageLog.Warning(*Warning.ToString());
			}
			
			const TArray<FText>& Errors = Processor->Log.GetErrors();
			for (const FText& Error : Errors)
			{
				MessageLog.Error(*Error.ToString());
			}
		}
	}

	if (ForSkeleton && !Node.bSuppressWarnings)
	{
		// validate that target bone chains exist on this skeleton
		const FReferenceSkeleton &RefSkel = ForSkeleton->GetReferenceSkeleton();
		const TArray<FBoneChain> &TargetBoneChains = TargetIKRig->GetRetargetChains();
		for (const FBoneChain &Chain : TargetBoneChains)
		{
			if (RefSkel.FindBoneIndex(Chain.StartBone.BoneName) == INDEX_NONE)
			{
				MessageLog.Warning(*FText::Format(LOCTEXT("StartBoneNotFound", "@@ - Start Bone '{0}' in target IK Rig Bone Chain not found."), FText::FromName(Chain.StartBone.BoneName)).ToString(), this);
			}

			if (RefSkel.FindBoneIndex(Chain.EndBone.BoneName) == INDEX_NONE)
			{
				MessageLog.Warning(*FText::Format(LOCTEXT("EndBoneNotFound", "@@ - End Bone '{0}' in target IK Rig Bone Chain not found."), FText::FromName(Chain.EndBone.BoneName)).ToString(), this);
			}
		}
	}
}

void UAnimGraphNode_RetargetPoseFromMesh::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	Super::CustomizeDetails(DetailBuilder);

	AddVariablesToDetailsPanel(DetailBuilder);
}

void UAnimGraphNode_RetargetPoseFromMesh::PreloadRequiredAssets()
{
	Super::PreloadRequiredAssets();
	
	if (Node.IKRetargeterAsset)
	{
		PreloadObject(Node.IKRetargeterAsset);
		PreloadObject(Node.IKRetargeterAsset->GetIKRigWriteable(ERetargetSourceOrTarget::Source));
		PreloadObject(Node.IKRetargeterAsset->GetIKRigWriteable(ERetargetSourceOrTarget::Target));
	}
}

#undef LOCTEXT_NAMESPACE