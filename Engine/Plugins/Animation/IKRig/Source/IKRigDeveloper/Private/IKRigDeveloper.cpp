// Copyright Epic Games, Inc. All Rights Reserved.

#include "IKRigDeveloper.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "IKRigDeveloperModule"

IMPLEMENT_MODULE(FIKRigDeveloperModule, IKRigDeveloper)

void FIKRigDeveloperModule::StartupModule()
{
}

void FIKRigDeveloperModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
