// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimGraphNode_CustomProperty.h"
#include "AnimNodes/AnimNode_IKRig.h"
#include "IDetailCustomNodeBuilder.h"

#include "AnimGraphNode_IKRig.generated.h"

#define UE_API IKRIGDEVELOPER_API

class FPrimitiveDrawInterface;
class USkeletalMeshComponent;

/////////////////////////////////////////////////////
// FIKRigGoalLayout

class FIKRigGoalLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FIKRigGoalLayout>
{
public:
	
	FIKRigGoalLayout(TSharedPtr<IPropertyHandle> InGoalPropHandle,
					 const bool InExposePosition,
					 const bool InExposeRotation)
		: GoalPropHandle(InGoalPropHandle)
		, bExposePosition(InExposePosition)
		, bExposeRotation(InExposeRotation)
	{}

	/** IDetailCustomNodeBuilder Interface*/
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& InOutGoalRow) override;
	virtual void GenerateChildContent(IDetailChildrenBuilder& InOutChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override;
	virtual bool InitiallyCollapsed() const override { return true; }

	static FName GetGoalName(TSharedPtr<IPropertyHandle> InGoalHandle);
	
private:

	EIKRigGoalTransformSource GetTransformSource() const;

	const struct FReferenceSkeleton& GetReferenceSkeleton() const;
	TSharedPtr<IPropertyHandle> GetBoneNameHandle() const;
	void OnBoneSelectionChanged(FName Name) const;
	FName GetSelectedBone(bool& bMultipleValues) const;

	TSharedRef<SWidget> CreatePropertyWidget() const;
	TSharedRef<SWidget> CreateBoneValueWidget() const;
	TSharedRef<SWidget> CreateValueWidget() const;
	
	TSharedPtr<IPropertyHandle> GoalPropHandle = nullptr;
	bool bExposePosition = false;
	bool bExposeRotation = false;
};

/////////////////////////////////////////////////////
// FIKRigGoalArrayLayout

class FIKRigGoalArrayLayout : public IDetailCustomNodeBuilder, public TSharedFromThis<FIKRigGoalArrayLayout>
{
public:
	
	FIKRigGoalArrayLayout(TSharedPtr<IPropertyHandle> InNodePropHandle)
		: NodePropHandle(InNodePropHandle)
	{}
	
	virtual ~FIKRigGoalArrayLayout() = default;

	/** IDetailCustomNodeBuilder Interface*/
	virtual void SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren) override {}
	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override {}
	virtual void GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder) override;
	virtual void Tick(float DeltaTime) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual FName GetName() const override { return NAME_None; }
	virtual bool InitiallyCollapsed() const override { return false; }

private:
	
	TSharedPtr<IPropertyHandle> NodePropHandle;
};

// Editor node for IKRig 
UCLASS(MinimalAPI)
class UAnimGraphNode_IKRig : public UAnimGraphNode_CustomProperty
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_IKRig Node;

public:

	UAnimGraphNode_IKRig() = default;
	UE_API virtual ~UAnimGraphNode_IKRig();

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;
	// End of UObject

	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void CreateCustomPins(TArray<UEdGraphPin*>* InOldPins) override;
	UE_API virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	UE_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	UE_API virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* AnimNode) override;
	UE_API virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const override;
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	// End of UAnimGraphNode_Base interface

	// UK2Node interface
	UE_API UObject* GetJumpTargetForDoubleClick() const;
	// END UK2Node

	// Begin UObject Interface.
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostEditUndo() override;
	// End UObject Interface.

	virtual bool NeedsToSpecifyValidTargetClass() const override { return false; }
	
protected:
	
	virtual FAnimNode_CustomProperty* GetCustomPropertyNode() override { return &Node; }
	virtual const FAnimNode_CustomProperty* GetCustomPropertyNode() const override { return &Node; }

private:

	// set pin's default value based on the FIKRigGoal default struct
	static UE_API void SetPinDefaultValue(UEdGraphPin* InPin, const FName& InPropertyName);

	// create custom pins from exposed goals as defined in the rig definition asset 
	UE_API void CreateCustomPinsFromValidAsset();

	// recreate custom pins from old pins if the rig definition asset is not completely loaded
	UE_API void CreateCustomPinsFromUnloadedAsset(TArray<UEdGraphPin*>* InOldPins);

	// Handle to the registered delegate
	FDelegateHandle OnAssetPropertyChangedHandle;

	// Global callback to anticipate on changes to the asset / goals
	UE_API bool NeedsUpdate(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent) const;
	UE_API void OnPropertyChanged(UObject* ObjectBeingModified, FPropertyChangedEvent& PropertyChangedEvent);
	UE_API void BindPropertyChanges();

	// update the goals' array within the anim node based on the asset
	UE_API void UpdateGoalsFromAsset();

	// setup goal based on it's asset definition 
	static UE_API void SetupGoal(const UIKRigEffectorGoal* InAssetGoal, FIKRigGoal& OutGoal);
};

#undef UE_API
