// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_CustomProperty.h"
#include "AnimNodes/AnimNode_RetargetPoseFromMesh.h"
#include "AnimGraphNode_RetargetPoseFromMesh.generated.h"

#define UE_API IKRIGDEVELOPER_API

class FPrimitiveDrawInterface;
class USkeletalMeshComponent;

UCLASS(MinimalAPI)
class UAnimGraphNode_RetargetPoseFromMesh : public UAnimGraphNode_CustomProperty
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_RetargetPoseFromMesh Node;

public:
	// UEdGraphNode interface
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	UE_API virtual void PreloadRequiredAssets() override;
	UE_API virtual void CopyNodeDataToPreviewNode(FAnimNode_Base* AnimNode) override;
	UE_API virtual FEditorModeID GetEditorMode() const override;
	UE_API virtual void Draw(FPrimitiveDrawInterface* PDI, USkeletalMeshComponent* PreviewSkelMeshComp) const override;
	UE_API virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	virtual void CreateCustomPins(TArray<UEdGraphPin*>* OldPins) override;
	virtual bool UsingCopyPoseFromMesh() const override { return true; };
	// End of UAnimGraphNode_Base interface

	// UK2Node interface
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	// End of UK2Node interface

	static UE_API const FName AnimModeName;

private:
	// UAnimGraphNode_CustomProperty interface
	virtual FAnimNode_CustomProperty* GetCustomPropertyNode() override { return &Node; }
	virtual const FAnimNode_CustomProperty* GetCustomPropertyNode() const override { return &Node; }
	// return nullptr here to completely disable the base class's automatic pin manager
	// we are manually handling the UI and Pin generation ourselves via CustomizeDetails
	virtual UStruct* GetTargetStruct() const override { return nullptr; }
	virtual UStruct* GetTargetSkeletonStruct() const override { return nullptr; }
	virtual bool NeedsToSpecifyValidTargetClass() const override { return false; }
	// End of UAnimGraphNode_CustomProperty interface

	void AddVariablesToDetailsPanel(IDetailLayoutBuilder& DetailBuilder);
	void CreateCustomPinsFromValidAsset();
	void CreateCustomPinsFromUnloadedAsset(TArray<UEdGraphPin*>* InOldPins);
	
	// Set of variable names from the asset that should be exposed as input pins
	UPROPERTY()
	TSet<FName> ExposedVariableNames;
};

#undef UE_API