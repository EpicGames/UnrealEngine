// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API IKRIGDEVELOPER_API

class FIKRigDeveloperModule : public IModuleInterface
{
public:
	UE_API void StartupModule() override;
	UE_API void ShutdownModule() override;
};

#undef UE_API
