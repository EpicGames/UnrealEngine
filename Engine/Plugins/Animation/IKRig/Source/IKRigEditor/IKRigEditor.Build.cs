// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
    public class IKRigEditor : ModuleRules
    {
        public IKRigEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			
			PublicDependencyModuleNames.AddRange(
                new string[]
                {
	                "Core",
	                "AdvancedPreviewScene",
                }
            );

			PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "InputCore",
                    "EditorFramework",
                    "UnrealEd",
                    "ToolMenus",
                    "CoreUObject",
                    "Engine",
                    "Slate",
                    "SlateCore",
                    "AssetTools",
                    "EditorWidgets",
                    "Kismet",
                    "KismetWidgets",
                    "Persona",
                    "SkeletonEditor",
                    "ContentBrowser",
                    "AnimationBlueprintLibrary",
                    "MessageLog",
                    "StructUtilsEditor",

                    "PropertyEditor",
					"BlueprintGraph",
					"AnimGraph",
					"AnimGraphRuntime",
					
					"IKRig",
					"IKRigDeveloper",
					"ToolWidgets",
					"AnimationCore",
                    "AnimationWidgets",
                    "ApplicationCore",
					"MeshDescription",
					"SkeletalMeshDescription"
                }
            );
        }
    }
}
