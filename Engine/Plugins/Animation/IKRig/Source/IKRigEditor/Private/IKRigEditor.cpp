// Copyright Epic Games, Inc. All Rights Reserved.

#include "IKRigEditor.h"
#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"
#include "AssetToolsModule.h"
#include "PropertyEditorModule.h"
#include "PropertyEditorDelegates.h"
#include "EditorModeRegistry.h"
#include "Rig/IKRigDefinition.h"
#include "RigEditor/AssetTypeActions_IKRigDefinition.h"
#include "RetargetEditor/AssetTypeActions_IKRetargeter.h"
#include "RetargetEditor/IKRetargetCommands.h"
#include "RetargetEditor/IKRetargetDefaultMode.h"
#include "RetargetEditor/IKRetargetDetails.h"
#include "RetargetEditor/IKRetargetEditPoseMode.h"
#include "RetargetEditor/IKRetargeterThumbnailRenderer.h"
#include "RetargetEditor/IKRetargetOpDetails.h"
#include "RetargetEditor/IKRetargetOverrideCustomization.h"
#include "Retargeter/IKRetargetOps.h"
#include "RigEditor/IKRigCommands.h"
#include "RigEditor/IKRigEditMode.h"
#include "RigEditor/IKRigSkeletonCommands.h"
#include "RigEditor/IKRigDetailCustomizations.h"
#include "RigEditor/IKRigEditorController.h"
#include "RigEditor/IKRigThumbnailRenderer.h"

DEFINE_LOG_CATEGORY(LogIKRigEditor);

IMPLEMENT_MODULE(FIKRigEditor, IKRigEditor)

#define LOCTEXT_NAMESPACE "IKRigEditor"

void FIKRigEditor::StartupModule()
{
	// register commands
	FIKRigCommands::Register();
	FIKRigSkeletonCommands::Register();
	FIKRetargetCommands::Register();
	
	// register custom asset type actions
	IAssetTools& ToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	//
	IKRigDefinitionAssetAction = MakeShareable(new FAssetTypeActions_IKRigDefinition);
	ToolsModule.RegisterAssetTypeActions(IKRigDefinitionAssetAction.ToSharedRef());
	//
	IKRetargeterAssetAction = MakeShareable(new FAssetTypeActions_IKRetargeter);
	ToolsModule.RegisterAssetTypeActions(IKRetargeterAssetAction.ToSharedRef());

	// extend the content browser menu
	FAssetTypeActions_IKRetargeter::ExtendAnimAssetMenusForBatchRetargeting();
	FAssetTypeActions_IKRetargeter::ExtendIKRigMenuToMakeRetargeter();
	FAssetTypeActions_IKRigDefinition::ExtendSkeletalMeshMenuToMakeIKRig();

	// register custom editor modes
	FEditorModeRegistry::Get().RegisterMode<FIKRigEditMode>(FIKRigEditMode::ModeName, LOCTEXT("IKRigEditMode", "IKRig"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FIKRetargetDefaultMode>(FIKRetargetDefaultMode::ModeName, LOCTEXT("IKRetargetDefaultMode", "IKRetargetDefault"), FSlateIcon(), false);
	FEditorModeRegistry::Get().RegisterMode<FIKRetargetEditPoseMode>(FIKRetargetEditPoseMode::ModeName, LOCTEXT("IKRetargetEditMode", "IKRetargetEditPose"), FSlateIcon(), false);

	// register detail customizations
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	// custom instanced struct details
	PropertyEditorModule.RegisterCustomClassLayout(
		UIKRigStructViewer::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FIKRigStructViewerCustomization::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UIKRigStructViewer::StaticClass()->GetFName());
	PropertyEditorModule.RegisterCustomClassLayout(
		UIKRigStructWrapperBase::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FIKRigStructWrapperCustomization::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UIKRigStructWrapperBase::StaticClass()->GetFName());
	// custom IK rig bone details
	PropertyEditorModule.RegisterCustomClassLayout(
		UIKRigBoneDetails::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FIKRigGenericDetailCustomization::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UIKRigBoneDetails::StaticClass()->GetFName());
	// custom IK rig goal details
	PropertyEditorModule.RegisterCustomClassLayout(
		UIKRigEffectorGoal::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FIKRigGenericDetailCustomization::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UIKRigEffectorGoal::StaticClass()->GetFName());
	// custom retargeter bone details
	PropertyEditorModule.RegisterCustomClassLayout(
		UIKRetargetBoneDetails::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FIKRetargetBoneDetailCustomization::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(UIKRetargetBoneDetails::StaticClass()->GetFName());
	// retarget op details
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
		"IKRetargetFKChainsOpSettings",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChainsFKOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
		"IKRetargetRunIKRigOpSettings",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRunIKRigOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
		"IKRetargetIKChainsOpSettings",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FIKChainOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
		"IKRetargetStrideWarpingOpSettings",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStrideWarpOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
		"IKRetargetSpeedPlantingOpSettings",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSpeedPlantOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
		"IKRetargetAlignPoleVectorOpSettings",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPoleVectorOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
			"IKRetargetAdditivePoseOpSettings",
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FAdditivePoseOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
			"IKRetargetStretchChainOpSettings",
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStretchChainOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
			"IKRetargetFloorConstraintOpSettings",
			FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FFloorConstraintOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
				"IKRetargetBlendToSourceOpSettings",
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlendToSourceOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
				"IKRetargetOffsetGoalsOpSettings",
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FOffsetGoalsOpCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
				"IKRetargetScaleGoalsOpSettings",
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FScaleGoalsOpCustomization::MakeInstance));
	// retarget profile details
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(
				"RetargetOverrideSet",
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRetargetOverrideCustomization::MakeInstance));

	// register a thumbnail renderer for the assets
	UThumbnailManager::Get().RegisterCustomRenderer(UIKRigDefinition::StaticClass(), UIKRigThumbnailRenderer::StaticClass());
	UThumbnailManager::Get().RegisterCustomRenderer(UIKRetargeter::StaticClass(), UIKRetargeterThumbnailRenderer::StaticClass());
}

void FIKRigEditor::ShutdownModule()
{
	FIKRigCommands::Unregister();
	FIKRigSkeletonCommands::Unregister();
	FIKRetargetCommands::Unregister();
	
	FEditorModeRegistry::Get().UnregisterMode(FIKRigEditMode::ModeName);
	FEditorModeRegistry::Get().UnregisterMode(FIKRetargetDefaultMode::ModeName);
	FEditorModeRegistry::Get().UnregisterMode(FIKRetargetEditPoseMode::ModeName);

	// unregister asset actions
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		IAssetTools& ToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		
		if (IKRigDefinitionAssetAction.IsValid())
		{
			ToolsModule.UnregisterAssetTypeActions(IKRigDefinitionAssetAction.ToSharedRef());
		}
		
		if (IKRetargeterAssetAction.IsValid())
		{
			ToolsModule.UnregisterAssetTypeActions(IKRetargeterAssetAction.ToSharedRef());
		}
	}

	if (UObjectInitialized())
	{
		UThumbnailManager::Get().UnregisterCustomRenderer(UIKRigDefinition::StaticClass());
		UThumbnailManager::Get().UnregisterCustomRenderer(UIKRetargeter::StaticClass());
	}
}

#undef LOCTEXT_NAMESPACE