// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/AssetTypeActions_IKRetargeter.h"

#include "Animation/AnimationAsset.h"
#include "ContentBrowserMenuContexts.h"
#include "ToolMenus.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/IKRetargetEditorStyle.h"
#include "RetargetEditor/IKRetargetFactory.h"
#include "RetargetEditor/SRetargetAnimAssetsWindow.h"
#include "Retargeter/IKRetargeter.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "RigEditor/IKRigEditorStyle.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

UClass* FAssetTypeActions_IKRetargeter::GetSupportedClass() const
{
	return UIKRetargeter::StaticClass();
}

void FAssetTypeActions_IKRetargeter::OpenAssetEditor(
	const TArray<UObject*>& InObjects,
	TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
	const EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;
    
	for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
	{
		if (UIKRetargeter* Asset = Cast<UIKRetargeter>(*ObjIt))
		{
			TSharedRef<FIKRetargetEditor> NewEditor(new FIKRetargetEditor());
			NewEditor->InitAssetEditor(Mode, EditWithinLevelEditor, Asset);
		}
	}
}

UThumbnailInfo* FAssetTypeActions_IKRetargeter::GetThumbnailInfo(UObject* Asset) const
{
	UIKRetargeter* IKRetargeter = CastChecked<UIKRetargeter>(Asset);
	return NewObject<USceneThumbnailInfo>(IKRetargeter, NAME_None, RF_Transactional);
}

void FAssetTypeActions_IKRetargeter::ExtendIKRigMenuToMakeRetargeter()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.IKRigDefinition");
	if (Menu == nullptr)
	{
		return;
	}

	FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
	Section.AddDynamicEntry("CreateIKRetargeter", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
		if (Context)
		{
			if (Context->SelectedAssets.Num() > 0)
			{
				InSection.AddMenuEntry(
					"CreateRetargeter",
					LOCTEXT("CreateIKRetargeter", "Create IK Retargeter"),
					LOCTEXT("CreateIKRetargeter_ToolTip", "Creates an IK Retargeter using this IK Rig as the source and target."),
					FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(), "IKRig", "ClassIcon.IKRigDefinition"),
					FExecuteAction::CreateLambda([InSelectedAssets = Context->SelectedAssets]()
					{
						TArray<UObject*> SelectedObjects;
						SelectedObjects.Reserve(InSelectedAssets.Num());

						for (const FAssetData& Asset : InSelectedAssets)
						{
							if (UObject* LoadedAsset = Asset.GetAsset())
							{
								SelectedObjects.Add(LoadedAsset);
							}
						}

						for (UObject* SelectedObject : SelectedObjects)
						{
							CreateNewIKRetargeterFromIKRig(SelectedObject);
						}
					})
				);
			}
		}
	}));
}

void FAssetTypeActions_IKRetargeter::CreateNewIKRetargeterFromIKRig(UObject* InSelectedObject)
{
	// validate ik rig
	UIKRigDefinition* IKRigAsset = Cast<UIKRigDefinition>(InSelectedObject);
	if(!IKRigAsset)
	{
		UE_LOGF(LogTemp, Error, "CreateNewIKRetargeterFromIKRig: Provided object has to be an IK Rig.");
		return;
	}
	
	// remove the standard "IK_" prefix which is standard for these asset types (convenience)
	FString AssetName = InSelectedObject->GetName();
	if (AssetName.Len() > 3)
	{
		AssetName.RemoveFromStart("IK_");	
	}

	// create unique package and asset names
	FString UniquePackageName;
	FString UniqueAssetName = FString::Printf(TEXT("RTG_%s"), *AssetName);
	
	FString PackagePath = InSelectedObject->GetPathName(); 
	int32 LastSlashPos = INDEX_NONE;
	if (PackagePath.FindLastChar('/', LastSlashPos))
	{
		PackagePath = PackagePath.Left(LastSlashPos);
	}
	
	PackagePath = PackagePath / UniqueAssetName;
	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(PackagePath, TEXT(""), UniquePackageName, UniqueAssetName);
	if (UniquePackageName.EndsWith(UniqueAssetName))
	{
		UniquePackageName = UniquePackageName.LeftChop(UniqueAssetName.Len() + 1);
	}
	
	// create the new IK Retargeter asset
	UIKRetargetFactory* Factory = NewObject<UIKRetargetFactory>();
	UObject* NewAsset = AssetToolsModule.Get().CreateAsset(*UniqueAssetName, *UniquePackageName, nullptr, Factory);
	const UIKRetargeter* RetargetAsset = Cast<UIKRetargeter>(NewAsset);

	// assign ik rig to the retargeter
	const UIKRetargeterController* Controller = UIKRetargeterController::GetController(RetargetAsset);
	Controller->SetIKRig(ERetargetSourceOrTarget::Source, IKRigAsset);
	Controller->SetIKRig(ERetargetSourceOrTarget::Target, IKRigAsset);
}

void FAssetTypeActions_IKRetargeter::ExtendAnimAssetMenusForBatchRetargeting()
{
	static const TArray<FName> MenusToExtend 
	{
		"ContentBrowser.AssetContextMenu.AnimSequence",
		"ContentBrowser.AssetContextMenu.BlendSpace",
		"ContentBrowser.AssetContextMenu.PoseAsset",
		"ContentBrowser.AssetContextMenu.AnimBlueprint",
		"ContentBrowser.AssetContextMenu.AnimMontage"
	};

	for (const FName& MenuName : MenusToExtend)
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(MenuName);
		check(Menu);

		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		Section.AddMenuEntry(
			"IKRetargetToDifferentSkeleton",
			LOCTEXT("RetargetAnimation_Label", "Retarget Animations"),
			LOCTEXT("RetargetAnimation_ToolTip", "Duplicate and retarget animation assets to a different skeleton. Works on sequences, blendspaces, pose assets, montages and animation blueprints."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCurveEditor.TabIcon"),
			FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& InContext)
			{
				if (const UContentBrowserAssetContextMenuContext* Context = InContext.FindContext<UContentBrowserAssetContextMenuContext>())
				{
					SRetargetAnimAssetsWindow::ShowWindow(Context->LoadSelectedObjects<UObject>());
				}
			})
		);
	}
}

const TArray<FText>& FAssetTypeActions_IKRetargeter::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		LOCTEXT("AnimRetargetingSubMenu", "Retargeting")
	};
	return SubMenus;
}

#undef LOCTEXT_NAMESPACE
