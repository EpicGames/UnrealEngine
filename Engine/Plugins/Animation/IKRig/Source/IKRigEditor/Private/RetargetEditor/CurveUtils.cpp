// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveUtils.h"
#include "Animation/AnimSequence.h"
#include "Animation/Skeleton.h"
#include "Interfaces//ITargetPlatformManagerModule.h"

bool FCurveUtils::LoadCurveValuesFromAnimSequence(const UAnimSequence* InSource, const USkeleton* InSourceSkeleton, TArray<FName>& OutCurveNames, TArray<FFrameTime>& OutFrameTimes, FCurveUtils::FFrameValues& OutCurveValuesPerFrame,
	TArray<int32>& OutCurveFlags, TArray<FLinearColor>& OutCurveColors)
{
	TArray<FString> CurveNames;
	if (LoadCurveValuesFromAnimSequence(InSource, CurveNames, OutFrameTimes, OutCurveValuesPerFrame))
	{
		OutCurveNames.Reserve(CurveNames.Num());
		OutCurveFlags.Reserve(CurveNames.Num());
		OutCurveColors.Reserve(CurveNames.Num());

		const IAnimationDataModel* SourceDataModel = InSource->GetDataModel();

		for (const FString& CurveName : CurveNames)
		{
			const FName CurCurveName(*CurveName);
			const FAnimationCurveIdentifier CurCurveId = UAnimationCurveIdentifierExtensions::FindCurveIdentifier(InSourceSkeleton, CurCurveName, ERawCurveTrackTypes::RCT_Float);

			const FFloatCurve* SourceCurve = SourceDataModel->FindFloatCurve(CurCurveId);
			if (SourceCurve)
			{
				OutCurveFlags.Add(SourceCurve->GetCurveTypeFlags());
				OutCurveColors.Add(SourceCurve->Color);
			}
			else
			{
				return false;
			}

			OutCurveNames.Add(CurCurveName);
		}
	}
	else
	{
		return false;
	}

	return true;
}

FFrameRate FCurveUtils::GetAnimSequenceRate(const UAnimSequence* InAnimSequence)
{
	return InAnimSequence ? InAnimSequence->GetTargetSamplingFrameRate(GetTargetPlatformManager()->GetRunningTargetPlatform()) : FFrameRate();
}

bool FCurveUtils::LoadCurveValuesFromAnimSequence(const UAnimSequence* InSource, TArray<FString>& OutCurveNames, TArray<FFrameTime>& OutFrameTimes, FFrameValues& OutCurveValuesPerFrame)
{
	FFramePoses Poses;

	const TArray<FFloatCurve>& FloatCurves = InSource->GetDataModel()->GetFloatCurves();

	if (!FloatCurves.IsEmpty())
	{
		OutFrameTimes.Reserve(FloatCurves[0].FloatCurve.GetNumKeys());

		for (const FFloatCurve& Curve : FloatCurves)
		{
			const FString CurveName = Curve.GetName().ToString();

			if (!Curve.FloatCurve.Keys.IsEmpty())
			{
				OutCurveNames.AddUnique(CurveName);

				for (const FRichCurveKey& Key : Curve.FloatCurve.Keys)
				{
					const FFrameTime FrameNumber = GetAnimSequenceRate(InSource).AsFrameTime(Key.Time);
					int32 PoseIndex = OutFrameTimes.Find(FrameNumber);

					if (PoseIndex == INDEX_NONE)
					{
						PoseIndex = OutFrameTimes.Add(FrameNumber);
						Poses.AddDefaulted_GetRef().Reserve(FloatCurves.Num());
					}

					Poses[PoseIndex].Add(CurveName, Key.Value);
				}
			}
		}
	}

	return BakeSparseKeys(Poses, OutCurveNames, OutFrameTimes, OutCurveValuesPerFrame);
}

bool FCurveUtils::BakeSparseKeys(const FFramePoses& InPoses, const TArray<FString>& InCurveNames, TArray<FFrameTime>& OutFrameTimes, FFrameValues& OutCurveValuesPerFrame)
{
	if (InPoses.IsEmpty())
	{
		return false;
	}
	if (InCurveNames.IsEmpty())
	{
		return false;
	}
	if (InPoses.Num() != OutFrameTimes.Num())
	{
		return false;
	}

	// This extra work is needed to make sure frame times and thus resulting curve values are in order
	TArray<FFrameTime> FrameTimesInOrder = OutFrameTimes;
	FrameTimesInOrder.Sort();

	OutCurveValuesPerFrame.Reset(FrameTimesInOrder.Num());
	for (int32 FrameIndex = 0; FrameIndex < OutFrameTimes.Num(); FrameIndex += 1)
	{
		OutCurveValuesPerFrame.AddDefaulted_GetRef().AddZeroed(InCurveNames.Num());
	}

	TArray<FString> BakedCurves;
	BakedCurves.Reserve(InCurveNames.Num());

	// If all curves are present on frame 0, we won't go further than one loop
	for (int32 FrameIndex = 0; FrameIndex < OutFrameTimes.Num() && BakedCurves.Num() < InCurveNames.Num(); FrameIndex += 1)
	{
		const int32 ActualFrameIndex = OutFrameTimes.Find(FrameTimesInOrder[FrameIndex]); // Lookup for non ordered frames

		for (const TPair<FString, float>& Keyframe : InPoses[ActualFrameIndex])
		{
			const FString& CurveName = Keyframe.Key;

			if (BakedCurves.Contains(CurveName))
			{
				continue;
			}
			BakedCurves.Add(CurveName);

			const float CurveValue = Keyframe.Value;
			const int32 CurveIndex = InCurveNames.Find(CurveName);

			SparseBakeCurve(CurveName, CurveIndex, CurveValue, OutCurveValuesPerFrame, FrameTimesInOrder, FrameIndex, InPoses, OutFrameTimes, ActualFrameIndex);

			if (BakedCurves.Num() == InCurveNames.Num())
			{
				break;
			}
		}
	}

	OutFrameTimes = FrameTimesInOrder;

	if (OutCurveValuesPerFrame.Num() != OutFrameTimes.Num())
	{
		return false;
	}
	if (OutCurveValuesPerFrame[0].Num() != InCurveNames.Num())
	{
		return false;
	}
	if (OutCurveValuesPerFrame.Last().Num() != InCurveNames.Num())
	{
		return false;
	}
	return true;
}

void FCurveUtils::SparseBakeCurve(const FString& InCurveName, int32 InCurveIndex, float InCurveValue, FFrameValues& OutCurveValuesPerFrame, const TArray<FFrameTime>& InFrameTimesInOrder, int32 InFrameIndex,
	const FFramePoses& InPoses, const TArray<FFrameTime>& InFrameTimes, const int32 InActualFrameIndex)
{
	// We haven't seen this control so far. So keep it const on all previous frames as well as the current one
	for (int32 BakeFrameIndex = 0; BakeFrameIndex <= InFrameIndex; BakeFrameIndex += 1)
	{
		OutCurveValuesPerFrame[BakeFrameIndex][InCurveIndex] = InCurveValue;
	}

	float LastKeyedValue = InCurveValue;
	int32 LastKeyedFrameIndex = InFrameIndex;
	int32 ActualLastKeyedFrameIndex = InActualFrameIndex; // Lookup for non ordered frames

	// For following frames, add a key a lerp all previously missed keys if need be
	for (int32 BakeFrameIndex = InFrameIndex + 1; BakeFrameIndex < InFrameTimes.Num(); BakeFrameIndex += 1)
	{
		const int32 ActualBakeFrameIndex = InFrameTimes.Find(InFrameTimesInOrder[BakeFrameIndex]); // Lookup for non ordered frames

		// If the current frame has a key for the control, set it, and try to lerp all the keys we might have missed since last time we saw the control
		if (InPoses[ActualBakeFrameIndex].Contains(InCurveName))
		{
			// Key this frame
			OutCurveValuesPerFrame[BakeFrameIndex][InCurveIndex] = InPoses[ActualBakeFrameIndex][InCurveName];
			LastKeyedValue = OutCurveValuesPerFrame[BakeFrameIndex][InCurveIndex].Get(0);

			// If we have not seen this control on the previous frame, lerp all keys with missed
			if (BakeFrameIndex > LastKeyedFrameIndex + 1)
			{
				// Lerp from the last key we found for this control to the current key, adding a lerp key for each frame
				for (int32 LerpFrameIndex = LastKeyedFrameIndex + 1; LerpFrameIndex < BakeFrameIndex; LerpFrameIndex += 1)
				{
					const double LerpAlpha = (InFrameTimesInOrder[LerpFrameIndex].AsDecimal() - InFrameTimesInOrder[LastKeyedFrameIndex].AsDecimal()) / (InFrameTimesInOrder[BakeFrameIndex].AsDecimal() - InFrameTimesInOrder[LastKeyedFrameIndex].AsDecimal());
					const double LerpValue = FMath::Lerp(InPoses[ActualLastKeyedFrameIndex][InCurveName], InPoses[ActualBakeFrameIndex][InCurveName], LerpAlpha);

					OutCurveValuesPerFrame[LerpFrameIndex][InCurveIndex] = static_cast<float>(LerpValue);
				}
			}
			LastKeyedFrameIndex = BakeFrameIndex;
			ActualLastKeyedFrameIndex = ActualBakeFrameIndex; // Lookup for non ordered frames
		}
		else
		{
			// Per default, add a const key to all next frames (in case there isn't any key next).
			// If it happens that there is one later, LastKeyedFrameIndex has not changed, and the above will lerp and override all const keys 
			OutCurveValuesPerFrame[BakeFrameIndex][InCurveIndex] = LastKeyedValue;
		}
	}
}