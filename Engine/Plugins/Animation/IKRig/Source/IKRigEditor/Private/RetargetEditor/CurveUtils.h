// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"


class UAnimSequence;
class USkeleton;

/**
 * Simple helper class to contain function for loading curve values from anim sequence.
 * Note, these functions are copies from ones in the RigMapper plugin Subsystem, but we do not want to make IKRig depend upon this plugin whilst it is experimental, and does not seem worthwhile
 * making an additional plugin.
 * When RigMapper plugin moves out of experimental, this code should be re-consolidated.
 */
class FCurveUtils
{
	
public:

	using FPoseValues = TArray<TOptional<float>>;
	using FFrameValues = TArray<FCurveUtils::FPoseValues>;

	/**
	 * Load curve values from animation sequence
	 */
	static bool LoadCurveValuesFromAnimSequence(const UAnimSequence* InSource, const USkeleton* InSourceSkeleton, TArray<FName>& OutCurveNames, TArray<FFrameTime>& OutFrameTimes, FCurveUtils::FFrameValues& OutCurveValuesPerFrame, TArray<int32> & OutCurveFlags, TArray<FLinearColor> & OutCurveColors);

	/**
	 * Get the frame rate for the animation sequence
	 */
	static FFrameRate GetAnimSequenceRate(const UAnimSequence* AnimSequence);


private:
	static bool LoadCurveValuesFromAnimSequence(const UAnimSequence* InSource, TArray<FString>& OutCurveNames, TArray<FFrameTime>& OutFrameTimes, FFrameValues& OutCurveValuesPerFrame);
	using FPose = TMap<FString, float>;
	using FFramePoses = TArray<FCurveUtils::FPose>;
	static bool BakeSparseKeys(const FFramePoses& InPoses, const TArray<FString>& InCurveNames, TArray<FFrameTime>& OutFrameTimes, FFrameValues& OutCurveValuesPerFrame);
	static void SparseBakeCurve(const FString& InCurveName, int32 InCurveIndex, float InCurveValue, FFrameValues& OutCurveValuesPerFrame, const TArray<FFrameTime>& InFrameTimesInOrder, int32 InFrameIndex, const FFramePoses& InPoses, const TArray<FFrameTime>& InFrameTimes, const int32 InActualFrameIndex);
};

