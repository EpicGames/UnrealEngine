// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetAnimInstance.h"
#include "Animation/AnimStats.h"
#include "Engine/SkeletalMesh.h"
#include "RetargetEditor/IKRetargetAnimInstanceProxy.h"
#include "RetargetEditor/IKRetargeterController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetAnimInstance)


void FAnimNode_PreviewRetargetPose::CacheBones_AnyThread(const FAnimationCacheBonesContext& Context)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)

	const FBoneContainer& RequiredBones = Context.AnimInstanceProxy->GetRequiredBones();
	if (!RequiredBones.IsValid())
	{
		return;
	}
	
	if (!Context.AnimInstanceProxy->GetSkelMeshComponent()->GetSkeletalMeshAsset())
	{
		return;
	}

	// rebuild mapping
	RequiredBoneToMeshBoneMap.Reset();

	const FReferenceSkeleton& RefSkeleton = RequiredBones.GetReferenceSkeleton();
	const FReferenceSkeleton& TargetSkeleton = Context.AnimInstanceProxy->GetSkelMeshComponent()->GetSkeletalMeshAsset()->GetRefSkeleton();
	const TArray<FBoneIndexType>& RequiredBonesArray = RequiredBones.GetBoneIndicesArray();
	for (int32 RequiredBoneIndex = 0; RequiredBoneIndex < RequiredBonesArray.Num(); ++RequiredBoneIndex)
	{
		const FBoneIndexType ReqBoneIndex = RequiredBonesArray[RequiredBoneIndex]; 
		if (ReqBoneIndex == INDEX_NONE)
		{
			continue;
		}
		
		const FName BoneName = RefSkeleton.GetBoneName(ReqBoneIndex);
		const int32 TargetBoneIndex = TargetSkeleton.FindBoneIndex(BoneName);
		if (TargetBoneIndex == INDEX_NONE)
		{
			continue;
		}

		// store require bone to target bone indices
		RequiredBoneToMeshBoneMap.Emplace(TargetBoneIndex);
	}
}

void FAnimNode_PreviewRetargetPose::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(PreviewRetargetPose, !IsInGameThread());

	Output.Pose.ResetToRefPose();
	
	if (!IKRetargeterAsset)
	{
		return;
	}
	
	const FIKRetargetPose* RetargetPose = IKRetargeterAsset->GetCurrentRetargetPose(SourceOrTarget);
	if (!RetargetPose)
	{
		return;	
	}
	
	// generate full (no LOD) LOCAL retarget pose by applying the local rotation offsets from the stored retarget pose
	// these poses are read back by the editor and need to be complete (not culled)
	const FReferenceSkeleton& RefSkeleton = Output.AnimInstanceProxy->GetSkelMeshComponent()->GetSkeletalMeshAsset()->GetRefSkeleton();
	const TArray<FTransform>& RefPose = RefSkeleton.GetRefBonePose(); 
	RetargetLocalPose = RefPose;
	for (const TTuple<FName, FQuat>& BoneDelta : RetargetPose->GetAllDeltaRotations())
	{
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(BoneDelta.Key);
		if (BoneIndex != INDEX_NONE)
		{
			FTransform& LocalBoneTransform = RetargetLocalPose[BoneIndex];
			LocalBoneTransform.SetRotation(LocalBoneTransform.GetRotation() * BoneDelta.Value);
		}
	}

	// generate GLOBAL space pose (for editor to query)
	FAnimationRuntime::FillUpComponentSpaceTransforms(RefSkeleton, RetargetLocalPose, RetargetGlobalPose);

	// apply root translation offset from the retarget pose (done in global space)
	const UIKRetargeterController* Controller = UIKRetargeterController::GetController(IKRetargeterAsset);
	const FName PelvisBoneName = Controller->GetPelvisBone(SourceOrTarget);
	const int32 PelvisBoneIndex = RefSkeleton.FindBoneIndex(PelvisBoneName);
	if (PelvisBoneIndex != INDEX_NONE)
	{
		FTransform& RootTransform = RetargetGlobalPose[PelvisBoneIndex];
		RootTransform.AddToTranslation(RetargetPose->GetRootTranslationDelta());
		
		// update local transform of pelvis
		const int32 ParentIndex = RefSkeleton.GetParentIndex(PelvisBoneIndex);
		if (ParentIndex == INDEX_NONE)
		{
			RetargetLocalPose[PelvisBoneIndex] = RootTransform;
		}
		else
		{
			const FTransform& ChildGlobalTransform = RetargetGlobalPose[PelvisBoneIndex];
			const FTransform& ParentGlobalTransform = RetargetGlobalPose[ParentIndex];
			RetargetLocalPose[PelvisBoneIndex] = ChildGlobalTransform.GetRelativeTransform(ParentGlobalTransform);
		}
	}

	// update GLOBAL space pose after root translation (for editor to query)
	FAnimationRuntime::FillUpComponentSpaceTransforms(RefSkeleton, RetargetLocalPose, RetargetGlobalPose);
	
	// copy to the compact output pose
	const int32 NumBones = Output.Pose.GetNumBones();
	for (int32 Index = 0; Index<NumBones; ++Index)
	{
		FCompactPoseBoneIndex BoneIndex(Index);
		const int32 MeshBoneIndex = RequiredBoneToMeshBoneMap[Index];

		BlendTransform<ETransformBlendMode::Overwrite>(RefPose[MeshBoneIndex], Output.Pose[BoneIndex], 1.f - RetargetPoseBlend);
		BlendTransform<ETransformBlendMode::Accumulate>(RetargetLocalPose[MeshBoneIndex], Output.Pose[BoneIndex], RetargetPoseBlend);
		Output.Pose.NormalizeRotations();
	}
}

UIKRetargetAnimInstance::UIKRetargetAnimInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = false;
}

void UIKRetargetAnimInstance::SetRetargetMode(const ERetargeterOutputMode& InOutputMode)
{
	FIKRetargetAnimInstanceProxy& Proxy = GetProxyOnGameThread<FIKRetargetAnimInstanceProxy>();
	Proxy.SetRetargetMode(InOutputMode);
}

void UIKRetargetAnimInstance::SetRetargetPoseBlend(const float& RetargetPoseBlend)
{
	FIKRetargetAnimInstanceProxy& Proxy = GetProxyOnGameThread<FIKRetargetAnimInstanceProxy>();
	Proxy.SetRetargetPoseBlend(RetargetPoseBlend);
}

void UIKRetargetAnimInstance::ConfigureAnimInstance(
	const ERetargetSourceOrTarget& InSourceOrTarget,
	UIKRetargeter* InAsset,
	TWeakObjectPtr<USkeletalMeshComponent> InSourceMeshComponent)
{
	FIKRetargetAnimInstanceProxy& Proxy = GetProxyOnGameThread<FIKRetargetAnimInstanceProxy>();
	Proxy.ConfigureAnimInstance(InSourceOrTarget, InAsset, InSourceMeshComponent);
}

void UIKRetargetAnimInstance::SetOverrideSets(const TArray<FName>& InOverrideSets)
{
	RetargetNode.bUseCustomOverrideSets = !InOverrideSets.IsEmpty();
	RetargetNode.OverrideSetsToApply = InOverrideSets;
}

FIKRetargetProcessor* UIKRetargetAnimInstance::GetRetargetProcessor()
{
	return RetargetNode.GetRetargetProcessor();
}

const FAnimNode_RetargetPoseFromMesh& UIKRetargetAnimInstance::GetRetargetAnimNode()
{
	return RetargetNode;
}

void UIKRetargetAnimInstance::ForceInitializeProcessor(USkeletalMeshComponent* TargetComponent)
{
	RetargetNode.IKRetargeterAsset->IncrementVersion();
	RetargetNode.EnsureProcessorIsInitialized(TargetComponent);
}

FAnimInstanceProxy* UIKRetargetAnimInstance::CreateAnimInstanceProxy()
{
	LLM_SCOPE_BYNAME(TEXT("Animation/IKRig"));
	return new FIKRetargetAnimInstanceProxy(this, &PreviewPoseNode, &RetargetNode);
}

