// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetAnimInstanceProxy.h"
#include "RetargetEditor/IKRetargetAnimInstance.h"
#include "AnimNodes/AnimNode_RetargetPoseFromMesh.h"
#include "RetargetEditor/IKRetargetEditorController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetAnimInstanceProxy)

FIKRetargetAnimInstanceProxy::FIKRetargetAnimInstanceProxy(
	UAnimInstance* InAnimInstance,
	FAnimNode_PreviewRetargetPose* InPreviewPoseNode,
	FAnimNode_RetargetPoseFromMesh* InRetargetNode)
	: FAnimPreviewInstanceProxy(InAnimInstance),
	PreviewPoseNode(InPreviewPoseNode),
	RetargetNode(InRetargetNode),
	OutputMode(ERetargeterOutputMode::EditRetargetPose)
{
	// retargeting is all done in world space, moving the source component breaks root motion retargeting
	bIgnoreRootMotion = true;
}

void FIKRetargetAnimInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	FAnimPreviewInstanceProxy::Initialize(InAnimInstance);
	PreviewPoseNode->InputPose.SetLinkNode(RetargetNode);

	FAnimationInitializeContext InitContext(this);
	PreviewPoseNode->Initialize_AnyThread(InitContext);
	RetargetNode->Initialize_AnyThread(InitContext);
}

void FIKRetargetAnimInstanceProxy::CacheBones()
{
	if (bBoneCachesInvalidated)
	{
		FAnimationCacheBonesContext Context(this);
		SingleNode.CacheBones_AnyThread(Context);
		RetargetNode->CacheBones_AnyThread(Context);
		PreviewPoseNode->CacheBones_AnyThread(Context);
		bBoneCachesInvalidated = false;
	}
}

bool FIKRetargetAnimInstanceProxy::Evaluate(FPoseContext& Output)
{
	if (PreviewPoseNode->IKRetargeterAsset)
	{
		bIgnoreRootLock = PreviewPoseNode->IKRetargeterAsset->bIgnoreRootLockInPreview;	
	}
	
	switch (OutputMode)
	{
	case ERetargeterOutputMode::RunRetarget:
		{
			if (SourceOrTarget == ERetargetSourceOrTarget::Source)
			{
				FAnimPreviewInstanceProxy::Evaluate(Output);
			}
			else
			{
				RetargetNode->Evaluate_AnyThread(Output);
			}
			break;
		}
	case ERetargeterOutputMode::EditRetargetPose:
		{
			PreviewPoseNode->Evaluate_AnyThread(Output);
			break;
		}
	default:
		checkNoEntry();
	}
	
	return true;
}

FAnimNode_Base* FIKRetargetAnimInstanceProxy::GetCustomRootNode()
{
	return PreviewPoseNode;
}

void FIKRetargetAnimInstanceProxy::GetCustomNodes(TArray<FAnimNode_Base*>& OutNodes)
{
	OutNodes.Add(RetargetNode);
	OutNodes.Add(PreviewPoseNode);
}

void FIKRetargetAnimInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	if (CurrentAsset != nullptr)
	{
		FAnimPreviewInstanceProxy::UpdateAnimationNode(InContext);
	}
	else
	{
		PreviewPoseNode->Update_AnyThread(InContext);
		RetargetNode->Update_AnyThread(InContext);
	}
}

void FIKRetargetAnimInstanceProxy::ConfigureAnimInstance(
	const ERetargetSourceOrTarget& InSourceOrTarget,
	UIKRetargeter* InIKRetargetAsset,
	TWeakObjectPtr<USkeletalMeshComponent> InSourceMeshComponent)
{
	SourceOrTarget = InSourceOrTarget;

	PreviewPoseNode->SourceOrTarget = InSourceOrTarget;
	PreviewPoseNode->IKRetargeterAsset = InIKRetargetAsset;

	if (SourceOrTarget == ERetargetSourceOrTarget::Target)
	{
		RetargetNode->IKRetargeterAsset = InIKRetargetAsset;
		RetargetNode->RetargetFrom = ERetargetSourceMode::CustomSkeletalMeshComponent;
		RetargetNode->SourceMeshComponent = InSourceMeshComponent;
		if (FIKRetargetProcessor* Processor = RetargetNode->GetRetargetProcessor())
		{
			Processor->SetNeedsInitialized();
		}
	}
}

void FIKRetargetAnimInstanceProxy::SetRetargetMode(const ERetargeterOutputMode& InOutputMode)
{
	OutputMode = InOutputMode;
	
	if (FIKRetargetProcessor* Processor = RetargetNode->GetRetargetProcessor())
	{
		Processor->SetNeedsInitialized();
	}
}

void FIKRetargetAnimInstanceProxy::SetRetargetPoseBlend(const float& InRetargetPoseBlend) const
{
	PreviewPoseNode->RetargetPoseBlend = InRetargetPoseBlend;
}


