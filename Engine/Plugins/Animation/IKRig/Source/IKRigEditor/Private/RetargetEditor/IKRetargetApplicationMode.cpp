// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetApplicationMode.h"

#include "IPersonaPreviewScene.h"
#include "PersonaModule.h"
#include "Modules/ModuleManager.h"
#include "PersonaTabs.h"
#include "RetargetEditor/IKRetargetAssetBrowserTabSummoner.h"

#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/IKRetargetHierarchyTabSummoner.h"
#include "RetargetEditor/IKRetargetOpStackTabSummoner.h"
#include "RetargetEditor/IKRetargetOutputLogTabSummoner.h"
#include "RetargetEditor/IKRetargetOverridesTabSummoner.h"
#include "RetargetEditor/IKRetargetVariablesTabSummoner.h"

#define LOCTEXT_NAMESPACE "IKRetargetMode"


FIKRetargetApplicationMode::FIKRetargetApplicationMode(
	TSharedRef<FWorkflowCentricApplication> InHostingApp,  
	TSharedRef<IPersonaPreviewScene> InPreviewScene)
	: FApplicationMode(IKRetargetApplicationModes::IKRetargetApplicationMode)
{
	IKRetargetEditorPtr = StaticCastSharedRef<FIKRetargetEditor>(InHostingApp);
	TSharedRef<FIKRetargetEditor> IKRetargetEditor = StaticCastSharedRef<FIKRetargetEditor>(InHostingApp);

	FPersonaViewportArgs ViewportArgs(InPreviewScene);
	ViewportArgs.bAlwaysShowTransformToolbar = true;
	ViewportArgs.bShowStats = false;
	ViewportArgs.bShowTurnTable = false;
	ViewportArgs.ContextName = TEXT("IKRetargetEditor.Viewport");
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(IKRetargetEditor, &FIKRetargetEditor::HandleViewportCreated);

	// register Persona tabs
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TabFactories.RegisterFactory(PersonaModule.CreatePersonaViewportTabFactory(InHostingApp, ViewportArgs));
	TabFactories.RegisterFactory(PersonaModule.CreateDetailsTabFactory(InHostingApp, FOnDetailsCreated::CreateSP(&IKRetargetEditor.Get(), &FIKRetargetEditor::HandleDetailsCreated)));
	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InHostingApp, IKRetargetEditor->GetPersonaToolkit()->GetPreviewScene()));

	// register custom tabs
	TabFactories.RegisterFactory(MakeShared<FIKRetargetAssetBrowserTabSummoner>(IKRetargetEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRetargetOutputLogTabSummoner>(IKRetargetEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRetargetHierarchyTabSummoner>(IKRetargetEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRetargetOpStackTabSummoner>(IKRetargetEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRetargetOverridesTabSummoner>(IKRetargetEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRetargetVariablesTabSummoner>(IKRetargetEditor));

	// create tab layout
	TabLayout = FTabManager::NewLayout("Standalone_IKRetargetEditor_Layout_v1.022")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.8f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
				FTabManager::NewSplitter()
					//->SetSizeCoefficient(0.8f)
					->SetOrientation(Orient_Horizontal)
					->Split
					(
						FTabManager::NewSplitter()
						->SetSizeCoefficient(0.2f)
						->SetOrientation(Orient_Vertical)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.6f)
							->AddTab(FIKRetargetOpStackTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FIKRetargetHierarchyTabSummoner::TabID, ETabState::OpenedTab)
						)
						->Split
						(
							FTabManager::NewStack()
							->SetSizeCoefficient(0.4f)
							->AddTab(FIKRetargetAssetBrowserTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FIKRetargetOverridesTabSummoner::TabID, ETabState::OpenedTab)
							->AddTab(FIKRetargetVariablesTabSummoner::TabID, ETabState::OpenedTab)
						)
					)
					->Split
					(
					FTabManager::NewSplitter()
							->SetSizeCoefficient(0.8f)
							->SetOrientation(Orient_Vertical)
							->Split
							(
							FTabManager::NewStack()
								->SetSizeCoefficient(0.9f)
								->SetHideTabWell(true)
								->AddTab(FPersonaTabs::PreviewViewportID, ETabState::OpenedTab)
							)
							->Split
							(
							FTabManager::NewStack()
								->SetSizeCoefficient(0.1f)
								->AddTab(FIKRetargetOutputLogTabSummoner::TabID, ETabState::OpenedTab)
							)
					)
				)
				->Split
				(
				FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->AddTab(FPersonaTabs::DetailsID, ETabState::OpenedTab)
					->AddTab(FPersonaTabs::AdvancedPreviewSceneSettingsID, ETabState::OpenedTab)
					->SetForegroundTab(FPersonaTabs::DetailsID)
					
				)
			)
		);

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InHostingApp);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
}

void FIKRetargetApplicationMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FIKRetargetEditor> IKRigEditor = IKRetargetEditorPtr.Pin();
	IKRigEditor->RegisterTabSpawners(InTabManager.ToSharedRef());
	IKRigEditor->PushTabFactories(TabFactories);
}

#undef LOCTEXT_NAMESPACE
