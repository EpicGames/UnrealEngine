// Copyright Epic Games, Inc. All Rights Reserved.
#include "RetargetEditor/IKRetargetAssetBrowserTabSummoner.h"

#include "IDocumentation.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/SIKRetargetAssetBrowser.h"

#define LOCTEXT_NAMESPACE "RetargetAssetBrowserTabSummoner"

const FName FIKRetargetAssetBrowserTabSummoner::TabID(TEXT("AssetBrowser"));

FIKRetargetAssetBrowserTabSummoner::FIKRetargetAssetBrowserTabSummoner(const TSharedRef<FIKRetargetEditor>& InRetargetEditor)
	: FWorkflowTabFactory(TabID, InRetargetEditor),
	IKRetargetEditor(InRetargetEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRetargetAssetBrowserTabLabel", "Asset Browser");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.TabIcon");

	ViewMenuDescription = LOCTEXT("IKRetargetAssetBrowser_ViewMenu_Desc", "Asset Browser");
	ViewMenuTooltip = LOCTEXT("IKRetargetAssetBrowser_ViewMenu_ToolTip", "Show the Asset Browser Tab");
}

TSharedPtr<SToolTip> FIKRetargetAssetBrowserTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRetargetAssetBrowserTooltip", "Select an animation asset to preview the retarget results."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRetargetAssetBrowser_Window"));
}

TSharedRef<SWidget> FIKRetargetAssetBrowserTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SIKRetargetAssetBrowser, IKRetargetEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
