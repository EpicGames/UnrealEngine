// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetBatchOperation.h"

#include "Engine/SkeletalMesh.h"
#include "Animation/AnimSequence.h"
#include "AnimationBlueprintLibrary.h"
#include "AnimPose.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "EditorReimportHandler.h"
#include "IContentBrowserSingleton.h"
#include "SSkeletonWidget.h"
#include "EditorFramework/AssetImportData.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/ScopedSlowTask.h"
#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Animation/AnimMontage.h"
#include "Retargeter/IKRetargetOps.h"
#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Retargeter/RetargetOps/SpeedPlantingOp.h"
#include "CurveUtils.h"
#include "Animation/AnimData/IAnimationDataModel.h"
#include "Animation/AnimCurveTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetBatchOperation)

#define LOCTEXT_NAMESPACE "RetargetBatchOperation"

int32 UIKRetargetBatchOperation::GenerateAssetLists(const FIKRetargetBatchOperationContext& Context)
{
	// re-generate lists of selected and referenced assets
	AnimationAssetsToRetarget.Reset();
	AnimBlueprintsToRetarget.Reset();

	for (TWeakObjectPtr<UObject> AssetPtr : Context.AssetsToRetarget)
	{
		if (!AssetPtr.IsValid())
		{
			continue;
		}
		UObject* Asset = AssetPtr.Get();
		if (UAnimationAsset* AnimAsset = Cast<UAnimationAsset>(Asset))
		{
			AnimationAssetsToRetarget.AddUnique(AnimAsset);

			// sequences that are used within the montage need to be added as well to be duplicated. They will then
			// be replaced in UAnimMontage::ReplaceReferredAnimations
			if (UAnimMontage* AnimMontage = Cast<UAnimMontage>(AnimAsset))
			{
				// add segments
				for (const FSlotAnimationTrack& Track: AnimMontage->SlotAnimTracks)
				{
					for (const FAnimSegment& Segment: Track.AnimTrack.AnimSegments)
					{
						if (Segment.IsValid() && Segment.GetAnimReference())
						{
							AnimationAssetsToRetarget.AddUnique(Segment.GetAnimReference());
						}
					}
				}

				// add preview pose
				if (AnimMontage->PreviewBasePose)
				{
					AnimationAssetsToRetarget.AddUnique(AnimMontage->PreviewBasePose);
				}
			}
		}
		else if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Asset))
		{
			// Add parents blueprint. 
			UAnimBlueprint* ParentBP = (AnimBlueprint->ParentClass && AnimBlueprint->ParentClass->ClassGeneratedBy)
				? Cast<UAnimBlueprint>(AnimBlueprint->ParentClass->ClassGeneratedBy)
				: nullptr;
			while (ParentBP)
			{
				AnimBlueprintsToRetarget.AddUnique(ParentBP);
				ParentBP = (ParentBP->ParentClass && ParentBP->ParentClass->ClassGeneratedBy)
					? Cast<UAnimBlueprint>(ParentBP->ParentClass->ClassGeneratedBy)
					: nullptr;
			}
				
			AnimBlueprintsToRetarget.AddUnique(AnimBlueprint);				
		}
	}

	if (Context.bIncludeReferencedAssets)
	{
		// Grab assets from the blueprint.
		// Do this first as it can add complex assets to the retarget array which will need to be processed next.
		for (UAnimBlueprint* AnimBlueprint : AnimBlueprintsToRetarget)
		{
			if (IsValid(AnimBlueprint) && AnimBlueprint->GetAnimBlueprintGeneratedClass() != nullptr)
			{
				GetAllAnimationSequencesReferredInBlueprint(AnimBlueprint, AnimationAssetsToRetarget);
			}
		}

		int32 AssetIndex = 0;
		while (AssetIndex < AnimationAssetsToRetarget.Num())
		{
			UAnimationAsset* AnimAsset = AnimationAssetsToRetarget[AssetIndex++];
			if (IsValid(AnimAsset))
			{
				AnimAsset->HandleAnimReferenceCollection(AnimationAssetsToRetarget, true);
			}
		}
	}

	return AnimationAssetsToRetarget.Num() + AnimBlueprintsToRetarget.Num();
}

void UIKRetargetBatchOperation::DuplicateRetargetAssets(
	const FIKRetargetBatchOperationContext& Context,
	FScopedSlowTask& Progress)
{
	Progress.EnterProgressFrame(1.f, FText(LOCTEXT("DuplicatingBatchRetarget", "Duplicating animation assets...")));
	
	UPackage* DestinationPackage = Context.TargetMesh->GetOutermost();

	TArray<UAnimationAsset*> AnimationAssetsToDuplicate = AnimationAssetsToRetarget;
	TArray<UAnimBlueprint*> AnimBlueprintsToDuplicate = AnimBlueprintsToRetarget;

	// We only want to duplicate unmapped assets, so we remove mapped assets from the list we're duplicating
	for (TPair<UAnimationAsset*, UAnimationAsset*>& Pair : RemappedAnimAssets)
	{
		AnimationAssetsToDuplicate.Remove(Pair.Key);
	}

	// duplicate each asset individually (not done as a batch so user can cancel)
	for (UAnimationAsset* Asset : AnimationAssetsToDuplicate)
	{
		if (Progress.ShouldCancel())
		{
			return;
		}

		FString AssetName = Asset->GetName();
		Progress.EnterProgressFrame(1.f, FText::Format(LOCTEXT("DuplicatingAnimation", "Duplicating animation: {0}"), FText::FromString(AssetName)));

		// if user wants to export files to the same location as the source, then replace the FolderPath in the duplication rule
		FNameDuplicationRule NameRule = Context.NameRule;
		if (Context.bUseSourcePath)
		{
			NameRule.FolderPath = FPackageName::GetLongPackagePath(Asset->GetPathName()) / TEXT("");
		}
		
		TMap<UAnimationAsset*, UAnimationAsset*> DuplicateMap = DuplicateAssets<UAnimationAsset>({Asset}, DestinationPackage, &NameRule);
		DuplicatedAnimAssets.Append(DuplicateMap);
	}
	for (UAnimBlueprint* Asset : AnimBlueprintsToDuplicate)
	{
		if (Progress.ShouldCancel())
		{
			return;
		}

		FString AssetName = Asset->GetName();
		Progress.EnterProgressFrame(1.f, FText::Format(LOCTEXT("DuplicatingBlueprint", "Duplicating blueprint: {0}"), FText::FromString(AssetName)));

		// if user wants to export files to the same location as the source, then replace the FolderPath in the duplication rule
		FNameDuplicationRule NameRule = Context.NameRule;
		if (Context.bUseSourcePath)
		{
			NameRule.FolderPath = FPackageName::GetLongPackagePath(Asset->GetPathName()) / TEXT("");
		}
		
		TMap<UAnimBlueprint*, UAnimBlueprint*> DuplicateMap = DuplicateAssets<UAnimBlueprint>({Asset}, DestinationPackage, &NameRule);
		DuplicatedBlueprints.Append(DuplicateMap);
	}

	// If we are moving the new asset to a different directory we need to fixup the reimport path.
	// This should only effect source FBX paths within the project.
	if (!Context.NameRule.FolderPath.IsEmpty())
	{
		for (TPair<UAnimationAsset*, UAnimationAsset*>& Pair : DuplicatedAnimAssets)
		{
			UAnimSequence* SourceSequence = Cast<UAnimSequence>(Pair.Key);
			UAnimSequence* DestinationSequence = Cast<UAnimSequence>(Pair.Value);
			if (!(SourceSequence && DestinationSequence))
			{
				continue;
			}
			
			for (int index = 0; index < SourceSequence->AssetImportData->SourceData.SourceFiles.Num(); index++)
			{
				const FString& RelativeFilename = SourceSequence->AssetImportData->SourceData.SourceFiles[index].RelativeFilename;
				const FString OldPackagePath = FPackageName::GetLongPackagePath(SourceSequence->GetPathName()) / TEXT("");
				const FString NewPackagePath = FPackageName::GetLongPackagePath(DestinationSequence->GetPathName()) / TEXT("");
				const FString AbsoluteSrcPath = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(OldPackagePath));
				const FString SrcFile = AbsoluteSrcPath / RelativeFilename;
				const bool bSrcFileExists = FPlatformFileManager::Get().GetPlatformFile().FileExists(*SrcFile);
				if (!bSrcFileExists || (NewPackagePath == OldPackagePath))
				{
					continue;
				}

				FString BasePath = FPackageName::LongPackageNameToFilename(OldPackagePath);
				FString OldSourceFilePath = FPaths::ConvertRelativePathToFull(BasePath, RelativeFilename);
				TArray<FString> Paths;
				Paths.Add(OldSourceFilePath);
				
				// update the FBX reimport file path
				FReimportManager::Instance()->UpdateReimportPaths(DestinationSequence, Paths);
			}
		}
	}

	// Remapped assets needs the duplicated ones added
	RemappedAnimAssets.Append(DuplicatedAnimAssets);

	DuplicatedAnimAssets.GenerateValueArray(AnimationAssetsToRetarget);
	DuplicatedBlueprints.GenerateValueArray(AnimBlueprintsToRetarget);
}

void UIKRetargetBatchOperation::RetargetAssets(
	const FIKRetargetBatchOperationContext& Context,
	FScopedSlowTask& Progress)
{
	USkeleton* OldSkeleton = Context.SourceMesh->GetSkeleton();
	USkeleton* NewSkeleton = Context.TargetMesh->GetSkeleton();

	TArray<FAdditiveRetargetSettings> SettingsToRestoreAfterRetarget;

	
	for (UAnimationAsset* AssetToRetarget : AnimationAssetsToRetarget)
	{
		if (Progress.ShouldCancel())
		{
			return;
		}
		
		// prepare animation sequence asset to receive retargeted animation
		if (UAnimSequence* AnimSequenceToRetarget = Cast<UAnimSequence>(AssetToRetarget))
		{
			FString AssetName = AnimSequenceToRetarget->GetName();
			Progress.EnterProgressFrame(1.f, FText::Format(LOCTEXT("PreparingAsset", "Preparing asset: {0}"), FText::FromString(AssetName)));

			// clear transform curves since those curves won't work in new skeleton
			IAnimationDataController& Controller = AnimSequenceToRetarget->GetController();
			constexpr bool bShouldTransact = false;
			Controller.OpenBracket(FText::FromString("Preparing for retargeted animation."), bShouldTransact);
			Controller.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Transform, bShouldTransact);

			// reset all additive animation properties to ensure WYSIWYG playback of additive anims between retargeter and sequence
			FAdditiveRetargetSettings SequenceSettings;
			SequenceSettings.PrepareForRetarget(AnimSequenceToRetarget);
			SettingsToRestoreAfterRetarget.Add(SequenceSettings);
			
			// set the retarget source to the target skeletal mesh
			AnimSequenceToRetarget->RetargetSource = NAME_None;
			AnimSequenceToRetarget->SetRetargetSourceAsset(Context.TargetMesh);

			// removes all bone tracks and add those that exist in the target skeleton
			Controller.RemoveAllBoneTracks(bShouldTransact);
			Controller.UpdateWithSkeleton(NewSkeleton, bShouldTransact);

			// done editing sequence data, close bracket
			Controller.CloseBracket(bShouldTransact);
		}

		// replace references to other animation
		AssetToRetarget->ReplaceReferredAnimations(RemappedAnimAssets);
		AssetToRetarget->SetSkeleton(NewSkeleton);
		AssetToRetarget->SetPreviewMesh(Context.TargetMesh);
	}

	// Call PostEditChange after the references of all assets were replaced, to prevent order dependence of post edit
	// change hooks. If PostEditChange is called right after ReplaceReferredAnimations it can access references that are
	// still queued for retarget and follow the current asset in the array.
	for (UAnimationAsset* AssetToRetarget : AnimationAssetsToRetarget)
	{
		if (Progress.ShouldCancel())
		{
			return;
		}
		
		// force updating of the retarget pose, this is normally done on PreSave() but is guarded against procedural saves
		if (UAnimSequence* AnimSequenceToRetarget = Cast<UAnimSequence>(AssetToRetarget))
		{
			AnimSequenceToRetarget->UpdateRetargetSourceAssetData();
		}
		
		AssetToRetarget->PostEditChange();
		AssetToRetarget->MarkPackageDirty();
	}

	// convert the animation using the IK retargeter	
	FIKRetargetProcessor Processor;
	FRetargetProfile RetargetProfile;
	RetargetProfile.FillProfileWithAssetSettings(Context.IKRetargetAsset);
	// initialize the retarget processor
	FRetargetInitParameters Params;
	Params.SourceSkeletalMesh = Context.SourceMesh;
	Params.TargetSkeletalMesh = Context.TargetMesh;
	Params.RetargeterAsset = Context.IKRetargetAsset;
	Params.CustomProfile = &RetargetProfile;
	Params.bSuppressWarnings = false;
	Processor.Initialize(Params);
	if (!Processor.IsInitialized())
	{
		UE_LOGF(LogTemp, Warning, "Unable to initialize the IK Retargeter. Newly created animations were not retargeted!");
		return;
	}

	ConvertAnimation(Context, Processor, Progress);


	// optionally restore the additive flags
	if (Context.bRetainAdditiveFlags)
	{
		for (FAdditiveRetargetSettings SettingsToRestore : SettingsToRestoreAfterRetarget)
		{
			SettingsToRestore.RestoreOnAsset();
		}
	}

	// convert all Animation Blueprints and compile 
	for (UAnimBlueprint* AnimBlueprint : AnimBlueprintsToRetarget)
	{
		if (Progress.ShouldCancel())
		{
			return;
		}
		
		// replace skeleton
		AnimBlueprint->TargetSkeleton = NewSkeleton;
		// replace preview mesh (uses skeleton default otherwise)
		AnimBlueprint->SetPreviewMesh(Context.TargetMesh);

		// if they have parent blueprint, make sure to re-link to the new one also
		UAnimBlueprint* CurrentParentBP = AnimBlueprint->ParentClass
			? Cast<UAnimBlueprint>(AnimBlueprint->ParentClass->ClassGeneratedBy)
			: nullptr;
		if (CurrentParentBP)
		{
			UAnimBlueprint* const * ParentBP = DuplicatedBlueprints.Find(CurrentParentBP);
			if (ParentBP)
			{
				AnimBlueprint->ParentClass = (*ParentBP)->GeneratedClass;
			}
		}

		if(RemappedAnimAssets.Num() > 0)
		{
			ReplaceReferredAnimationsInBlueprint(AnimBlueprint, RemappedAnimAssets);
		}

		FBlueprintEditorUtils::RefreshAllNodes(AnimBlueprint);
		FKismetEditorUtilities::CompileBlueprint(AnimBlueprint, EBlueprintCompileOptions::SkipGarbageCollection);
		AnimBlueprint->PostEditChange();
		AnimBlueprint->MarkPackageDirty();
	}

	// apply any curve operations to duplicate sequences
	ApplyCurveOps(Context, Processor, Progress);
}

void UIKRetargetBatchOperation::ConvertAnimation(
	const FIKRetargetBatchOperationContext& Context,
	FIKRetargetProcessor& OutProcessor,
	FScopedSlowTask& Progress)
{
	// target skeleton data
	const FRetargetSkeleton& TargetSkeleton = OutProcessor.GetSkeleton(ERetargetSourceOrTarget::Target);
	const TArray<FName>& TargetBoneNames = TargetSkeleton.BoneNames;
	const int32 NumTargetBones = TargetBoneNames.Num();

	// allocate target keyframe data
	TArray<FRawAnimSequenceTrack> BoneTracks;
	BoneTracks.SetNumZeroed(NumTargetBones);

	// source skeleton data
	const FRetargetSkeleton& SourceSkeleton = OutProcessor.GetSkeleton(ERetargetSourceOrTarget::Source);
	const TArray<FName>& SourceBoneNames = SourceSkeleton.BoneNames;
	const int32 NumSourceBones = SourceBoneNames.Num();

	TArray<FTransform> SourceComponentPose;
	SourceComponentPose.SetNum(NumSourceBones);
	
	// get list of all speed curves referenced by the speed planting ops (if there are any)
	TArray<FName> SpeedCurveNames;
	const TArray<FIKRetargetSpeedPlantingOp*> SpeedPlantingOps = Context.IKRetargetAsset->GetAllRetargetOpsOfType<FIKRetargetSpeedPlantingOp>();
	for (FIKRetargetSpeedPlantingOp* SpeedPlantingOp : SpeedPlantingOps)
	{
		SpeedCurveNames.Append(SpeedPlantingOp->GetRequiredSpeedCurves());
	}

	// build a map of all curve names that are bound to override set properties
	// this mirrors FAnimNode_RetargetPoseFromMesh::CopyBoundCurveValues() for the offline batch path
	TMap<FName, float> BoundCurveValues;
	for (const TTuple<FName, FRetargetOverrideSet>& OverrideSet : Context.IKRetargetAsset->GetOverrideSets())
	{
		for (const FRetargetOpOverrides& OpOverrides : OverrideSet.Value.OpOverrides)
		{
			for (const FRetargetOpPropertyOverride& PropertyOverride : OpOverrides.PropertyOverrides)
			{
				const FName BoundCurveName = PropertyOverride.GetBoundCurveName();
				if (!BoundCurveName.IsNone())
				{
					BoundCurveValues.Emplace(BoundCurveName, 0.0f);
				}
			}
		}
	}

	// while retargeting, record a map of (sequence name, curve name) for any curve missing in a sequence
	TMap<FString, TSet<FName>> MissingSpeedCurves;
	
	// for each pair of source / target animation sequences
	for (TPair<UAnimationAsset*, UAnimationAsset*>& Pair : DuplicatedAnimAssets)
	{
		if (Progress.ShouldCancel())
		{
			return;
		}
		
		UAnimSequence* SourceSequence = Cast<UAnimSequence>(Pair.Key);
		UAnimSequence* TargetSequence = Cast<UAnimSequence>(Pair.Value);
		if (!(SourceSequence && TargetSequence))
		{
			continue;
		}

		// increment progress bar
		FString AssetName = TargetSequence->GetName();
		Progress.EnterProgressFrame(1.f, FText::Format(LOCTEXT("RunningBatchRetarget", "Retargeting animation asset: {0}"), FText::FromString(AssetName)));

		// update the sequence to use the new skeleton
		IAnimationDataController& TargetSeqController = TargetSequence->GetController();
		constexpr bool bShouldTransact = false;
		TargetSeqController.OpenBracket(FText::FromString("Generating Retargeted Animation Data"), bShouldTransact);
		TargetSeqController.NotifyPopulated(); // must set bIsPopulated=true otherwise "UpdateWithSkeleton" will early out
		// this removes all bone tracks and reinitializes the FKControlRig to use the new skeleton.
		// NOTE: we do NOT ResetModel() because we want to keep curves and attributes from the source
		TargetSeqController.RemoveAllBoneTracks(bShouldTransact);
		TargetSeqController.UpdateWithSkeleton(const_cast<USkeleton*>(TargetSkeleton.SkeletalMesh->GetSkeleton()), bShouldTransact);

		// number of frames in this animation
		const int32 NumFrames = SourceSequence->GetNumberOfSampledKeys();

		// BoneTracks arrays allocation
		for (int32 TargetBoneIndex=0; TargetBoneIndex<NumTargetBones; ++TargetBoneIndex)
		{
			BoneTracks[TargetBoneIndex].PosKeys.SetNum(NumFrames);
			BoneTracks[TargetBoneIndex].RotKeys.SetNum(NumFrames);
			BoneTracks[TargetBoneIndex].ScaleKeys.SetNum(NumFrames);
		}

		// ensure we evaluate the source animation using the skeletal mesh proportions that were evaluated in the viewport
		FAnimPoseEvaluationOptions EvaluationOptions = FAnimPoseEvaluationOptions();
		EvaluationOptions.OptionalSkeletalMesh = const_cast<USkeletalMesh*>(SourceSkeleton.SkeletalMesh);
		// ensure WYSIWYG with editor by ensuring the same root motion is applied to the pose, not to the component
		EvaluationOptions.bExtractRootMotion = false;
		EvaluationOptions.bIncorporateRootMotionIntoPose = true;

		// reset playback of ops
		OutProcessor.OnPlaybackReset();
		
		// retarget each frame's pose from source to target
		for (int32 FrameIndex=0; FrameIndex<NumFrames; ++FrameIndex)
		{
			if (Progress.ShouldCancel())
			{
				TargetSeqController.CloseBracket(bShouldTransact);
				return;
			}
			
			// get the source global pose
			FAnimPose SourcePoseAtFrame;
			UAnimPoseExtensions::GetAnimPoseAtFrame(SourceSequence, FrameIndex, EvaluationOptions, SourcePoseAtFrame);

			// we don't use UAnimPoseExtensions::GetBoneNames as the sequence can store bones that only exist on the
			// skeleton, but not on the current mesh. This results in indices discrepancy
			for (int32 BoneIndex = 0; BoneIndex < NumSourceBones; BoneIndex++)
			{
				const FName& BoneName = SourceBoneNames[BoneIndex];
				SourceComponentPose[BoneIndex] = UAnimPoseExtensions::GetBonePose(SourcePoseAtFrame, BoneName, EAnimPoseSpaces::World);
			}
			
			// strip all scale out of the pose values, the translation of a component-space pose has incorporated scale values
			for (FTransform& Transform : SourceComponentPose)
			{
				Transform.SetScale3D(FVector::OneVector);
			}
			
			// calculate the delta time
			const float TimeAtCurrentFrame = SourceSequence->GetTimeAtFrame(FrameIndex);
			float DeltaTime = TimeAtCurrentFrame;
			if (FrameIndex > 0)
			{
				const float TimeAtPrevFrame = SourceSequence->GetTimeAtFrame(FrameIndex-1);
				DeltaTime = TimeAtCurrentFrame - TimeAtPrevFrame;
			}
			
			// get the curve values from the source sequence (for speed-based IK planting)
			TMap<FName, float> SpeedCurveValues;
			constexpr bool bForceUseRawData = false;
			for (const FName& SpeedCurveName : SpeedCurveNames)
			{
				if (!SourceSequence->HasCurveData(SpeedCurveName, bForceUseRawData))
				{
					TSet<FName>& MissingCurves = MissingSpeedCurves.FindOrAdd(SourceSequence->GetName());
					MissingCurves.Add(SpeedCurveName);
					continue;
				}

				SpeedCurveValues.Add(SpeedCurveName, SourceSequence->EvaluateCurveData(SpeedCurveName, FAnimExtractContext(static_cast<double>(TimeAtCurrentFrame))));
			}

			// get the settings profile
			FRetargetProfile SettingsProfile;
			SettingsProfile.FillProfileWithAssetSettings(Context.IKRetargetAsset);
			
			// let the retargeter scale the input pose
			OutProcessor.ApplySourceScaleToPose(SourceComponentPose);
			
			// Let ops sample data directly from anim sequence
			OutProcessor.UpdateOpsFromAnimSequence(SourceSequence, TimeAtCurrentFrame);

			// sample curve-bound override values from the source sequence at the current frame time
			for (TTuple<FName, float>& BoundCurve : BoundCurveValues)
			{
				BoundCurve.Value = SourceSequence->HasCurveData(BoundCurve.Key, bForceUseRawData)
					? SourceSequence->EvaluateCurveData(BoundCurve.Key, FAnimExtractContext(static_cast<double>(TimeAtCurrentFrame)))
					: 0.0f;
			}

			// run the retargeter
			FRetargetRunParameters Params;
			Params.SourceGlobalPose = &SourceComponentPose;
			Params.Profile = &SettingsProfile;
			Params.OverrideSetsToApply = Context.OverrideSetNames;
			Params.BoundCurveValues = BoundCurveValues.IsEmpty() ? nullptr : &BoundCurveValues;
			Params.DeltaTime = DeltaTime;
			const TArray<FTransform>& TargetComponentPose = OutProcessor.RunRetargeter(Params);

			// convert to a local-space pose
			TArray<FTransform> TargetLocalPose = TargetComponentPose;
			TargetSkeleton.UpdateLocalTransformsBelowBone(0,TargetLocalPose, TargetComponentPose);

			// store key data for each bone
			for (int32 TargetBoneIndex=0; TargetBoneIndex<NumTargetBones; ++TargetBoneIndex)
			{
				const FTransform& LocalPose = TargetLocalPose[TargetBoneIndex];
				
				FRawAnimSequenceTrack& BoneTrack = BoneTracks[TargetBoneIndex];
				
				BoneTrack.PosKeys[FrameIndex] = FVector3f(LocalPose.GetLocation());
				BoneTrack.RotKeys[FrameIndex] = FQuat4f(LocalPose.GetRotation().GetNormalized());
				BoneTrack.ScaleKeys[FrameIndex] = FVector3f(LocalPose.GetScale3D());
			}
			
		} // END for each frame

		// add keys to bone tracks
		for (int32 TargetBoneIndex=0; TargetBoneIndex<NumTargetBones; ++TargetBoneIndex)
		{
			const FName& TargetBoneName = TargetBoneNames[TargetBoneIndex];

			const FRawAnimSequenceTrack& RawTrack = BoneTracks[TargetBoneIndex];
			TargetSeqController.AddBoneCurve(TargetBoneName, bShouldTransact);
			TargetSeqController.SetBoneTrackKeys(TargetBoneName, RawTrack.PosKeys, RawTrack.RotKeys, RawTrack.ScaleKeys, bShouldTransact);
		}

		TargetSeqController.CloseBracket(bShouldTransact);
	}

	// warn about any missing curves
	for (const TPair<FString, TSet<FName>>& Pair : MissingSpeedCurves)
	{
		for (const FName& MissingCurve : Pair.Value)
		{
			UE_LOGF(LogTemp, Warning, "IK Retarget Batch Operation: Missing speed curve, %ls on sequence %ls.", *MissingCurve.ToString(), *Pair.Key);	
		}
	}
}

void UIKRetargetBatchOperation::ApplyCurveOps(const FIKRetargetBatchOperationContext& Context, const FIKRetargetProcessor& InProcessor, FScopedSlowTask& Progress)
{
	USkeleton* SourceSkeleton = Context.SourceMesh->GetSkeleton();
	USkeleton* TargetSkeleton = Context.TargetMesh->GetSkeleton();

	// update progress bar
	Progress.EnterProgressFrame(1.f, FText(LOCTEXT("ApplyCurveOps", "Applying Curve Operations")));

	auto NoCurveOps = [](const TArray<FInstancedStruct>&InOpStack)
		{
			for (const FInstancedStruct& OpStruct : InOpStack)
			{
				const UStruct* StructType = OpStruct.GetScriptStruct();
				if (StructType && StructType->IsChildOf(FIKRetargetCurvesOpBase::StaticStruct()) &&
					OpStruct.Get<FIKRetargetCurvesOpBase>().HasCurveProcessing())
				{
					return false;
				}
			}

			return true;
		};

	const TArray<FInstancedStruct>& OpStack = Context.IKRetargetAsset->GetRetargetOps();

	// for each exported animation, apply any required operations to curves, but if no curve ops, skip entirely for efficiency
	if (NoCurveOps(OpStack))
	{
		return;
	}
	
	for (TPair<UAnimationAsset*, UAnimationAsset*>& Pair : DuplicatedAnimAssets)
	{
		UAnimSequence* SourceSequence = Cast<UAnimSequence>(Pair.Key);
		UAnimSequence* TargetSequence = Cast<UAnimSequence>(Pair.Value);
		if (!(SourceSequence && TargetSequence))
		{
			continue;
		}

		// increment progress bar
		if (Progress.ShouldCancel())
		{
			return;
		}

		// all curves were copied when we duplicated the animation sequence, so now we have to modify curves based upon the operations in the Op Stack
		IAnimationDataController& TargetSeqController = TargetSequence->GetController();
		constexpr bool bShouldTransact = false;
		TargetSeqController.OpenBracket(FText::FromString("Applying Curve Operations"), bShouldTransact);

		FIKRetargetCurvesOpBase::FFrameValues InputCurveFrameValues;
		FIKRetargetCurvesOpBase::FCurveData InputCurveMetaData;

		TArray<FFrameTime> InputFrameTimes;
		bool bLoadedData = FCurveUtils::LoadCurveValuesFromAnimSequence(SourceSequence, SourceSkeleton, InputCurveMetaData.Names, InputFrameTimes, InputCurveFrameValues, InputCurveMetaData.Flags, InputCurveMetaData.Colors);
		if (bLoadedData)
		{
			// Save a copy of the original (pre-op) per-frame values so we can detect which
			// curves were actually modified by a curve op. Unmodified curves will be copied
			// directly from source to preserve tangents and exact key times.
			const FIKRetargetCurvesOpBase::FFrameValues OriginalCurveFrameValues = InputCurveFrameValues;

			FIKRetargetCurvesOpBase::FFrameValues OutputCurveFrameValues;
			FIKRetargetCurvesOpBase::FCurveData OutputCurveMetaData;

			for (const FInstancedStruct& OpStruct : InProcessor.GetRetargetOps())
			{
				const UStruct* StructType = OpStruct.GetScriptStruct();
				if (StructType && StructType->IsChildOf(FIKRetargetCurvesOpBase::StaticStruct()))
				{
					const FIKRetargetCurvesOpBase& Op = OpStruct.Get<FIKRetargetCurvesOpBase>();
					if (Op.IsEnabled())
					{
						Op.ProcessAnimSequenceCurves(InputCurveMetaData, InputCurveFrameValues, OutputCurveMetaData, OutputCurveFrameValues);

						// Move everything so that it is input for the next stack op
						InputCurveMetaData = MoveTemp(OutputCurveMetaData);
						InputCurveFrameValues = MoveTemp(OutputCurveFrameValues);
					}
				}
			}

			// set the output values and metadata into the anim sequence
			// note that outputs are contained in the Input data variables since they are moved there after each operation
			const FFrameRate InputFrameRate = FCurveUtils::GetAnimSequenceRate(SourceSequence);
			AddCurveValuesToAnimSequence(SourceSequence, SourceSkeleton, TargetSkeleton, InputCurveMetaData, InputCurveFrameValues, OriginalCurveFrameValues, InputFrameRate, InputFrameTimes, bShouldTransact, TargetSeqController);
		}
			
		if(!bLoadedData)
		{
			UE_LOGF(LogTemp, Warning, "No animation curves found for retargeting in Anim Sequence: %ls", *SourceSequence->GetName());
		}

		TargetSeqController.CloseBracket(bShouldTransact);
	}
}


void UIKRetargetBatchOperation::AddCurveValuesToAnimSequence(
	const UAnimSequence* InSourceSequence,
	USkeleton* InSourceSkeleton,
	USkeleton* InTargetSkeleton,
	const FIKRetargetCurvesOpBase::FCurveData& InCurveMetaData,
	const FIKRetargetCurvesOpBase::FFrameValues& InCurveValuesPerFrame,
	const FIKRetargetCurvesOpBase::FFrameValues& InOriginalCurveValuesPerFrame,
	const FFrameRate& InFrameRate,
	const TArray<FFrameTime>& InFrameTimes,
	bool bInShouldTransact,
	IAnimationDataController& OutTargetSeqController) const
{
	// remove existing float curves, as we will replace them all
	OutTargetSeqController.RemoveAllCurvesOfType(ERawCurveTrackTypes::RCT_Float, bInShouldTransact);

	const IAnimationDataModel* SourceDataModel = InSourceSequence->GetDataModel();

	for (int32 CurveIndex = 0; CurveIndex < InCurveMetaData.Names.Num(); CurveIndex++)
	{
		const FName& CurveName = InCurveMetaData.Names[CurveIndex];
		FAnimationCurveIdentifier CurveId = UAnimationCurveIdentifierExtensions::GetCurveIdentifier(InTargetSkeleton, CurveName, ERawCurveTrackTypes::RCT_Float);

		if (!CurveId.IsValid())
		{
			continue;
		}

		OutTargetSeqController.AddCurve(CurveId, InCurveMetaData.Flags[CurveIndex], bInShouldTransact);

		// Detect whether this curve was modified by any curve op by comparing the final
		// per-frame values against the original values loaded from source.
		bool bCurveWasModified = InCurveValuesPerFrame.Num() != InOriginalCurveValuesPerFrame.Num();
		if (!bCurveWasModified)
		{
			for (int32 FrameIndex = 0; FrameIndex < InCurveValuesPerFrame.Num() && !bCurveWasModified; FrameIndex++)
			{
				const TOptional<float> Original = InOriginalCurveValuesPerFrame[FrameIndex].IsValidIndex(CurveIndex) ? InOriginalCurveValuesPerFrame[FrameIndex][CurveIndex] : TOptional<float>();
				const TOptional<float> Modified = InCurveValuesPerFrame[FrameIndex].IsValidIndex(CurveIndex) ? InCurveValuesPerFrame[FrameIndex][CurveIndex] : TOptional<float>();
				if (Original.IsSet() != Modified.IsSet() || (Original.IsSet() && !FMath::IsNearlyEqual(Original.GetValue(), Modified.GetValue())))
				{
					bCurveWasModified = true;
				}
			}
		}

		if (!bCurveWasModified)
		{
			// Curve was not modified by any op — copy keys directly from source to preserve
			// tangents and exact key times (including negative-frame keys).
			const FAnimationCurveIdentifier SourceCurveId = UAnimationCurveIdentifierExtensions::FindCurveIdentifier(InSourceSkeleton, CurveName, ERawCurveTrackTypes::RCT_Float);
			if (const FFloatCurve* SourceCurve = SourceDataModel->FindFloatCurve(SourceCurveId))
			{
				for (const FRichCurveKey& Key : SourceCurve->FloatCurve.Keys)
				{
					OutTargetSeqController.SetCurveKey(CurveId, Key, bInShouldTransact);
				}
			}
		}
		else
		{
			// Curve was modified by a curve op — write baked per-frame values.
			// Tangent data is not preserved in this path as the op pipeline works on
			// sampled values rather than spline keys.
			for (int32 FrameIndex = 0; FrameIndex < InFrameTimes.Num(); FrameIndex++)
			{
				const double Time = InFrameRate.AsSeconds(InFrameTimes[FrameIndex]);
				const TOptional<float> Value = InCurveValuesPerFrame[FrameIndex][CurveIndex];
				if (Value.IsSet())
				{
					OutTargetSeqController.SetCurveKey(CurveId, FRichCurveKey(static_cast<float>(Time), Value.GetValue()), bInShouldTransact);
				}
			}
		}

		OutTargetSeqController.SetCurveColor(CurveId, InCurveMetaData.Colors[CurveIndex], bInShouldTransact);
	}
}


void UIKRetargetBatchOperation::OverwriteExistingAssets(const FIKRetargetBatchOperationContext& Context, FScopedSlowTask& Progress)
{
	if (!Context.bOverwriteExistingFiles)
	{
		return;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

	// for each retargeted asset, check if we need to replace an existing asset (one with the desired name, same location, same type)
	for (TPair<UAnimationAsset*, UAnimationAsset*>& Pair : DuplicatedAnimAssets)
	{
		UAnimationAsset* OldAsset = Pair.Key;
		UAnimationAsset* NewAsset = Pair.Value;
		
		// get desired name
		FString DesiredObjectName = Context.NameRule.Rename(OldAsset);
		if (NewAsset->GetName() == DesiredObjectName)
		{
			// asset was not renamed due to collision with existing asset, so there's nothing to replace
			continue;
		}

		// destination path
		FString PathName = FPackageName::GetLongPackagePath(NewAsset->GetPathName());
		FString DesiredPackageName = PathName + "/" + DesiredObjectName;
		FString DesiredObjectPath = DesiredPackageName + "." + DesiredObjectName;
		FAssetData AssetDataToReplace = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(DesiredObjectPath));
		const bool bHasDuplicateToReplace = AssetDataToReplace.IsValid() && AssetDataToReplace.GetAsset()->GetClass() == OldAsset->GetClass();
		if (!bHasDuplicateToReplace)
		{
			// this could happen if the desired name was already in use by a different asset type
			continue;
		}

		// reroute all references from old asset to new asset
		UObject* AssetToReplace = AssetDataToReplace.GetAsset();
		TArray<UObject*> AssetsToReplace = {AssetToReplace};
		ObjectTools::ForceReplaceReferences(NewAsset, AssetsToReplace);
		
		// delete the old asset
		ObjectTools::ForceDeleteObjects({AssetToReplace}, false /*bShowConfirmation*/);
			
		// rename the new asset with the desired name
		FString CurrentAssetPath = NewAsset->GetPathName();
		TArray<FAssetRenameData> AssetsToRename = { FAssetRenameData(CurrentAssetPath, DesiredObjectPath) };
		AssetToolsModule.Get().RenameAssets(AssetsToRename);
	}
}

void UIKRetargetBatchOperation::NotifyUserOfResults(
	const FIKRetargetBatchOperationContext& Context,
	FScopedSlowTask& Progress) const
{
	// skip UI interaction if batch retargeter is ran in headless mode
	if (IsRunningCommandlet() || FApp::IsUnattended() )
	{
		return;
	}
	
	// gather newly created objects
	TArray<UObject*> NewAssets;
	GetNewAssets(NewAssets);
	
	// select all new assets and show in the content browser
	TArray<FAssetData> CurrentSelection;
	for(UObject* NewObject : NewAssets)
	{
		if (NewObject)
		{
			CurrentSelection.Add(FAssetData(NewObject));
		}
	}
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets(CurrentSelection);

	// create pop-up notification in editor UI
	constexpr float NotificationDuration = 5.0f;
	if (Progress.ShouldCancel())
	{
		Progress.EnterProgressFrame(1.f, FText(LOCTEXT("CancelledBatchRetarget", "Cancelled.")));
		
		// notify user that retarget was cancelled
		FNotificationInfo Notification(FText::GetEmpty());
		Notification.ExpireDuration = NotificationDuration;
		Notification.Text = FText(LOCTEXT("BatchRetargetCancelled", "Batch retarget cancelled."));
		FSlateNotificationManager::Get().AddNotification(Notification);
	}
	else
	{
		Progress.EnterProgressFrame(1.f, FText(LOCTEXT("DoneBatchRetarget", "Duplicate and retarget complete!")));
		
		// log details of what assets were created
		for (const UObject* NewAsset : NewAssets)
		{
			UE_LOGF(LogTemp, Display, "Duplicate and Retarget - New Asset Created: %ls", *NewAsset->GetName());
		}
	
		// notify user that retarget completed
		FNotificationInfo Notification(FText::GetEmpty());
		Notification.ExpireDuration = NotificationDuration;
		Notification.Text = FText::Format(
			LOCTEXT("MultiNonDuplicatedAsset", "{0} assets were retargeted to new skeleton {1}. See Output for details."),
			FText::AsNumber(NewAssets.Num()),
			FText::FromString(Context.TargetMesh->GetName()));
		FSlateNotificationManager::Get().AddNotification(Notification);
	}
}

void UIKRetargetBatchOperation::GetNewAssets(TArray<UObject*>& NewAssets) const
{
	TArray<UAnimationAsset*> NewAnims;
	DuplicatedAnimAssets.GenerateValueArray(NewAnims);
	for (UAnimationAsset* NewAnim : NewAnims)
	{
		NewAssets.Add(Cast<UObject>(NewAnim));
	}

	TArray<UAnimBlueprint*> NewBlueprints;
	DuplicatedBlueprints.GenerateValueArray(NewBlueprints);
	for (UAnimBlueprint* NewBlueprint : NewBlueprints)
	{
		NewAssets.Add(Cast<UObject>(NewBlueprint));
	}
}

void UIKRetargetBatchOperation::CleanupIfCancelled(const FScopedSlowTask& Progress) const
{
	if (!Progress.ShouldCancel())
	{
		return;
	}

	// get list of all the assets that were created
	// (to be removed after being cancelled)
	TArray<UObject*> NewAssets;
	GetNewAssets(NewAssets);
	
	// delete any newly created assets
	constexpr bool bShowConfirmation = true;
	ObjectTools::DeleteObjects(NewAssets, bShowConfirmation);
}

void FAdditiveRetargetSettings::PrepareForRetarget(UAnimSequence* InSequenceAsset)
{
	if (!ensure(InSequenceAsset))
	{
		return;
	}

	SequenceAsset = InSequenceAsset;

	// store setting values
	AdditiveAnimType = SequenceAsset->AdditiveAnimType;
	RefPoseType = SequenceAsset->RefPoseType;
	RefFrameIndex = SequenceAsset->RefFrameIndex;
	RefPoseSeq = SequenceAsset->RefPoseSeq;

	// remove all additive settings so that retarget happens on base motion
	SequenceAsset->AdditiveAnimType = EAdditiveAnimationType::AAT_None;
	SequenceAsset->RefPoseType = EAdditiveBasePoseType::ABPT_None;
	SequenceAsset->RefFrameIndex = 0;
	SequenceAsset->RefPoseSeq = nullptr;
}

void FAdditiveRetargetSettings::RestoreOnAsset() const
{
	if (!ensure(SequenceAsset))
	{
		return;
	}
		
	SequenceAsset->AdditiveAnimType = AdditiveAnimType;
	SequenceAsset->RefPoseType = RefPoseType;
	SequenceAsset->RefFrameIndex = RefFrameIndex;
	SequenceAsset->RefPoseSeq = RefPoseSeq;
}

TArray<FAssetData> UIKRetargetBatchOperation::RunBatchRetarget(const FIKRetargetBatchOperationInputs& Inputs)
{
	// fill the context with all the data needed to run a batch retarget
	FIKRetargetBatchOperationContext Context;
	for (const FAssetData& Asset : Inputs.AssetsToRetarget)
	{
		if (UObject* Object = Asset.GetAsset())
		{
			Context.AssetsToRetarget.Add(Object);
		}
	}

	Context.SourceMesh = Inputs.SourceMesh;
	Context.TargetMesh = Inputs.TargetMesh;
	Context.IKRetargetAsset = Inputs.IKRetargetAsset;
	
	Context.NameRule.Prefix = Inputs.Prefix;
	Context.NameRule.Suffix = Inputs.Suffix;
	Context.NameRule.ReplaceFrom = Inputs.Search;
	Context.NameRule.ReplaceTo = Inputs.Replace;
	
	if (!Inputs.TargetPath.IsEmpty())
	{
		Context.NameRule.FolderPath = Inputs.TargetPath;
	}

	Context.bUseSourcePath = Inputs.bUseSourcePath;
	Context.bIncludeReferencedAssets = Inputs.bIncludeReferencedAssets;
	Context.bOverwriteExistingFiles = Inputs.bOverwriteExistingFiles;
	Context.bRetainAdditiveFlags = Inputs.bRetainAdditiveFlags;
	Context.OverrideSetNames = Inputs.InOverrideSetNames;

	// actually run the batch operation
	UIKRetargetBatchOperation* BatchOperation = NewObject<UIKRetargetBatchOperation>();
	BatchOperation->AddToRoot();
	BatchOperation->RunRetarget(Context);

	// create array of FAssetData to return
	TArray<FAssetData> Results;
	for (const UAnimationAsset* RetargetedAsset : BatchOperation->AnimationAssetsToRetarget)
	{
		Results.Add(FAssetData(RetargetedAsset));
	}
	
	BatchOperation->RemoveFromRoot();
	return Results;
}

TArray<FAssetData> UIKRetargetBatchOperation::DuplicateAndRetarget(
	const TArray<FAssetData>& AssetsToRetarget,
	USkeletalMesh* SourceMesh,
	USkeletalMesh* TargetMesh,
	UIKRetargeter* IKRetargetAsset,
	const FString& Search,
	const FString& Replace,
	const FString& Prefix,
	const FString& Suffix,
	const FString& TargetPath,
	const bool bUseSourcePath,
	const bool bIncludeReferencedAssets,
	const bool bOverwriteExistingFiles)
{
	// pack parameters into the new struct
	FIKRetargetBatchOperationInputs Inputs;
	Inputs.AssetsToRetarget = AssetsToRetarget;
	Inputs.SourceMesh = SourceMesh;
	Inputs.TargetMesh = TargetMesh;
	Inputs.IKRetargetAsset = IKRetargetAsset;
	Inputs.Search = Search;
	Inputs.Replace = Replace;
	Inputs.Prefix = Prefix;
	Inputs.Suffix = Suffix;
	Inputs.TargetPath = TargetPath;
	Inputs.bUseSourcePath = bUseSourcePath;
	Inputs.bIncludeReferencedAssets = bIncludeReferencedAssets;
	Inputs.bOverwriteExistingFiles = bOverwriteExistingFiles;

	// route the call to the new consolidated function
	return RunBatchRetarget(Inputs);
}

void UIKRetargetBatchOperation::RunRetarget(FIKRetargetBatchOperationContext& Context)
{
	Reset();
	
	// validate animation assets were provided
	const int32 NumAssets = GenerateAssetLists(Context);
	if (NumAssets == 0)
	{
		UE_LOGF(LogTemp, Warning, "Batch retarget aborted. No animation assets were specified.");
		return;
	}

	// validate a retarget asset was provided
	if (!Context.IKRetargetAsset)
	{
		UE_LOGF(LogTemp, Warning, "Batch retarget aborted. No IK Retargeter asset was specified.");
		return;
	}

	
	// validate a source mesh was provided
	if (!Context.SourceMesh)
	{
		// fallback to mesh provided by IK Rig
		const UIKRigDefinition* SrcIKRig = Context.IKRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Source);
		if (SrcIKRig)
		{
			Context.SourceMesh = SrcIKRig->GetPreviewMesh();
		}
		
		if (!Context.SourceMesh)
		{
			UE_LOGF(LogTemp, Warning, "Batch retarget aborted. No source mesh was specified and the source IK Rig did not have one. ");
			return;	
		}
	}

	// validate a target mesh was provided
	if (!Context.TargetMesh)
	{
		// fallback to mesh provided by IK Rig
		const UIKRigDefinition* TgtIKRig = Context.IKRetargetAsset->GetIKRig(ERetargetSourceOrTarget::Target);
		if (TgtIKRig)
		{
			Context.TargetMesh = TgtIKRig->GetPreviewMesh();
		}
		
		if (!Context.TargetMesh)
		{
			UE_LOGF(LogTemp, Warning, "Batch retarget aborted. No target mesh was specified and the target IK Rig did not have one. ");
			return;
		}
	}
	
	// show progress bar
	constexpr int32 NumAdditionalProgressFrames = 4;
	constexpr int32 NumPassesOverAssets = 3;
	const int32 NumProgressSteps = (NumAssets * NumPassesOverAssets) + NumAdditionalProgressFrames;
	FScopedSlowTask Progress(static_cast<float>(NumProgressSteps), LOCTEXT("GatheringBatchRetarget", "Gathering animation assets..."));
	constexpr bool bShowCancelButton = true;
	Progress.MakeDialog(bShowCancelButton);
	
	DuplicateRetargetAssets(Context, Progress);
	RetargetAssets(Context, Progress);
	OverwriteExistingAssets(Context, Progress);
	NotifyUserOfResults(Context, Progress);
	CleanupIfCancelled(Progress);
}

void UIKRetargetBatchOperation::Reset()
{
	AnimationAssetsToRetarget.Reset();
	AnimBlueprintsToRetarget.Reset();
	DuplicatedAnimAssets.Reset();
	DuplicatedBlueprints.Reset();
	RemappedAnimAssets.Reset();
}

#undef LOCTEXT_NAMESPACE
