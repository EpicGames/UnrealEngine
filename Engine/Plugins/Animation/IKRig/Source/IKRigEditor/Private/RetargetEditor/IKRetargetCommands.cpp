// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetCommands.h"

#define LOCTEXT_NAMESPACE "IKRetargetCommands"

void FIKRetargetCommands::RegisterCommands()
{
	UI_COMMAND(RunRetargeter, "Run Retargeter", "Resume playback of animation and retarget to the target skeleton.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(EditRetargetPose, "Edit Retarget Pose", "Enter into mode allowing manual editing of the target skeleton pose in the viewport.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowRetargetPose, "Show Retarget Pose", "Display the retarget pose and retarget to the target skeleton.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ShowAssetSettings, "Asset Settings", "Set the source or target IK Rigs, preview meshes. Adjust debug settings.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowOverrideSets, "Override Sets", "Open the Override Sets tab to create / edit sets of op property overrides.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowVariables, "Variables", "Open the Variables tab to create / edit variables to drive property overrides.", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(ResetAllBones, "Reset All", "Sets the selected bones to the mesh reference pose.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetSelectedBones, "Reset Selected Bones", "Sets the selected bones to the mesh reference pose.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ResetSelectedAndChildrenBones, "Reset Selected and Children Bones", "Sets the selected bones (and all children recursively) to the mesh reference pose.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(AlignAllBones, "Align All Bones", "Resets the retarget pose and then auto-aligns ALL retargeted bones using the specified method.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AlignSelected, "Align Selected", "Auto-aligns only selected bones using the specified method.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AlignSelectedAndChildren, "Align Selected and Children", "Auto-aligns selected bones and their children (recursive) using the specified method.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SnapCharacterToGround, "Snap Character to Ground", "Translates the whole skeleton vertically to restore original height from the ground. Uses the selected bone as the reference point for a grounded limb. Otherwise searches the skeleton for the lowest retargeted bone.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(NewRetargetPose, "Create", "Create a new retarget pose.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DuplicateRetargetPose, "Duplicate Current", "Duplicate the current retarget pose.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteRetargetPose, "Delete", "Delete current retarget pose.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RenameRetargetPose, "Rename", "Rename current retarget pose.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportRetargetPose, "Import Pose Asset", "Import a retarget pose asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ImportRetargetPoseFromAnim, "Import from Animation Sequence", "Import a retarget pose from an animation sequence.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportRetargetPose, "Export Pose Asset", "Export current retarget pose as a Pose Asset.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
