// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetDefaultMode.h"

#include "EditorViewportClient.h"
#include "AssetEditorModeManager.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/IKRetargetHitProxies.h"
#include "ReferenceSkeleton.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"


#define LOCTEXT_NAMESPACE "IKRetargetDefaultMode"

FName FIKRetargetDefaultMode::ModeName("IKRetargetAssetDefaultMode");

bool FIKRetargetDefaultMode::GetCameraTarget(FSphere& OutTarget) const
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	return Controller->GetCameraTargetForSelection(OutTarget);
}

IPersonaPreviewScene& FIKRetargetDefaultMode::GetAnimPreviewScene() const
{
	return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

void FIKRetargetDefaultMode::Initialize()
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	bIsInitialized = true;
}

void FIKRetargetDefaultMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);

	if (!EditorController.IsValid())
	{
		return;
	}
	
	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();

	// render source and target skeletons
	Controller->RenderSkeleton(PDI, ERetargetSourceOrTarget::Source);
	Controller->RenderSkeleton(PDI, ERetargetSourceOrTarget::Target);

	// render all the chain and root debug proxies
	RenderDebugProxies(PDI, Controller);
}

void FIKRetargetDefaultMode::RenderDebugProxies(FPrimitiveDrawInterface* PDI, const FIKRetargetEditorController* Controller)
{
	// asset settings can disable debug drawing
	const UIKRetargeter* Asset = Controller->AssetController->GetAsset();
	if (!Asset->bDebugDraw)
	{
		return;
	}

	// skip until processor has been initialized
	const FIKRetargetProcessor* RetargetProcessor = Controller->GetRetargetProcessor();
	if (!(RetargetProcessor && RetargetProcessor->IsInitialized()))
	{
		return;
	}

	// give ops a chance to debug draw
	RetargetProcessor->DebugDrawAllOps(PDI, Controller->GetSelectionState());
}

bool FIKRetargetDefaultMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}

	const bool bLeftButtonClicked = Click.GetKey() == EKeys::LeftMouseButton;
	const bool bCtrlOrShiftHeld = Click.IsControlDown() || Click.IsShiftDown();
	const ESelectionEdit EditMode = bCtrlOrShiftHeld ? ESelectionEdit::Add : ESelectionEdit::Replace;
	
	// did we click on a bone in the viewport?
	const bool bHitBone = HitProxy && HitProxy->IsA(HIKRetargetEditorBoneProxy::StaticGetType());
	if (bLeftButtonClicked && bHitBone)
	{
		const HIKRetargetEditorBoneProxy* BoneProxy = static_cast<HIKRetargetEditorBoneProxy*>(HitProxy);
		const TArray BoneNames{BoneProxy->BoneName};
		constexpr bool bFromHierarchy = false;
		Controller->EditBoneSelection(BoneNames, EditMode, bFromHierarchy);
		return true;
	}

	// did we click on a chain in the viewport?
	const bool bHitChain = HitProxy && HitProxy->IsA(HIKRetargetEditorChainProxy::StaticGetType());
	if (bLeftButtonClicked && bHitChain)
	{
		const HIKRetargetEditorChainProxy* ChainProxy = static_cast<HIKRetargetEditorChainProxy*>(HitProxy);
		const TArray ChainNames{ChainProxy->TargetChainName};
		constexpr bool bFromChainView = false;
		Controller->EditChainSelection(ChainNames, EditMode, bFromChainView);
		return true;
	}

	// did we click on the root in the viewport?
	const bool bHitRoot = HitProxy && HitProxy->IsA(HIKRetargetEditorRootProxy::StaticGetType());
	if (bLeftButtonClicked && bHitRoot)
	{
		Controller->SetRootSelected(true);
		return true;
	}

	// we didn't hit anything, therefore clicked in empty space in viewport
	Controller->ClearSelection(); // deselect all meshes, bones, chains and update details view
	return true;
}

void FIKRetargetDefaultMode::Enter()
{
	IPersonaEditMode::Enter();

	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	// record which skeleton is being viewed/edited
	SkeletonMode = Controller->GetSourceOrTarget();
}

void FIKRetargetDefaultMode::Exit()
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	IPersonaEditMode::Exit();
}

void FIKRetargetDefaultMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);
	
	CurrentWidgetMode = ViewportClient->GetWidgetMode();

	// ensure selection callbacks have been generated
	if (!bIsInitialized)
	{
		Initialize();
	}
}

#undef LOCTEXT_NAMESPACE
