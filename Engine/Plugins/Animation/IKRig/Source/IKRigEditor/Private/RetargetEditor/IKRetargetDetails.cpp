// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetDetails.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Animation/AnimInstance.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "RetargetEditor/IKRetargetEditorController.h"
#include "RetargetEditor/IKRetargeterController.h"
#include "Retargeter/IKRetargeter.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "ScopedTransaction.h"
#include "AnimationRuntime.h"
#include "SPositiveActionButton.h"
#include "Editor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetDetails)

#if WITH_EDITOR
#include "HAL/PlatformApplicationMisc.h"
#endif


#define LOCTEXT_NAMESPACE "IKRetargeterDetails"


FEulerTransform UIKRetargetBoneDetails::GetTransform(EIKRetargetTransformType TransformType, const bool bLocalSpace) const
{
	// editor setup?
	if (!EditorController.IsValid())
	{
		return FEulerTransform::Identity;
	}
	
	// get controller
	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();

	// ensure we have a valid skeletal mesh
	const bool bEditingSource = Controller->GetSourceOrTarget() == ERetargetSourceOrTarget::Source;
	UDebugSkelMeshComponent* Mesh = bEditingSource ? Controller->SourceSkelMeshComponent : Controller->TargetSkelMeshComponent;
	if (!(Mesh && Mesh->GetSkeletalMeshAsset()))
	{
		return FEulerTransform::Identity;
	}

	// ensure bone is valid
	const FReferenceSkeleton& RefSkeleton = Mesh->GetSkeletalMeshAsset()->GetRefSkeleton();
	const int32 BoneIndex = RefSkeleton.FindBoneIndex(SelectedBone);
	if (BoneIndex == INDEX_NONE)
	{
		return FEulerTransform::Identity;
	}
	
	switch(TransformType)
	{
	case EIKRetargetTransformType::Current:
		{
			if (bLocalSpace)
			{
				const TArray<FTransform>& LocalTransforms = Mesh->GetBoneSpaceTransforms();
				return LocalTransforms.IsValidIndex(BoneIndex) ? FEulerTransform(LocalTransforms[BoneIndex]) : FEulerTransform::Identity;
			}
			else
			{
				return FEulerTransform(Mesh->GetBoneTransform(BoneIndex, FTransform::Identity));
			}
		}
	
	case EIKRetargetTransformType::Reference:
		{
			if (bLocalSpace)
			{
				return FEulerTransform(RefSkeleton.GetRefBonePose()[BoneIndex]);
			}
			else
			{
				return FEulerTransform(FAnimationRuntime::GetComponentSpaceTransform(RefSkeleton, RefSkeleton.GetRefBonePose(), BoneIndex));
			}
		}
		
	case EIKRetargetTransformType::RelativeOffset:
		{
			// this is the only stored data we have for bone pose offsets
			const ERetargetSourceOrTarget SourceOrTarget = Controller->GetSourceOrTarget();
			const FRotator LocalRotationDelta = Controller->AssetController->GetRotationOffsetForRetargetPoseBone(SelectedBone, SourceOrTarget).Rotator();
			const FVector GlobalTranslationDelta = IsRootBone() ? Controller->AssetController->GetCurrentRetargetPose(SourceOrTarget).GetRootTranslationDelta() : FVector::Zero();
			const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);

			if (bLocalSpace)
			{
				// create partial local delta transform
				FEulerTransform LocalDeltaTransform = FEulerTransform::Identity;
				LocalDeltaTransform.Rotation = LocalRotationDelta;
				// get parent global transform to calculate local translation delta
				FTransform ParentRefGlobalTransform = FTransform::Identity;
				if (ParentIndex != INDEX_NONE)
				{
					ParentRefGlobalTransform = FAnimationRuntime::GetComponentSpaceTransform(RefSkeleton, RefSkeleton.GetRefBonePose(), ParentIndex);
				}
				// now calculate local translational delta from global
				LocalDeltaTransform.SetLocation(ParentRefGlobalTransform.InverseTransformVector(GlobalTranslationDelta));
				return LocalDeltaTransform;
			}
			else
			{
				// get the CURRENT parent global transform and reference LOCAL transform to calculate the
				// current global transform of the bone without any offsets applied
				FTransform ParentGlobalTransform = FTransform::Identity;
				if (ParentIndex != INDEX_NONE)
				{
					ParentGlobalTransform = Mesh->GetBoneTransform(ParentIndex, FTransform::Identity);
				}
				FTransform LocalRefTransform = RefSkeleton.GetRefBonePose()[BoneIndex];
				FTransform GlobalTransformNoOffset = LocalRefTransform * ParentGlobalTransform;
				// get global rotation plus delta
				FQuat GlobalRotationPlusDelta = GlobalTransformNoOffset.GetRotation() * LocalRotationDelta.Quaternion();
				// get global delta rotation
				FQuat GlobalDeltaRotation = GlobalRotationPlusDelta * GlobalTransformNoOffset.GetRotation().Inverse();
				// combine with translation delta
				return FEulerTransform(GlobalTranslationDelta, GlobalDeltaRotation.Rotator(), FVector::OneVector);
			}
		}

	case EIKRetargetTransformType::Bone:
		{
			// this is the only stored data we have for bone pose offsets
			const ERetargetSourceOrTarget SourceOrTarget = Controller->GetSourceOrTarget();
			const FQuat LocalRotationOffset = Controller->AssetController->GetRotationOffsetForRetargetPoseBone(SelectedBone, SourceOrTarget);
			const FVector GlobalTranslationDelta = IsRootBone() ? Controller->AssetController->GetCurrentRetargetPose(SourceOrTarget).GetRootTranslationDelta() : FVector::Zero();
			const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			
			// combine the local space offset from ref pose with the recorded offset in the retarget pose
			FTransform LocalRefTransform = RefSkeleton.GetRefBonePose()[BoneIndex];
			FQuat CombinedLocalRotation = LocalRefTransform.GetRotation() * LocalRotationOffset;

			// get parent global transform to calculate local translation delta
			FTransform ParentRefGlobalTransform = FTransform::Identity;
			if (ParentIndex != INDEX_NONE)
			{
				ParentRefGlobalTransform = FAnimationRuntime::GetComponentSpaceTransform(RefSkeleton, RefSkeleton.GetRefBonePose(), ParentIndex);
			}
			// now calculate local translational delta from global
			FVector LocalTranslation = ParentRefGlobalTransform.InverseTransformVector(GlobalTranslationDelta);
            
			// create an Euler transform of local space
            return FEulerTransform(LocalTranslation, CombinedLocalRotation.Rotator(), FVector::OneVector);
		}
		
	default:
		checkNoEntry();
	}

	return FEulerTransform::Identity;
}

bool UIKRetargetBoneDetails::IsComponentRelative(
	ESlateTransformComponent::Type Component,
	EIKRetargetTransformType TransformType) const
{
	switch(TransformType)
	{
	case EIKRetargetTransformType::Current:
		{
			return CurrentTransformRelative[(int32)Component]; 
		}
	case EIKRetargetTransformType::Reference:
		{
			return ReferenceTransformRelative[(int32)Component]; 
		}
	case EIKRetargetTransformType::RelativeOffset:
		{
			return RelativeOffsetTransformRelative[(int32)Component];
		}
	case EIKRetargetTransformType::Bone:
		{
			return BoneRelative[(int32)Component];
		}
	default:
		checkNoEntry();
	}
	return true;
}

void UIKRetargetBoneDetails::OnComponentRelativeChanged(
	ESlateTransformComponent::Type Component,
	bool bIsRelative,
	EIKRetargetTransformType TransformType)
{
	switch(TransformType)
	{
	case EIKRetargetTransformType::Current:
		{
			CurrentTransformRelative[(int32)Component] = bIsRelative;
			break; 
		}
	case EIKRetargetTransformType::Reference:
		{
			ReferenceTransformRelative[(int32)Component] = bIsRelative;
			break; 
		}
	case EIKRetargetTransformType::RelativeOffset:
		{
			RelativeOffsetTransformRelative[(int32)Component] = bIsRelative;
			break; 
		}
	case EIKRetargetTransformType::Bone:
		{
			BoneRelative[(int32)Component] = true;
			break; 
		}
	default:
		checkNoEntry();
	}
}

void UIKRetargetBoneDetails::OnCopyToClipboard(
	ESlateTransformComponent::Type Component,
	EIKRetargetTransformType TransformType) const
{
	// get is local or global space
	bool bIsRelative = false;
	switch(TransformType)
	{
	case EIKRetargetTransformType::Current:
		{
			bIsRelative = CurrentTransformRelative[(int32)Component];
			break;
		}
	case EIKRetargetTransformType::Reference:
		{
			bIsRelative = ReferenceTransformRelative[(int32)Component];
			break;
		}
	case EIKRetargetTransformType::RelativeOffset:
		{
			bIsRelative = RelativeOffsetTransformRelative[(int32)Component];
			break;
		}
	case EIKRetargetTransformType::Bone:
		{
			bIsRelative = BoneRelative[(int32)Component];
			break;
		}
	default:
		checkNoEntry();
	}

	// get the transform of correct type and space
	const FEulerTransform Transform = GetTransform(TransformType, bIsRelative);
	
	FString Content;
	switch(Component)
	{
	case ESlateTransformComponent::Location:
		{
			GetContentFromData(Transform.GetLocation(), Content);
			break;
		}
	case ESlateTransformComponent::Rotation:
		{
			GetContentFromData(Transform.Rotator(), Content);
			break;
		}
	case ESlateTransformComponent::Scale:
		{
			GetContentFromData(Transform.GetScale3D(), Content);
			break;
		}
	case ESlateTransformComponent::Max:
	default:
		{
			GetContentFromData(Transform, Content);
			TBaseStructure<FTransform>::Get()->ExportText(Content, &Transform, &Transform, nullptr, PPF_None, nullptr);
			break;
		}
	}

	if(!Content.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Content);
	}
}

void UIKRetargetBoneDetails::OnPasteFromClipboard(
	ESlateTransformComponent::Type Component,
	EIKRetargetTransformType TransformType)
{
	// only allow editing of relative offsets in retarget poses
	if(TransformType != EIKRetargetTransformType::RelativeOffset)
	{
		return;
	}

	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return;
	}

	// must have valid controller
	UIKRetargeterController* AssetController = Controller->AssetController;
	if(!AssetController)
	{
		return;
	}
	
	// get content of clipboard to paste
	FString Content;
	FPlatformApplicationMisc::ClipboardPaste(Content);
	if(Content.IsEmpty())
	{
		return;
	}
	
	class FRetargetPasteTransformWidgetErrorPipe : public FOutputDevice
	{
	public:

		int32 NumErrors;

		FRetargetPasteTransformWidgetErrorPipe(FIKRigLogger* InLog)	: FOutputDevice(), NumErrors(0), Log(InLog) {}

		virtual void Serialize(const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category) override
		{
			if (Log)
			{
				Log->LogError(LOCTEXT("RetargetPasteError", "Error pasting transform data to bone."));
			}
			
			NumErrors++;
		}

		FIKRigLogger* Log;
	};

	FIKRigLogger* Log = nullptr;
	if (FIKRetargetProcessor* Processor = Controller->GetRetargetProcessor())
	{
		Log = &Processor->Log;
	}
	FRetargetPasteTransformWidgetErrorPipe ErrorPipe(Log);
	
	// get the transform of correct type and space
	const bool bIsRelative = IsComponentRelative(Component, TransformType);
	FEulerTransform Transform = GetTransform(TransformType, bIsRelative);

	const ERetargetSourceOrTarget SourceOrTarget = Controller->GetSourceOrTarget();

	// create a transaction on the asset
	FScopedTransaction Transaction(LOCTEXT("PasteTransform", "Paste Transform"));
	Controller->AssetController->GetAsset()->Modify();
	
	switch(Component)
	{
	case ESlateTransformComponent::Location:
		{
			FVector Data = Transform.GetLocation();
			const TCHAR* Result = TBaseStructure<FVector>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FVector>::Get()->GetName(), true);
			if (Result && ErrorPipe.NumErrors == 0)
			{
				Transform.SetLocation(Data);
				Controller->AssetController->GetCurrentRetargetPose(SourceOrTarget).SetRootTranslationDelta(Transform.GetLocation());
			}
			break;
		}
	case ESlateTransformComponent::Rotation:
	case ESlateTransformComponent::Max:
		{
			FRotator Data = Transform.Rotator();
			const TCHAR* Result = TBaseStructure<FRotator>::Get()->ImportText(*Content, &Data, nullptr, PPF_None, &ErrorPipe, TBaseStructure<FRotator>::Get()->GetName(), true);
			if (Result && ErrorPipe.NumErrors == 0)
			{
				Transform.SetRotator(Data);
				Controller->AssetController->SetRotationOffsetForRetargetPoseBone(SelectedBone, Transform.GetRotation(), SourceOrTarget);
			}
			break;
		}
	
	default:
		// scale not supported / ignored if pasted
		break;
	}
}

TOptional<FVector::FReal> UIKRetargetBoneDetails::GetNumericValue(
	EIKRetargetTransformType TransformType,
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent)
{
	FEulerTransform Transform = FEulerTransform::Identity;
	
	switch(TransformType)
	{
	case EIKRetargetTransformType::Current:
		{
			Transform = GetTransform(TransformType, CurrentTransformRelative[(int32)Component]);
			break;
		}
	
	case EIKRetargetTransformType::Reference:
		{
			Transform = GetTransform(TransformType, ReferenceTransformRelative[(int32)Component]);
			break;
		}

	case EIKRetargetTransformType::RelativeOffset:
		{
			Transform = GetTransform(TransformType, RelativeOffsetTransformRelative[(int32)Component]);
			break;
		}
	case EIKRetargetTransformType::Bone:
		{
			Transform = GetTransform(TransformType, BoneRelative[(int32)Component]);
			break;
		}
	default:
		checkNoEntry();
	}

	return CleanRealValue(SAdvancedTransformInputBox<FEulerTransform>::GetNumericValueFromTransform(Transform, Component, Representation, SubComponent));
}

void UIKRetargetBoneDetails::OnNumericValueCommitted(
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	FVector::FReal Value,
	ETextCommit::Type CommitType,
	EIKRetargetTransformType TransformType,
	bool bIsCommit)
{
	const bool bIsRelativeOffset = TransformType == EIKRetargetTransformType::RelativeOffset;
	const bool bIsLocal = TransformType == EIKRetargetTransformType::Bone;
	if(!(bIsRelativeOffset || bIsLocal))
	{
		return;
	}

	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return;
	}

	UIKRetargeterController* AssetController = Controller->AssetController;
	if(!AssetController)
	{
		return;
	}

	// ensure we have a valid skeletal mesh
	const bool bEditingSource = Controller->GetSourceOrTarget() == ERetargetSourceOrTarget::Source;
	UDebugSkelMeshComponent* Mesh = bEditingSource ? Controller->SourceSkelMeshComponent : Controller->TargetSkelMeshComponent;
	if (!(Mesh && Mesh->GetSkeletalMeshAsset()))
	{
		return;
	}

	// ensure bone is valid
	const FReferenceSkeleton& RefSkeleton = Mesh->GetSkeletalMeshAsset()->GetRefSkeleton();
	const int32 BoneIndex = RefSkeleton.FindBoneIndex(SelectedBone);
	if (BoneIndex == INDEX_NONE)
	{
		return;
	}

	const ERetargetSourceOrTarget SourceOrTarget = Controller->GetSourceOrTarget();

	switch (TransformType)
	{
	case EIKRetargetTransformType::RelativeOffset:
		{
			CommitValueAsRelativeOffset(
				AssetController,
				RefSkeleton,
				SourceOrTarget,
				BoneIndex,
				Mesh,
				Component,
				Representation,
				SubComponent,
				Value,
				bIsCommit);
			break;
		}
	case EIKRetargetTransformType::Bone:
		{
			CommitValueAsBoneSpace(
				AssetController,
				RefSkeleton,
				SourceOrTarget,
				BoneIndex,
				Mesh,
				Component,
				Representation,
				SubComponent,
				Value,
				bIsCommit);
			break;
		}
	default:
		// cannot edit any other transform types
		checkNoEntry();
	}
}

void UIKRetargetBoneDetails::CommitValueAsRelativeOffset(
	UIKRetargeterController* AssetController,
	const FReferenceSkeleton& RefSkeleton,
	const ERetargetSourceOrTarget SourceOrTarget,
	const int32 BoneIndex,
	UDebugSkelMeshComponent* Mesh,
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	FVector::FReal Value,
	bool bShouldTransact)
{
	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return;
	}

	switch(Component)
	{
	case ESlateTransformComponent::Location:
		{
			const bool bIsTranslationLocal = RelativeOffsetTransformRelative[0];
			FTransform CurrentGlobalOffset = FTransform::Identity;
			CurrentGlobalOffset.SetTranslation(AssetController->GetCurrentRetargetPose(SourceOrTarget).GetRootTranslationDelta());
			
			if (bIsTranslationLocal)
			{	
				// get the current LOCAL offset
				FTransform CurrentLocalOffset = CurrentGlobalOffset;
				FTransform ParentGlobalRefTransform = FTransform::Identity;
				const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					ParentGlobalRefTransform = FAnimationRuntime::GetComponentSpaceTransform(RefSkeleton, RefSkeleton.GetRefBonePose(), ParentIndex);
				}
				CurrentLocalOffset = CurrentLocalOffset.GetRelativeTransform(ParentGlobalRefTransform);

				// apply the numerical value to the local space values
				SAdvancedTransformInputBox<FTransform>::ApplyNumericValueChange(CurrentLocalOffset, Value, Component, Representation, SubComponent);

				// convert back to global space for storage in the pose
				CurrentGlobalOffset = CurrentLocalOffset * ParentGlobalRefTransform;
			}
			else
			{
				// apply the edit
				SAdvancedTransformInputBox<FTransform>::ApplyNumericValueChange(CurrentGlobalOffset, Value, Component, Representation, SubComponent);
			}
			
			// store the new transform in the retarget pose
			FScopedTransaction Transaction(LOCTEXT("EditPelvisTranslation", "Edit Pelvis Pose Translation"), bShouldTransact);
			Controller->AssetController->GetAsset()->Modify();
			Controller->AssetController->GetCurrentRetargetPose(SourceOrTarget).SetRootTranslationDelta(CurrentGlobalOffset.GetTranslation());
			
			break;
		}
	case ESlateTransformComponent::Rotation:
		{
			const bool bIsRotationLocal = RelativeOffsetTransformRelative[1];
			FQuat NewLocalRotationDelta;
			
			if (bIsRotationLocal)
			{
				const FQuat LocalRotationDelta = AssetController->GetRotationOffsetForRetargetPoseBone(SelectedBone, SourceOrTarget);
				FEulerTransform LocalDeltaTransform = FEulerTransform(FVector::ZeroVector, LocalRotationDelta.Rotator(), FVector::OneVector);
				
				// rotations are stored in local space, so just apply the edit
				SAdvancedTransformInputBox<FEulerTransform>::ApplyNumericValueChange(LocalDeltaTransform, Value, Component, Representation, SubComponent);
				NewLocalRotationDelta = LocalDeltaTransform.GetRotation();
			}
			else
			{
				FTransform ParentGlobalTransform = FTransform::Identity;
				const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					ParentGlobalTransform = Mesh->GetBoneTransform(ParentIndex, FTransform::Identity);
				}
				FTransform LocalRefTransform = RefSkeleton.GetRefBonePose()[BoneIndex];
				FTransform CurrentGlobalTransformNoDelta = LocalRefTransform * ParentGlobalTransform;
				
				// get reference global transform
				// get offset global transform
				const FQuat LocalRotationDelta = AssetController->GetRotationOffsetForRetargetPoseBone(SelectedBone, SourceOrTarget);
				FQuat GlobalRefRotationPlusDelta = CurrentGlobalTransformNoDelta.GetRotation() * LocalRotationDelta;
				// get global delta
				FQuat GlobalRotationOffset = GlobalRefRotationPlusDelta * CurrentGlobalTransformNoDelta.GetRotation().Inverse();
				// apply edit to global delta
				FEulerTransform GlobalDeltaTransform = FEulerTransform(FVector::ZeroVector, GlobalRotationOffset.Rotator(), FVector::OneVector);
				SAdvancedTransformInputBox<FEulerTransform>::ApplyNumericValueChange(GlobalDeltaTransform, Value, Component, Representation, SubComponent);

				// convert world space delta quaternion to bone-space
				FQuat NewGlobalDeltaRotation = GlobalDeltaTransform.GetRotation();
				const FVector RotationAxis = NewGlobalDeltaRotation.GetRotationAxis();
				const FVector UnRotatedAxis = CurrentGlobalTransformNoDelta.InverseTransformVector(RotationAxis);
				NewLocalRotationDelta = FQuat(UnRotatedAxis, NewGlobalDeltaRotation.GetAngle());
			}
			
			// store the new rotation in the retarget pose
			FScopedTransaction Transaction(LOCTEXT("EditRootRotation", "Edit Retarget Pose Rotation"), bShouldTransact);
			Controller->AssetController->GetAsset()->Modify();
			Controller->AssetController->SetRotationOffsetForRetargetPoseBone(SelectedBone, NewLocalRotationDelta, SourceOrTarget);
			break;
		}
	default:
		checkNoEntry();
	}
}

void UIKRetargetBoneDetails::CommitValueAsBoneSpace(
	UIKRetargeterController* AssetController,
	const FReferenceSkeleton& RefSkeleton,
	const ERetargetSourceOrTarget SourceOrTarget,
	const int32 BoneIndex,
	UDebugSkelMeshComponent* Mesh,
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	FVector::FReal Value,
	bool bShouldTransact)
{
	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return;
	}

	switch(Component)
	{
	case ESlateTransformComponent::Location:
		{
			const bool bIsTranslationLocal = BoneRelative[0];
			FTransform CurrentGlobalOffset = FTransform::Identity;
			CurrentGlobalOffset.SetTranslation(AssetController->GetCurrentRetargetPose(SourceOrTarget).GetRootTranslationDelta());
			
			if (bIsTranslationLocal)
			{	
				// get the current LOCAL offset
				FTransform CurrentLocalOffset = CurrentGlobalOffset;
				FTransform ParentGlobalRefTransform = FTransform::Identity;
				const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
				if (ParentIndex != INDEX_NONE)
				{
					ParentGlobalRefTransform = FAnimationRuntime::GetComponentSpaceTransform(RefSkeleton, RefSkeleton.GetRefBonePose(), ParentIndex);
				}
				CurrentLocalOffset = CurrentLocalOffset.GetRelativeTransform(ParentGlobalRefTransform);

				// apply the numerical value to the local space values
				SAdvancedTransformInputBox<FTransform>::ApplyNumericValueChange(CurrentLocalOffset, Value, Component, Representation, SubComponent);

				// convert back to global space for storage in the pose
				CurrentGlobalOffset = CurrentLocalOffset * ParentGlobalRefTransform;
			}
			else
			{
				// apply the edit
				SAdvancedTransformInputBox<FTransform>::ApplyNumericValueChange(CurrentGlobalOffset, Value, Component, Representation, SubComponent);
			}
			
			// store the new transform in the retarget pose
			FScopedTransaction Transaction(LOCTEXT("EditPelvisTranslation", "Edit Pelvis Pose Translation"), bShouldTransact);
			Controller->AssetController->GetAsset()->Modify();
			Controller->AssetController->GetCurrentRetargetPose(SourceOrTarget).SetRootTranslationDelta(CurrentGlobalOffset.GetTranslation());
			
			break;
		}
	case ESlateTransformComponent::Rotation:
		{
			// combine the local space offset from ref pose with the recorded offset in the retarget pose
			FTransform LocalRefTransform = RefSkeleton.GetRefBonePose()[BoneIndex];
			const FQuat LocalRotationOffset = AssetController->GetRotationOffsetForRetargetPoseBone(SelectedBone, SourceOrTarget);
			FQuat CombinedLocalRotation = LocalRefTransform.GetRotation() * LocalRotationOffset;
			FEulerTransform CombinedLocalDeltaTransform = FEulerTransform(FVector::ZeroVector, CombinedLocalRotation.Rotator(), FVector::OneVector);
		
			// rotations are stored in local space, so just apply the edit
			SAdvancedTransformInputBox<FEulerTransform>::ApplyNumericValueChange(CombinedLocalDeltaTransform, Value, Component, Representation, SubComponent);

			// subtract the local space from the result to be left with JUST the retarget pose offset
			FQuat NewLocalRotationDelta = LocalRefTransform.GetRotation().Inverse() * CombinedLocalDeltaTransform.GetRotation();
		
			// store the new rotation in the retarget pose
			FScopedTransaction Transaction(LOCTEXT("EditRootRotation", "Edit Retarget Pose Rotation"), bShouldTransact);
			Controller->AssetController->GetAsset()->Modify();
			Controller->AssetController->SetRotationOffsetForRetargetPoseBone(SelectedBone, NewLocalRotationDelta, SourceOrTarget);
			
			break;
		}
	default:
		checkNoEntry();
	}
}

bool UIKRetargetBoneDetails::IsRootBone() const
{
	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return false;
	}

	const FName RootBone = Controller->AssetController->GetPelvisBone(Controller->GetSourceOrTarget());
	return SelectedBone == RootBone;
}

void UIKRetargetBoneDetails::OnMultiNumericValueCommitted(
	ESlateTransformComponent::Type Component,
	ESlateRotationRepresentation::Type Representation,
	ESlateTransformSubComponent::Type SubComponent,
	FVector::FReal Value,
	ETextCommit::Type CommitType,
	EIKRetargetTransformType TransformType,
	TArrayView<TObjectPtr<UIKRetargetBoneDetails>> Bones,
	bool bIsCommit)
{
	for(TObjectPtr<UIKRetargetBoneDetails> Bone : Bones)
	{	
		Bone->OnNumericValueCommitted(Component, Representation, SubComponent, Value, CommitType, TransformType, bIsCommit);
	}
}

template <typename DataType>
void UIKRetargetBoneDetails::GetContentFromData(const DataType& InData, FString& Content) const
{
	TBaseStructure<DataType>::Get()->ExportText(Content, &InData, &InData, nullptr, PPF_None, nullptr);
}

TOptional<FVector::FReal> UIKRetargetBoneDetails::CleanRealValue(TOptional<FVector::FReal> InValue)
{
	// remove insignificant decimal noise and sign bit if value near zero
	if (InValue.IsSet() && FMath::IsNearlyZero(InValue.GetValue(), UE_KINDA_SMALL_NUMBER))
	{
		InValue = 0.0f;
	}
	
	return InValue;
}

// ------------------------------------------- BEGIN FIKRetargetBoneDetailCustomization -------------------------------

void FIKRetargetBoneDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized = DetailBuilder.GetSelectedObjects();
	Bones.Reset();
	for (const TWeakObjectPtr<UObject>& Object : ObjectsBeingCustomized)
	{
		if (UIKRetargetBoneDetails* Bone = Cast<UIKRetargetBoneDetails>(Object.Get()))
		{
			Bones.Add(Bone);
		}
	}
	
	if (Bones.IsEmpty())
	{
		return;
	}

	if (!Bones[0]->EditorController.IsValid())
	{
		return;
	}
	
	const FIKRetargetEditorController& Controller = *Bones[0]->EditorController.Pin().Get();
	const UIKRetargeterController* AssetController = Controller.AssetController;
	
	const bool bIsEditingPose = Controller.IsEditingPose();

	const FName CurrentRootName = AssetController->GetPelvisBone(Controller.GetSourceOrTarget());
	const bool bIsRootSelected =  Bones[0]->SelectedBone == CurrentRootName;

	FIKRetargetTransformUIData UIData;
	GetTransformUIData( bIsEditingPose, DetailBuilder, UIData);

	TSharedPtr<SSegmentedControl<EIKRetargetTransformType>> TransformChoiceWidget =
		SSegmentedControl<EIKRetargetTransformType>::Create(
			UIData.TransformTypes,
			UIData.ButtonLabels,
			UIData.ButtonTooltips,
			UIData.VisibleTransforms
		);

	DetailBuilder.EditCategory(TEXT("Selection")).SetSortOrder(1);

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("Transforms"));
	CategoryBuilder.SetSortOrder(2);
	CategoryBuilder.AddCustomRow(FText::FromString(TEXT("TransformType")))
	.ValueContent()
	.MinDesiredWidth(375.f)
	.MaxDesiredWidth(375.f)
	.HAlign(HAlign_Left)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			TransformChoiceWidget.ToSharedRef()
		]
	];

	SAdvancedTransformInputBox<FTransform>::FArguments TransformWidgetArgs = SAdvancedTransformInputBox<FTransform>::FArguments()
	.ConstructLocation(!bIsEditingPose || (bIsEditingPose && bIsRootSelected))
	.ConstructRotation(true)
	.ConstructScale(!bIsEditingPose)
	.DisplayRelativeWorld(true)
	.DisplayScaleLock(false)
	.AllowEditRotationRepresentation(true)
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.UseQuaternionForRotation(true);

	TArrayView< TObjectPtr<UIKRetargetBoneDetails> > BonesView = TArrayView< TObjectPtr<UIKRetargetBoneDetails> >(Bones);
	
	for(int32 PropertyIndex=0;PropertyIndex<UIData.Properties.Num();PropertyIndex++)
	{
		const EIKRetargetTransformType TransformType = UIData.TransformTypes[PropertyIndex];

		// only enable editing of the relative offset transform type while in edit mode
		const bool bIsRelativeOffset = TransformType == EIKRetargetTransformType::RelativeOffset;
		const bool bIsBoneOffset = TransformType == EIKRetargetTransformType::Bone;
		const bool bIsEditable = bIsEditingPose && (bIsRelativeOffset || bIsBoneOffset);
		
		TransformWidgetArgs.IsEnabled(bIsEditable);
		// edit transform
		if(bIsEditable)
		{
			TransformWidgetArgs.OnNumericValueCommitted_Static(
				&UIKRetargetBoneDetails::OnMultiNumericValueCommitted,
				TransformType,
				BonesView,
				true);
			
			TransformWidgetArgs.OnNumericValueChanged_Static(
				&UIKRetargetBoneDetails::OnMultiNumericValueCommitted,
				ETextCommit::Default,
				TransformType,
				BonesView,
				false);

			TransformWidgetArgs.OnBeginSliderMovement_Lambda([](
				ESlateTransformComponent::Type Component,
				ESlateRotationRepresentation::Type Representation,
				ESlateTransformSubComponent::Type SubComponent)
			{
				GEditor->BeginTransaction(LOCTEXT("EditRetargetPoseSlider", "Edit Retarget Pose Transform Slider"));
			});
			
			TransformWidgetArgs.OnEndSliderMovement_Lambda([](
				ESlateTransformComponent::Type Component,
				ESlateRotationRepresentation::Type Representation,
				ESlateTransformSubComponent::Type SubComponent,
				double)
			{
				GEditor->EndTransaction();
			});
		}

		// get/set relative
		TransformWidgetArgs.OnGetIsComponentRelative_Lambda( [bIsEditable, BonesView, TransformType](ESlateTransformComponent::Type InComponent)
		{
			return BonesView.ContainsByPredicate( [&](const TObjectPtr<UIKRetargetBoneDetails> Bone)
			{
				return Bone->IsComponentRelative(InComponent, TransformType);
			} );
		})
		.OnIsComponentRelativeChanged_Lambda( [bIsEditable, BonesView, TransformType](ESlateTransformComponent::Type InComponent, bool bIsRelative)
		{
			for (const TObjectPtr<UIKRetargetBoneDetails>& Bone: BonesView)
			{
				Bone->OnComponentRelativeChanged(InComponent, bIsRelative, TransformType);
			}
		} );

		TransformWidgetArgs.OnGetNumericValue_Lambda([BonesView, TransformType](
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent) -> TOptional<FVector::FReal>
		{
			if (BonesView.IsEmpty() || !BonesView[0]->IsValidLowLevel())
			{
				return TOptional<FVector::FReal>();
			}
			
			TOptional<FVector::FReal> FirstValue = BonesView[0]->GetNumericValue(TransformType, Component, Representation, SubComponent);
			
			if (FirstValue)
			{
				for (int32 Index = 1; Index < BonesView.Num(); Index++)
				{
					const TOptional<FVector::FReal> CurrentValue = BonesView[Index]->GetNumericValue(TransformType, Component, Representation, SubComponent);
					if (CurrentValue.IsSet())
					{
						// using a more permissive precision to avoid "Multiple Values" in details panel caused
						// by floating point noise introduced by normal rotation calculations
						constexpr double EDITING_PRECISION = 1.e-2;
						if(! FMath::IsNearlyEqual(FirstValue.GetValue(), CurrentValue.GetValue(), EDITING_PRECISION))
						{
							return TOptional<FVector::FReal>();
						}
					}
				}
			}
			return FirstValue;
		});

		// copy/paste bones transforms
		TransformWidgetArgs.OnCopyToClipboard_UObject(Bones[0].Get(), &UIKRetargetBoneDetails::OnCopyToClipboard, TransformType);
		TransformWidgetArgs.OnPasteFromClipboard_UObject(Bones[0].Get(), &UIKRetargetBoneDetails::OnPasteFromClipboard, TransformType);

		TransformWidgetArgs.Visibility_Lambda([TransformChoiceWidget, TransformType]() -> EVisibility
		{
			return TransformChoiceWidget->HasValue(TransformType) ? EVisibility::Visible : EVisibility::Collapsed;
		});

		SAdvancedTransformInputBox<FTransform>::ConstructGroupedTransformRows(
			CategoryBuilder, 
			UIData.ButtonLabels[PropertyIndex], 
			UIData.ButtonTooltips[PropertyIndex], 
			TransformWidgetArgs);
	}
}

void FIKRetargetBoneDetailCustomization::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(Bones);
}

FString FIKRetargetBoneDetailCustomization::GetReferencerName() const
{
	return TEXT("FIKRetargetBoneDetailCustomization");
}

void FIKRetargetBoneDetailCustomization::GetTransformUIData(
	const bool bIsEditingPose,
	const IDetailLayoutBuilder& DetailBuilder,
	FIKRetargetTransformUIData& OutData) const
{
	// read-only transform meta data
	static TArray<FIKRetargetTransformWidgetData> ReadOnlyTransformMetaData =
	{
		FIKRetargetTransformWidgetData(
			EIKRetargetTransformType::Current,
			LOCTEXT("CurrentTransform", "Current"),
			LOCTEXT("CurrentBoneTransformTooltip", "The current transform of the bone.")),
		FIKRetargetTransformWidgetData(
			EIKRetargetTransformType::Reference,
			LOCTEXT("ReferenceTransform", "Reference"),
			LOCTEXT("ReferenceBoneTransformTooltip", "The reference transform of the bone."))
	};
	static TAttribute<TArray<EIKRetargetTransformType>> ReadOnlyVisibleTransforms =
		TArray<EIKRetargetTransformType>({EIKRetargetTransformType::Current});

	// editable transform meta data
	static TArray<FIKRetargetTransformWidgetData> EditableTransformMetaData =
	{
		FIKRetargetTransformWidgetData(
			EIKRetargetTransformType::RelativeOffset,
			LOCTEXT("EditableRelativeOffsetTransform", "Relative Offset"),
			LOCTEXT("RelativeOffsetBoneTransformTooltip", "The offset transform in the current retarget pose, relative to the reference pose.")),
		FIKRetargetTransformWidgetData(
			EIKRetargetTransformType::Bone,
			LOCTEXT("EditableBoneTransform", "Bone"),
			LOCTEXT("EditableBoneTransformTooltip", "The offset transform in the current retarget pose, relative to the parent bone.")),
		FIKRetargetTransformWidgetData(
			EIKRetargetTransformType::Reference,
			LOCTEXT("EditableReferenceTransform", "Reference"),
			LOCTEXT("EditableReferenceBoneTransformTooltip", "The transform of the bone in the reference pose."))
	};
	static TAttribute<TArray<EIKRetargetTransformType>> EditableVisibleTransforms =
		TArray<EIKRetargetTransformType>({EIKRetargetTransformType::RelativeOffset});

	if (bIsEditingPose)
	{
		for (const FIKRetargetTransformWidgetData& TransformData : EditableTransformMetaData)
		{
			OutData.TransformTypes.Add(TransformData.TransformType);
			OutData.ButtonLabels.Add(TransformData.ButtonLabel);
			OutData.ButtonTooltips.Add(TransformData.ButtonTooltip);
		}

		OutData.VisibleTransforms = EditableVisibleTransforms;

		OutData.Properties.Append(
			{
			DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRetargetBoneDetails, OffsetTransform)),
			DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRetargetBoneDetails, LocalTransform)),
			DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRetargetBoneDetails, ReferenceTransform))
			});
	}
	else
	{
		for (const FIKRetargetTransformWidgetData& TransformData : ReadOnlyTransformMetaData)
		{
			OutData.TransformTypes.Add(TransformData.TransformType);
			OutData.ButtonLabels.Add(TransformData.ButtonLabel);
			OutData.ButtonTooltips.Add(TransformData.ButtonTooltip);
		}
		OutData.VisibleTransforms = ReadOnlyVisibleTransforms;

		OutData.Properties.Append(
			{
			DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRetargetBoneDetails, CurrentTransform)),
			DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRetargetBoneDetails, ReferenceTransform))
			});
	}
}

#undef LOCTEXT_NAMESPACE

