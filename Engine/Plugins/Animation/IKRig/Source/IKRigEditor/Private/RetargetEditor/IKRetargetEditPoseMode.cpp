// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetEditPoseMode.h"

#include "EditorViewportClient.h"
#include "AssetEditorModeManager.h"
#include "IKRigDebugRendering.h"
#include "Rig/IKRigProcessor.h"
#include "IPersonaPreviewScene.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Preferences/PersonaOptions.h"
#include "RetargetEditor/IKRetargetAnimInstance.h"
#include "RetargetEditor/IKRetargetHitProxies.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "RigEditor/IKRigEditMode.h"


#define LOCTEXT_NAMESPACE "IKRetargeterEditMode"

FName FIKRetargetEditPoseMode::ModeName("IKRetargetAssetEditMode");

bool FIKRetargetEditPoseMode::GetCameraTarget(FSphere& OutTarget) const
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	return Controller->GetCameraTargetForSelection(OutTarget);
}

IPersonaPreviewScene& FIKRetargetEditPoseMode::GetAnimPreviewScene() const
{
	return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

void FIKRetargetEditPoseMode::GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const
{
	// todo: provide warnings
}

void FIKRetargetEditPoseMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);

	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return;
	}

	Controller->RenderSkeleton(PDI, ERetargetSourceOrTarget::Source);
	Controller->RenderSkeleton(PDI, ERetargetSourceOrTarget::Target);
}

bool FIKRetargetEditPoseMode::AllowWidgetMove()
{
	return false;
}

bool FIKRetargetEditPoseMode::ShouldDrawWidget() const
{
	return UsesTransformWidget(CurrentWidgetMode);
}

bool FIKRetargetEditPoseMode::UsesTransformWidget() const
{
	return UsesTransformWidget(CurrentWidgetMode);
}

bool FIKRetargetEditPoseMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	const bool bIsOnlyRootSelected = IsOnlyRootSelected();
	const bool bIsAnyBoneSelected = !Controller->GetSelectedBones().IsEmpty();

	if (!bIsAnyBoneSelected)
	{
		return false; // no bones selected, can't transform anything
	}
	
	if (bIsOnlyRootSelected && CheckMode == UE::Widget::EWidgetMode::WM_Translate)
	{
		return true; // can translate only the root
	}

	if (bIsAnyBoneSelected && CheckMode == UE::Widget::EWidgetMode::WM_Rotate)
	{
		return true; // can rotate any bone
	}
	
	return false;
}

FVector FIKRetargetEditPoseMode::GetWidgetLocation() const
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return FVector::ZeroVector; 
	}

	const TArray<FName>& SelectedBones = Controller->GetSelectedBones();
	if (SelectedBones.IsEmpty())
	{
		return FVector::ZeroVector; 
	}

	const USkeletalMesh* SkeletalMesh = Controller->GetSkeletalMesh(SourceOrTarget);
	if (!SkeletalMesh)
	{
		return FVector::ZeroVector; 
	}
	
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 BoneIndex = RefSkeleton.FindBoneIndex(SelectedBones.Last());
	if (BoneIndex == INDEX_NONE)
	{
		return FVector::ZeroVector;
	}
	
	float Scale;
	FVector Offset;
	GetEditedComponentScaleAndOffset(Scale,Offset);
	
	return Controller->GetGlobalRetargetPoseOfBone(SourceOrTarget, BoneIndex, Scale, Offset).GetTranslation();
}

bool FIKRetargetEditPoseMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}

	
	if (Click.GetKey() != EKeys::LeftMouseButton)
	{
		return false;
	}
	
	// selected bone ?
	if (HitProxy && HitProxy->IsA(HIKRetargetEditorBoneProxy::StaticGetType()))
	{
		HIKRetargetEditorBoneProxy* BoneProxy = static_cast<HIKRetargetEditorBoneProxy*>(HitProxy);
		const TArray<FName> BoneNames{BoneProxy->BoneName};
		constexpr bool bFromHierarchy = false;
		const ESelectionEdit EditMode = Click.IsControlDown() || Click.IsShiftDown() ? ESelectionEdit::Add : ESelectionEdit::Replace;
		Controller->EditBoneSelection(BoneNames, EditMode, bFromHierarchy);
		return true;
	}

	// clicking in empty space clears selection
	Controller->ClearSelection();
	return true;
}

bool FIKRetargetEditPoseMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	TrackingState = FIKRetargetTrackingState::None;
	
	// not manipulating any widget axes, so stop tracking
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	if (CurrentAxis == EAxisList::None)
	{
		return false; 
	}
	
	return HandleBeginTransform(InViewportClient);
}

bool FIKRetargetEditPoseMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return HandleEndTransform(InViewportClient);
}

bool FIKRetargetEditPoseMode::BeginTransform(const FGizmoState& InState)
{
	TrackingState = FIKRetargetTrackingState::None;
	return HandleBeginTransform(Owner->GetFocusedViewportClient());
}

bool FIKRetargetEditPoseMode::EndTransform(const FGizmoState& InState)
{
	return HandleEndTransform(Owner->GetFocusedViewportClient());
}

bool FIKRetargetEditPoseMode::HandleBeginTransform(const FEditorViewportClient* InViewportClient)
{
	if (!InViewportClient)
	{
		return false;
	}

	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; // invalid editor state
	}

	// get state of viewport
	const bool bTranslating = InViewportClient->GetWidgetMode() == UE::Widget::EWidgetMode::WM_Translate;
	const bool bRotating = InViewportClient->GetWidgetMode() == UE::Widget::EWidgetMode::WM_Rotate;
	const bool bAnyBoneSelected = !Controller->GetSelectedBones().IsEmpty();
	const bool bOnlyRootSelected = IsOnlyRootSelected();

	// is any bone being rotated?
	if (bRotating && bAnyBoneSelected)
	{
		// Start a rotation transaction
		GEditor->BeginTransaction(LOCTEXT("RotateRetargetPoseBone", "Rotate Retarget Pose Bone"));
		Controller->AssetController->GetAsset()->Modify();
		TrackingState = FIKRetargetTrackingState::RotatingBone;
		UpdateWidgetTransform();
		return true;
	}

	// is the root being translated?
	if (bTranslating && bOnlyRootSelected)
	{
		// Start a translation transaction
		GEditor->BeginTransaction(LOCTEXT("TranslateRetargetPoseBone", "Translate Retarget Pose Bone"));
		Controller->AssetController->GetAsset()->Modify();
		TrackingState = FIKRetargetTrackingState::TranslatingRoot;
		UpdateWidgetTransform();
		BoneEdit.AccumulatedPositionOffset = Controller->AssetController->GetCurrentRetargetPose(SourceOrTarget).GetRootTranslationDelta();
		return true;
	}

	return false;
}

bool FIKRetargetEditPoseMode::HandleEndTransform(FEditorViewportClient* InViewportClient)
{
	if (!InViewportClient)
	{
		return false;
	}
	
	if (TrackingState == FIKRetargetTrackingState::None)
	{
		const bool bIsRootSelected = IsRootSelected();
		const bool bTranslating = InViewportClient->GetWidgetMode() == UE::Widget::EWidgetMode::WM_Translate;
		// forcibly prevent translation of anything but the root
		if (!bIsRootSelected && bTranslating)
		{
			InViewportClient->SetWidgetMode(UE::Widget::EWidgetMode::WM_Rotate);
		}
		
		return true; // not handled
	}

	GEditor->EndTransaction();
	TrackingState = FIKRetargetTrackingState::None;
	return true;
}

bool FIKRetargetEditPoseMode::InputDelta(
	FEditorViewportClient* InViewportClient,
	FViewport* InViewport,
	FVector& InDrag,
	FRotator& InRot,
	FVector& InScale)
{
	if (TrackingState == FIKRetargetTrackingState::None)
	{
		return false; // not handled
	}

	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}

	const TObjectPtr<UIKRetargeterController> AssetController = Controller->AssetController;
	if (!AssetController)
	{
		return false; 
	}
	
	// rotating any bone
	if (TrackingState == FIKRetargetTrackingState::RotatingBone)
	{
		if(InViewportClient->GetWidgetMode() != UE::Widget::WM_Rotate)
		{
			return false;
		}

		// accumulate the rotation from the viewport gizmo (amount of rotation since tracking started)
		BoneEdit.AccumulatedGlobalOffset = InRot.Quaternion() * BoneEdit.AccumulatedGlobalOffset;

		// convert world space delta quaternion to bone-space
		const FVector RotationAxis = BoneEdit.AccumulatedGlobalOffset.GetRotationAxis();
		const FVector UnRotatedAxis = BoneEdit.GlobalTransform.InverseTransformVector(RotationAxis);
		const FQuat BoneLocalDelta = FQuat(UnRotatedAxis, BoneEdit.AccumulatedGlobalOffset.GetAngle());
		
		// apply rotation delta to all selected bones
		const TArray<FName>& SelectedBones = Controller->GetSelectedBones();
		for (int32 SelectionIndex=0; SelectionIndex<SelectedBones.Num(); ++SelectionIndex)
		{
			const FName& BoneName = SelectedBones[SelectionIndex];

			// apply the new delta to the retarget pose
			const FQuat TotalDeltaRotation = BoneEdit.PreviousDeltaRotation[SelectionIndex] * BoneLocalDelta;
			AssetController->SetRotationOffsetForRetargetPoseBone(BoneName, TotalDeltaRotation, Controller->GetSourceOrTarget());
		}

		return true;
	}
	
	// translating root
	if (TrackingState == FIKRetargetTrackingState::TranslatingRoot)
	{
		if(InViewportClient->GetWidgetMode() != UE::Widget::WM_Translate)
		{
			return false;
		}

		// must scale drag vector by inverse of component scale or else it will create feedback cycle and go to outer space
		float Scale;
		FVector Offset;
		GetEditedComponentScaleAndOffset(Scale,Offset);
		Scale = FMath::Max(Scale, UE_KINDA_SMALL_NUMBER);
		BoneEdit.AccumulatedPositionOffset += InDrag * (1.0f / Scale);
		
		// apply translation delta to root
		AssetController->GetCurrentRetargetPose(SourceOrTarget).SetRootTranslationDelta(BoneEdit.AccumulatedPositionOffset);
		return true;
	}
	
	return false;
}

bool FIKRetargetEditPoseMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}

	if (Controller->GetSelectedBones().IsEmpty())
	{
		return false; // nothing selected to manipulate
	}

	if (TrackingState == FIKRetargetTrackingState::None)
	{
		UpdateWidgetTransform();
	}

	InMatrix = BoneEdit.GlobalTransform.ToMatrixNoScale().RemoveTranslation();
	return true;
}

bool FIKRetargetEditPoseMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

void FIKRetargetEditPoseMode::Enter()
{
	IPersonaEditMode::Enter();
	
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	// clear bone edit
	BoneEdit.Reset();

	// deselect everything except bones
	constexpr bool bKeepBoneSelection = true;
	Controller->ClearSelection(bKeepBoneSelection);
	
	// record skeleton to edit (must be constant between enter/exit)
	SourceOrTarget = Controller->GetSourceOrTarget();
}

void FIKRetargetEditPoseMode::Exit()
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

	IPersonaEditMode::Exit();
}

void FIKRetargetEditPoseMode::GetEditedComponentScaleAndOffset(float& OutScale, FVector& OutOffset) const
{
	OutScale = 1.0f;
	OutOffset = FVector::Zero();

	const bool bIsSource = EditorController.Pin()->GetSourceOrTarget() == ERetargetSourceOrTarget::Source;
	const UIKRetargeter* Asset = EditorController.Pin()->AssetController->GetAsset();
	if (!Asset)
	{
		return;
	}
	
	OutScale = bIsSource ? 1.0f : Asset->TargetMeshScale;
	OutOffset = bIsSource ? Asset->SourceMeshOffset : Asset->TargetMeshOffset;
}

void FIKRetargetEditPoseMode::UpdateWidgetTransform()
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		BoneEdit.GlobalTransform = FTransform::Identity;
		return;
	}

	const TArray<FName>& SelectedBones = Controller->GetSelectedBones();
	if (SelectedBones.IsEmpty())
	{
		BoneEdit.GlobalTransform = FTransform::Identity;
		return;
	}

	const USkeletalMesh* SkeletalMesh = Controller->GetSkeletalMesh(SourceOrTarget);
	if (!SkeletalMesh)
	{
		BoneEdit.GlobalTransform = FTransform::Identity;
		return;
	}

	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	const int32 BoneIndex = RefSkeleton.FindBoneIndex(SelectedBones.Last());
	if (BoneIndex == INDEX_NONE)
	{
		BoneEdit.GlobalTransform = FTransform::Identity;
		return;
	}
	
	float Scale;
	FVector Offset;
	GetEditedComponentScaleAndOffset(Scale,Offset);

	BoneEdit.Name = SelectedBones.Last();
	BoneEdit.Index = BoneIndex;
	BoneEdit.GlobalTransform = Controller->GetGlobalRetargetPoseOfBone(SourceOrTarget, BoneEdit.Index, Scale, Offset);
	BoneEdit.AccumulatedGlobalOffset = FQuat::Identity;

	BoneEdit.PreviousDeltaRotation.Reset();
	const FIKRetargetPose& RetargetPose =  Controller->AssetController->GetCurrentRetargetPose(SourceOrTarget);
	for (int32 SelectionIndex=0; SelectionIndex<SelectedBones.Num(); ++SelectionIndex)
	{
		FQuat PrevDeltaRotation = RetargetPose.GetDeltaRotationForBone(SelectedBones[SelectionIndex]);
		BoneEdit.PreviousDeltaRotation.Add(PrevDeltaRotation);
	}
	
	const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneEdit.Index);
	BoneEdit.ParentGlobalTransform = Controller->GetGlobalRetargetPoseOfBone(SourceOrTarget, ParentIndex, Scale, Offset);
}

bool FIKRetargetEditPoseMode::IsRootSelected() const
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	const TArray<FName>& SelectedBones = Controller->GetSelectedBones();
	const FName& RootName = Controller->AssetController->GetPelvisBone(SourceOrTarget);
	return SelectedBones.Contains(RootName);
}

bool FIKRetargetEditPoseMode::IsOnlyRootSelected() const
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}

	const TArray<FName>& SelectedBones = Controller->GetSelectedBones();
	if (SelectedBones.Num() != 1)
	{
		return false;
	}

	return Controller->AssetController->GetPelvisBone(SourceOrTarget) == SelectedBones[0];
}

void FIKRetargetEditPoseMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);
	
	CurrentWidgetMode = ViewportClient->GetWidgetMode();
}

void FIKRetargetEditPoseMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	FEdMode::DrawHUD(ViewportClient, Viewport, View, Canvas);
}

#undef LOCTEXT_NAMESPACE
