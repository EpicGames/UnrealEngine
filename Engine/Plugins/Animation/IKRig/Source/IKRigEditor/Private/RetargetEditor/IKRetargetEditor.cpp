// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetEditor.h"

#include "AnimationEditorViewportClient.h"
#include "AnimationEditorPreviewActor.h"
#include "EditorModeManager.h"
#include "GameFramework/WorldSettings.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "IPersonaToolkit.h"
#include "IAssetFamily.h"
#include "ISkeletonEditorModule.h"
#include "Preferences/PersonaOptions.h"
#include "AnimCustomInstanceHelper.h"
#include "IPersonaViewport.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetOps.h"
#include "RetargetEditor/IKRetargetAnimInstance.h"
#include "RetargetEditor/IKRetargetCommands.h"
#include "RetargetEditor/IKRetargetEditPoseMode.h"
#include "RetargetEditor/IKRetargetApplicationMode.h"
#include "RetargetEditor/IKRetargetDefaultMode.h"
#include "RetargetEditor/IKRetargetEditorController.h"
#include "RetargetEditor/IKRetargetOverridesTabSummoner.h"
#include "RetargetEditor/IKRetargetVariablesTabSummoner.h"

#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "IKRetargeterEditor"

const FName IKRetargetApplicationModes::IKRetargetApplicationMode("IKRetargetApplicationMode");
const FName IKRetargetEditorAppName = FName(TEXT("IKRetargetEditorApp"));

FIKRetargetEditor::FIKRetargetEditor()
	: EditorController(MakeShared<FIKRetargetEditorController>())
	, PreviousTime(-1.0f)
{
}

void FIKRetargetEditor::InitAssetEditor(
	const EToolkitMode::Type Mode,
    const TSharedPtr<IToolkitHost>& InitToolkitHost, 
    UIKRetargeter* InAsset)
{
	EditorController->Initialize(SharedThis(this), InAsset);

	BindCommands();
	
	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneCreated = FOnPreviewSceneCreated::FDelegate::CreateSP(this, &FIKRetargetEditor::HandlePreviewSceneCreated);
	PersonaToolkitArgs.OnPreviewSceneSettingsCustomized = FOnPreviewSceneSettingsCustomized::FDelegate::CreateSP(this, &FIKRetargetEditor::HandleOnPreviewSceneSettingsCustomized);
	
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(InAsset, PersonaToolkitArgs);

	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(
		Mode, 
		InitToolkitHost, 
		IKRetargetEditorAppName, 
		FTabManager::FLayout::NullLayout, 
		bCreateDefaultStandaloneMenu, 
		bCreateDefaultToolbar, 
		InAsset);

	// this sets the application mode which defines the tab factory that builds the editor layout
	AddApplicationMode(
		IKRetargetApplicationModes::IKRetargetApplicationMode,
		MakeShareable(new FIKRetargetApplicationMode(SharedThis(this), PersonaToolkit->GetPreviewScene())));
	SetCurrentMode(IKRetargetApplicationModes::IKRetargetApplicationMode);

	// set the default editing mode to use in the editor
	GetEditorModeManager().SetDefaultMode(FIKRetargetDefaultMode::ModeName);
	
	// give default editing mode a pointer to the editor controller
	GetEditorModeManager().ActivateMode(FIKRetargetDefaultMode::ModeName);
	FIKRetargetDefaultMode* DefaultMode = GetEditorModeManager().GetActiveModeTyped<FIKRetargetDefaultMode>(FIKRetargetDefaultMode::ModeName);
	DefaultMode->SetEditorController(EditorController);
	GetEditorModeManager().DeactivateMode(FIKRetargetDefaultMode::ModeName);

	// give edit pose mode a pointer to the editor controller
	GetEditorModeManager().ActivateMode(FIKRetargetEditPoseMode::ModeName);
	FIKRetargetEditPoseMode* EditPoseMode = GetEditorModeManager().GetActiveModeTyped<FIKRetargetEditPoseMode>(FIKRetargetEditPoseMode::ModeName);
	EditPoseMode->SetEditorController(EditorController);
	GetEditorModeManager().DeactivateMode(FIKRetargetEditPoseMode::ModeName);

	ExtendToolbar();
	RegenerateMenusAndToolbars();

	// run retarget by default
	EditorController->SetRetargeterMode(ERetargeterOutputMode::RunRetarget);
}

void FIKRetargetEditor::OnClose()
{
	FPersonaAssetEditorToolkit::OnClose();
	EditorController->Close();
}

void FIKRetargetEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_IKRigEditor", "IK Rig Editor"));
	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

void FIKRetargetEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
}

void FIKRetargetEditor::BindCommands()
{
	const FIKRetargetCommands& Commands = FIKRetargetCommands::Get();

	//
	// Retarget output modes
	//
	ToolkitCommands->MapAction(
		Commands.RunRetargeter,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::SetRetargeterMode, ERetargeterOutputMode::RunRetarget),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsReadyToRetarget));

	ToolkitCommands->MapAction(
		Commands.EditRetargetPose,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::SetRetargeterMode, ERetargeterOutputMode::EditRetargetPose),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsCurrentMeshLoaded),
		FIsActionChecked());
	
	// show asset settings in details panel
	ToolkitCommands->MapAction(
		Commands.ShowAssetSettings,
		FExecuteAction::CreateLambda([this]()
		{
			UIKRetargeter* Asset = EditorController->AssetController->GetAsset();
			return EditorController->SetDetailsObject(Asset);
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() ->bool
		{
			const UIKRetargeter* Asset = EditorController->AssetController->GetAsset();
			return EditorController->IsObjectInDetailsView(Asset);	
		}));

	ToolkitCommands->MapAction(
		Commands.ShowOverrideSets,
		FExecuteAction::CreateLambda([this]()
		{
			TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(FIKRetargetOverridesTabSummoner::TabID);
			if (Tab.IsValid())
			{
				Tab->ActivateInParent(ETabActivationCause::SetDirectly);
				
				if (TSharedPtr<SWindow> ParentWindow = Tab->GetParentWindow())
				{
					ParentWindow->BringToFront();
				}
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() ->bool
		{
			TSharedPtr<SDockTab> ExistingTab = TabManager->FindExistingLiveTab(FIKRetargetOverridesTabSummoner::TabID);
			return ExistingTab.IsValid() && ExistingTab->IsForeground();
		}));

	ToolkitCommands->MapAction(
		Commands.ShowVariables,
		FExecuteAction::CreateLambda([this]()
		{
			TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(FIKRetargetVariablesTabSummoner::TabID);
			if (Tab.IsValid())
			{
				Tab->ActivateInParent(ETabActivationCause::SetDirectly);

				if (TSharedPtr<SWindow> ParentWindow = Tab->GetParentWindow())
				{
					ParentWindow->BringToFront();
				}
			}
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() ->bool
		{
			TSharedPtr<SDockTab> ExistingTab = TabManager->FindExistingLiveTab(FIKRetargetVariablesTabSummoner::TabID);
			return ExistingTab.IsValid() && ExistingTab->IsForeground();
		}));

	//
	// Edit pose commands
	//
	
	ToolkitCommands->MapAction(
		Commands.ResetAllBones,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleResetAllBones),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsEditingPose),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.ResetSelectedBones,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleResetSelectedBones),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsEditingPoseWithAnyBoneSelected),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.ResetSelectedAndChildrenBones,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleResetSelectedAndChildrenBones),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsEditingPoseWithAnyBoneSelected),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.NewRetargetPose,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleNewPose),
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.DuplicateRetargetPose,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleDuplicatePose),
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.DeleteRetargetPose,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleDeletePose),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::CanDeletePose),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.RenameRetargetPose,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleRenamePose),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::CanRenamePose),
		EUIActionRepeatMode::RepeatDisabled);

	//
	// Auto-gen retarget pose
	//
	
	ToolkitCommands->MapAction(
		Commands.AlignAllBones,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleAlignBones, false /* include children*/, true /*all bones*/),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsEditingPose),
		EUIActionRepeatMode::RepeatDisabled);
	
	ToolkitCommands->MapAction(
		Commands.AlignSelected,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleAlignBones, false /* include children*/, false /*all bones*/),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsEditingPoseWithAnyBoneSelected),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.AlignSelectedAndChildren,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleAlignBones, true /* include children*/, false /*all bones*/),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsEditingPoseWithAnyBoneSelected),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.SnapCharacterToGround,
		FExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::HandleSnapToGround),
		FCanExecuteAction::CreateSP(EditorController, &FIKRetargetEditorController::IsEditingPose),
		EUIActionRepeatMode::RepeatDisabled);
	

	//
	// Pose exporter
	//
	const TSharedRef<FIKRetargetPoseExporter> PoseExporterRef = EditorController->PoseExporter.ToSharedRef();
	
	ToolkitCommands->MapAction(
		Commands.ImportRetargetPose,
		FExecuteAction::CreateSP(PoseExporterRef, &FIKRetargetPoseExporter::HandleImportFromPoseAsset),
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.ImportRetargetPoseFromAnim,
		FExecuteAction::CreateSP(PoseExporterRef, &FIKRetargetPoseExporter::HandleImportFromSequenceAsset),
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.ExportRetargetPose,
		FExecuteAction::CreateSP(PoseExporterRef, &FIKRetargetPoseExporter::HandleExportPoseAsset),
		FCanExecuteAction(),
		EUIActionRepeatMode::RepeatDisabled);
}

void FIKRetargetEditor::ExtendToolbar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	ToolbarExtender->AddToolBarExtension(
        "Asset",
        EExtensionHook::After,
        GetToolkitCommands(),
        FToolBarExtensionDelegate::CreateSP(this, &FIKRetargetEditor::FillToolbar)
    );
}

void FIKRetargetEditor::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginSection("Retarget Modes");

	ToolbarBuilder.AddToolBarButton(
			FExecuteAction::CreateLambda([this]{ EditorController->SetRetargetModeToPreviousMode(); }),
			NAME_None, 
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(EditorController, &FIKRetargetEditorController::GetRetargeterModeLabel)),
			TAttribute<FText>(), 
			TAttribute<FSlateIcon>::Create(TAttribute<FSlateIcon>::FGetter::CreateSP(EditorController, &FIKRetargetEditorController::GetCurrentRetargetModeIcon))
		);

	ToolbarBuilder.EndSection();
	
	ToolbarBuilder.AddWidget(SNew(SSpacer), NAME_None, true, HAlign_Right);

	ToolbarBuilder.BeginSection("Show Settings");
	{
		ToolbarBuilder.AddToolBarButton(
		FIKRetargetCommands::Get().ShowAssetSettings,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FIKRetargetEditorStyle::Get().GetStyleSetName(),"IKRetarget.AssetSettings"));

		ToolbarBuilder.AddToolBarButton(
		FIKRetargetCommands::Get().ShowOverrideSets,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FIKRetargetEditorStyle::Get().GetStyleSetName(),"IKRetarget.AssetSettings"));

		ToolbarBuilder.AddToolBarButton(
		FIKRetargetCommands::Get().ShowVariables,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FIKRetargetEditorStyle::Get().GetStyleSetName(),"IKRetarget.AssetSettings"));
	}
	ToolbarBuilder.EndSection();
}

FName FIKRetargetEditor::GetToolkitFName() const
{
	return FName("IKRetargetEditor");
}

FText FIKRetargetEditor::GetBaseToolkitName() const
{
	return LOCTEXT("IKRetargetEditorAppLabel", "IK Retarget Editor");
}

FText FIKRetargetEditor::GetToolkitName() const
{
	return FText::FromString(EditorController->AssetController->GetAsset()->GetName());
}

FLinearColor FIKRetargetEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

FString FIKRetargetEditor::GetWorldCentricTabPrefix() const
{
	return TEXT("IKRetargetEditor");
}

void FIKRetargetEditor::Tick(float DeltaTime)
{
	// update with latest offsets and scale values
	EditorController->UpdateSkeletalMeshComponents();

	// retargeter IK planting must be reset when time is reversed or playback jumps ahead 
	const float CurrentTime = EditorController->SourceAnimInstance->GetCurrentTime();
	constexpr float MaxSkipTimeBeforeReset = 0.25f;
	if (CurrentTime < PreviousTime || CurrentTime > PreviousTime + MaxSkipTimeBeforeReset)
	{
		EditorController->OnPlaybackReset();
	}
	PreviousTime = CurrentTime;
	
	// forces viewport to always update, even when mouse pressed down in other tabs
	GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();

	// copy the LOD from the source to ensure they are always in-sync for previewing LOD'd ops
	const int32 SourceLOD = EditorController->SourceSkelMeshComponent->GetForcedLOD();
	EditorController->TargetSkelMeshComponent->SetDebugForcedLOD(SourceLOD);
	EditorController->TargetSkelMeshComponent->bTrackAttachedInstanceLOD = false;
}

TStatId FIKRetargetEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FIKRetargetEditor, STATGROUP_Tickables);
}

void FIKRetargetEditor::HandleViewportCreated(const TSharedRef<class IPersonaViewport>& InViewport)
{
	// register callbacks to allow the asset to store the Bone Size viewport setting
	FEditorViewportClient& ViewportClient = InViewport->GetViewportClient();
	if (FAnimationViewportClient* AnimViewportClient = static_cast<FAnimationViewportClient*>(&ViewportClient))
	{
		AnimViewportClient->OnSetBoneSize.BindLambda([this](float InBoneSize)
		{
			if (UIKRetargeter* Asset = EditorController->AssetController->GetAsset())
			{
				Asset->Modify();
				Asset->BoneDrawSize = InBoneSize;
			}
		});
		
		AnimViewportClient->OnGetBoneSize.BindLambda([this]() -> float
		{
			if (const UIKRetargeter* Asset = EditorController->AssetController->GetAsset())
			{
				return Asset->BoneDrawSize;
			}

			return 1.0f;
		});
	}

	auto GetBorderColorAndOpacity = [this]()
	{
		// no processor or processor not initialized
		const FIKRetargetProcessor* Processor = EditorController.Get().GetRetargetProcessor();
		if (!Processor || !Processor->IsInitialized() )
		{
			return FLinearColor::Red;
		}

		const ERetargeterOutputMode OutputMode = EditorController.Get().GetRetargeterMode();
		if (OutputMode == ERetargeterOutputMode::RunRetarget)
		{
			return FLinearColor::Transparent;
		}

		return FLinearColor::Blue;
	};
	
	InViewport->AddOverlayWidget(
		SNew(SBorder)
		.BorderImage(FIKRetargetEditorStyle::Get().GetBrush( "IKRetarget.Viewport.Border"))
		.BorderBackgroundColor_Lambda(GetBorderColorAndOpacity)
		.Visibility(EVisibility::HitTestInvisible)
		.Padding(0.0f)
		.ShowEffectWhenDisabled(false)
	);
}

void FIKRetargetEditor::HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
{	
	AAnimationEditorPreviewActor* Actor = InPersonaPreviewScene->GetWorld()->SpawnActor<AAnimationEditorPreviewActor>(AAnimationEditorPreviewActor::StaticClass(), FTransform::Identity);
	Actor->SetFlags(RF_Transient);
	InPersonaPreviewScene->SetActor(Actor);
	
	// create the skeletal mesh components
	EditorController->SourceSkelMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
	EditorController->TargetSkelMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);

	// do not process root motion, we need all motion in world space for retargeting to work correctly
	EditorController->SourceSkelMeshComponent->SetProcessRootMotionMode(EProcessRootMotionMode::Ignore);
	EditorController->TargetSkelMeshComponent->SetProcessRootMotionMode(EProcessRootMotionMode::Ignore);

	// show flags only control the source mesh, suppress the default trajectory mode on target mesh to avoid drawing a second trajectory.
	EditorController->TargetSkelMeshComponent->SetVisualizeRootMotionMode(EVisualizeRootMotionMode::None);

	// hide skeletons, we want to do custom rendering
	EditorController->SourceSkelMeshComponent->SkeletonDrawMode = ESkeletonDrawMode::Hidden;
	EditorController->TargetSkelMeshComponent->SkeletonDrawMode = ESkeletonDrawMode::Hidden;

	// don't want selectable meshes as it gets in the way of bone selection
	EditorController->SourceSkelMeshComponent->bSelectable = false;
	EditorController->TargetSkelMeshComponent->bSelectable = false;
	
	// setup and apply an anim instance to the skeletal mesh component
	EditorController->SourceAnimInstance = NewObject<UIKRetargetAnimInstance>(EditorController->SourceSkelMeshComponent, TEXT("IKRetargetSourceAnimScriptInstance"));
	EditorController->TargetAnimInstance = NewObject<UIKRetargetAnimInstance>(EditorController->TargetSkelMeshComponent, TEXT("IKRetargetTargetAnimScriptInstance"));
	SetupAnimInstance();
	
	// set components to use custom animation mode
	EditorController->SourceSkelMeshComponent->SetAnimationMode(EAnimationMode::AnimationCustomMode);
	EditorController->TargetSkelMeshComponent->SetAnimationMode(EAnimationMode::AnimationCustomMode);
	
	// must call AddComponent() BEFORE assigning the mesh to prevent auto-assignment of a default anim instance
	EditorController->SourceRootComponent = NewObject<USceneComponent>(Actor);
	EditorController->SourceSkelMeshComponent->AttachToComponent(EditorController->SourceRootComponent, FAttachmentTransformRules::KeepRelativeTransform);
	InPersonaPreviewScene->AddComponent(EditorController->SourceRootComponent, FTransform::Identity);
	InPersonaPreviewScene->AddComponent(EditorController->SourceSkelMeshComponent, FTransform::Identity);
    InPersonaPreviewScene->AddComponent(EditorController->TargetSkelMeshComponent, FTransform::Identity);
    
    // apply component to the preview scene (must be done BEFORE setting mesh)
    InPersonaPreviewScene->SetPreviewMeshComponent(EditorController->SourceSkelMeshComponent);
    InPersonaPreviewScene->SetAdditionalMeshesSelectable(false);
    
    // assign source mesh to the preview scene (applies the mesh to the source component)
	// (must be done AFTER adding the component to prevent InitAnim() from overriding anim instance)
    USkeletalMesh* SourceMesh = EditorController->GetSkeletalMesh(ERetargetSourceOrTarget::Source);
    InPersonaPreviewScene->SetPreviewMesh(SourceMesh);
    
    // assign target mesh directly to the target component
    USkeletalMesh* TargetMesh = EditorController->GetSkeletalMesh(ERetargetSourceOrTarget::Target);
    EditorController->TargetSkelMeshComponent->SetSkeletalMesh(TargetMesh);
	
	EditorController->FixZeroHeightRetargetRoot(ERetargetSourceOrTarget::Source);
	EditorController->FixZeroHeightRetargetRoot(ERetargetSourceOrTarget::Target);

	// bind a callback to update the UI whenever the retarget processor in the target anim instance is initialized
	EditorController->RetargeterInitializedDelegateHandle = EditorController->GetRetargetProcessor()->OnRetargeterInitialized().AddLambda([this]()
	{
		EditorController->RefreshHierarchyView();
	});
}

void FIKRetargetEditor::SetupAnimInstance()
{
	UIKRetargeter* Asset = EditorController->AssetController->GetAsset();
	
	// configure SOURCE anim instance (will only output retarget pose)
	EditorController->SourceAnimInstance->ConfigureAnimInstance(ERetargetSourceOrTarget::Source, Asset, nullptr);
	// configure TARGET anim instance (will output retarget pose AND retarget pose from source skel mesh component)
	EditorController->TargetAnimInstance->ConfigureAnimInstance(ERetargetSourceOrTarget::Target, Asset, EditorController->SourceSkelMeshComponent);

	EditorController->SourceSkelMeshComponent->PreviewInstance = EditorController->SourceAnimInstance.Get();
	EditorController->TargetSkelMeshComponent->PreviewInstance = EditorController->TargetAnimInstance.Get();
}

void FIKRetargetEditor::HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder) const
{
	DetailBuilder.HideCategory("Additional Meshes");
	DetailBuilder.HideCategory("Physics");
	DetailBuilder.HideCategory("Mesh");
	DetailBuilder.HideCategory("Animation Blueprint");
}

void FIKRetargetEditor::HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView)
{
	InDetailsView->OnFinishedChangingProperties().AddSP(this, &FIKRetargetEditor::OnFinishedChangingDetails);
	InDetailsView->SetObject(EditorController->AssetController->GetAsset());
	EditorController->SetDetailsView(InDetailsView);
}

void FIKRetargetEditor::OnFinishedChangingDetails(const FPropertyChangedEvent& PropertyChangedEvent)
{
	UIKRetargeterController* AssetController = EditorController->AssetController;

	// determine which properties were modified
	const bool bSourceIKRigChanged = PropertyChangedEvent.GetPropertyName() == UIKRetargeter::GetSourceIKRigPropertyName();
	const bool bTargetIKRigChanged = PropertyChangedEvent.GetPropertyName() == UIKRetargeter::GetTargetIKRigPropertyName();
	const bool bSourcePreviewChanged = PropertyChangedEvent.GetPropertyName() == UIKRetargeter::GetSourcePreviewMeshPropertyName();
	const bool bTargetPreviewChanged = PropertyChangedEvent.GetPropertyName() == UIKRetargeter::GetTargetPreviewMeshPropertyName();

	// if no override target mesh has been specified, update the override to reflect the mesh in the ik rig asset
	if (bTargetIKRigChanged)
	{
		UIKRigDefinition* NewIKRig = AssetController->GetIKRigWriteable(ERetargetSourceOrTarget::Target);
		AssetController->SetIKRig(ERetargetSourceOrTarget::Target, NewIKRig);
	}

	// if no override source mesh has been specified, update the override to reflect the mesh in the ik rig asset
	if (bSourceIKRigChanged)
	{
		UIKRigDefinition* NewIKRig = AssetController->GetIKRigWriteable(ERetargetSourceOrTarget::Source);
		AssetController->SetIKRig(ERetargetSourceOrTarget::Source, NewIKRig);
	}

	// if either the source or target meshes are possibly modified, update scene components, anim instance and UI
	if (bSourcePreviewChanged)
	{
		USkeletalMesh* Mesh = AssetController->GetPreviewMesh(ERetargetSourceOrTarget::Source);
		AssetController->SetPreviewMesh(ERetargetSourceOrTarget::Source, Mesh);
	}
	
	if (bTargetPreviewChanged)
	{
		USkeletalMesh* Mesh = AssetController->GetPreviewMesh(ERetargetSourceOrTarget::Target);
		AssetController->SetPreviewMesh(ERetargetSourceOrTarget::Target, Mesh);
	}
}

#undef LOCTEXT_NAMESPACE
