// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetEditorController.h"

#include "AnimPose.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "EditorModeManager.h"
#include "IContentBrowserSingleton.h"
#include "SkeletalDebugRendering.h"
#include "Widgets/Input/SButton.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Animation/PoseAsset.h"
#include "Dialog/SCustomDialog.h"
#include "Preferences/PersonaOptions.h"
#include "IKRigEditor.h"
#include "MeshDescription.h"
#include "SkeletalMeshAttributes.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "RetargetEditor/IKRetargetAnimInstance.h"
#include "RetargetEditor/IKRetargetDefaultMode.h"
#include "RetargetEditor/IKRetargetEditPoseMode.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/IKRetargetEditorStyle.h"
#include "RetargetEditor/IKRetargetHitProxies.h"
#include "RetargetEditor/SIKRetargetChainMapList.h"
#include "RetargetEditor/SIKRetargetHierarchy.h"
#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetOps.h"
#include "RetargetEditor/SRetargetOpStack.h"
#include "RigEditor/SIKRigOutputLog.h"
#include "RigEditor/IKRigController.h"
#include "ScopedTransaction.h"
#include "RigEditor/IKRigDetailCustomizations.h"
#include "RigEditor/IKRigStructViewer.h"
#include "Styling/AppStyle.h"

#include "Misc/ITransaction.h"
#include "RetargetEditor/SIKRetargetOverrideManager.h"
#include "RetargetEditor/SIKRetargetVariablesEditor.h"

#define LOCTEXT_NAMESPACE "IKRetargetEditorController"


FBoundIKRig::FBoundIKRig(UIKRigDefinition* InIKRig, const FIKRetargetEditorController& InController)
{
	check(InIKRig);
	IKRig = InIKRig;
	UIKRigController* IKRigController = UIKRigController::GetController(InIKRig);
	ReInitIKDelegateHandle = IKRigController->OnIKRigNeedsInitialized().AddSP(&InController, &FIKRetargetEditorController::HandleIKRigNeedsInitialized);
	AddedChainDelegateHandle = IKRigController->OnRetargetChainAdded().AddSP(&InController, &FIKRetargetEditorController::HandleRetargetChainAdded);
	RemoveChainDelegateHandle = IKRigController->OnRetargetChainRemoved().AddSP(&InController, &FIKRetargetEditorController::HandleRetargetChainRemoved);
	RenameChainDelegateHandle = IKRigController->OnRetargetChainRenamed().AddSP(&InController, &FIKRetargetEditorController::HandleRetargetChainRenamed);
}

void FBoundIKRig::UnBind() const
{
	if (!IKRig.IsValid())
	{
		return;
	}
	
	UIKRigController* IKRigController = UIKRigController::GetController(IKRig.Get());
	IKRigController->OnIKRigNeedsInitialized().Remove(ReInitIKDelegateHandle);
	IKRigController->OnRetargetChainAdded().Remove(AddedChainDelegateHandle);
	IKRigController->OnRetargetChainRemoved().Remove(RemoveChainDelegateHandle);
	IKRigController->OnRetargetChainRenamed().Remove(RenameChainDelegateHandle);
}

FRetargetPlaybackManager::FRetargetPlaybackManager(const TWeakPtr<FIKRetargetEditorController>& InEditorController)
{
	check(InEditorController.Pin())
	EditorController = InEditorController;
}

void FRetargetPlaybackManager::PlayAnimationAsset(UAnimationAsset* AssetToPlay)
{
	UIKRetargetAnimInstance* AnimInstance = EditorController.Pin()->SourceAnimInstance.Get();
	if (!AnimInstance)
	{
		return;
	}
	
	if (AssetToPlay)
	{
		AnimInstance->SetAnimationAsset(AssetToPlay);
		AnimInstance->SetPlaying(true);
		AnimThatWasPlaying = AssetToPlay;
		
		if (FIKRetargetEditorController* Controller = EditorController.Pin().Get())
		{
			if (UDebugSkelMeshComponent* SourceMesh = Controller->SourceSkelMeshComponent)
			{
				// reset root motion to avoid drawing previous trajectory
				SourceMesh->ConsumeRootMotionPreviousPlaybackTime = 0.0f;
				SourceMesh->SetProcessRootMotionMode(EProcessRootMotionMode::Ignore);
				
				// reset preview options on animation switch (mirrors UDebugSkelMeshComponent::EnablePreview).
				if (!SourceMesh->DoesCurrentAssetHaveRootMotion())
				{
					SourceMesh->SetVisualizeRootMotionMode(EVisualizeRootMotionMode::None);
				}
			}
		}

		// ensure we are running the retargeter so you can see the animation
		if (FIKRetargetEditorController* Controller = EditorController.Pin().Get())
		{
			Controller->SetRetargeterMode(ERetargeterOutputMode::RunRetarget);
		}
	}
}

void FRetargetPlaybackManager::StopPlayback()
{
	UIKRetargetAnimInstance* AnimInstance = EditorController.Pin()->SourceAnimInstance.Get();
	if (!AnimInstance)
	{
		return;
	}

	AnimThatWasPlaying = AnimInstance->GetAnimationAsset();
	AnimInstance->SetPlaying(false);
	AnimInstance->SetAnimationAsset(nullptr);
}

void FRetargetPlaybackManager::PausePlayback()
{
	UIKRetargetAnimInstance* AnimInstance = EditorController.Pin()->SourceAnimInstance.Get();
	if (!AnimInstance)
	{
		return;
	}
	
	if (AnimThatWasPlaying)
	{
		TimeWhenPaused = AnimInstance->GetCurrentTime();
	}

	AnimThatWasPlaying = AnimInstance->GetAnimationAsset();
	AnimInstance->SetPlaying(false);
}

void FRetargetPlaybackManager::ResumePlayback() const
{
	UIKRetargetAnimInstance* AnimInstance = EditorController.Pin()->SourceAnimInstance.Get();
	if (!AnimInstance)
	{
		return;
	}
	
	if (AnimThatWasPlaying)
	{
		AnimInstance->SetAnimationAsset(AnimThatWasPlaying);
		AnimInstance->SetPosition(TimeWhenPaused);
		AnimInstance->SetPlaying(true);
	}
}

bool FRetargetPlaybackManager::IsStopped() const
{
	const UIKRetargetAnimInstance* AnimInstance = EditorController.Pin()->SourceAnimInstance.Get();
	if (!AnimInstance)
	{
		return true;
	}
	
	return !AnimInstance->GetAnimationAsset();
}

void FIKRetargetEditorController::Initialize(TSharedPtr<FIKRetargetEditor> InEditor, UIKRetargeter* InAsset)
{
	Editor = InEditor;
	AssetController = UIKRetargeterController::GetController(InAsset);
	CurrentlyEditingSourceOrTarget = ERetargetSourceOrTarget::Target;
	OutputMode = ERetargeterOutputMode::EditRetargetPose;
	PreviousMode = ERetargeterOutputMode::EditRetargetPose;
	PoseExporter = MakeShared<FIKRetargetPoseExporter>();
	PoseExporter->Initialize(SharedThis(this));
	RefreshPoseList();

	PlaybackManager = MakeUnique<FRetargetPlaybackManager>(SharedThis(this));

	Selection.SelectedBoneNames.Add(ERetargetSourceOrTarget::Source);
	Selection.SelectedBoneNames.Add(ERetargetSourceOrTarget::Target);
	Selection.LastSelectedType = ERetargetSelectionType::NONE;

	// clean the asset before editing
	AssetController->CleanAsset();

	// bind callbacks when SOURCE or TARGET IK Rigs are modified
	BindToIKRigAssets();

	// bind callback when retargeter needs reinitialized
	RetargeterReInitDelegateHandle = AssetController->OnRetargeterNeedsInitialized().AddSP(this, &FIKRetargetEditorController::HandleRetargeterNeedsInitialized);
	// bind callback when retargeter op stack is modified
	OpStackModifiedDelegateHandle = AssetController->OnOpStackModified().AddLambda([this]()
	{
		RefreshOpStackView();
	});
	// bind callback when override sets are modified
	OverrideSetsModifiedDelegateHandle = AssetController->OnOverrideSetsModified().AddLambda([this]()
	{
		RefreshOverrideSetView();
	});
	// bind callback when variables are modified
	VariablesModifiedDelegateHandle = AssetController->OnVariablesModified().AddLambda([this]()
	{
		RefreshVariablesView();
	});
	
	// bind callback when IK Rig asset is replaced with a different asset
	IKRigReplacedDelegateHandle = AssetController->OnIKRigReplaced().AddSP(this, &FIKRetargetEditorController::HandleIKRigReplaced);
	// bind callback when Preview Mesh asset is replaced with a different asset
	PreviewMeshReplacedDelegateHandle = AssetController->OnPreviewMeshReplaced().AddSP(this, &FIKRetargetEditorController::HandlePreviewMeshReplaced);
}

void FIKRetargetEditorController::Close()
{
	AssetController->OnRetargeterNeedsInitialized().Remove(RetargeterReInitDelegateHandle);
	AssetController->OnOverrideSetsModified().Remove(OverrideSetsModifiedDelegateHandle);
	AssetController->OnVariablesModified().Remove(VariablesModifiedDelegateHandle);
	AssetController->OnOpStackModified().Remove(OpStackModifiedDelegateHandle);
	AssetController->OnIKRigReplaced().Remove(IKRigReplacedDelegateHandle);
	AssetController->OnPreviewMeshReplaced().Remove(PreviewMeshReplacedDelegateHandle);
	GetRetargetProcessor()->OnRetargeterInitialized().Remove(RetargeterInitializedDelegateHandle);

	for (const FBoundIKRig& BoundIKRig : BoundIKRigs)
	{
		BoundIKRig.UnBind();
	}
}

bool FIKRetargetEditorController::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	auto TransactionIncludesObject = [&InContext, &TransactionObjectContexts](UObject * Object)
	{
		if (InContext.PrimaryObject == Object)
		{
			return true;		
		}
		for (const TPair<UObject*, FTransactionObjectEvent>& ObjectTransactionPair : TransactionObjectContexts)
		{
			if (ObjectTransactionPair.Get<0>() == Object)
			{
				return true;
			}
		}
		return false;
	};

	const UIKRetargeter* Asset = AssetController->GetAsset();
	if (Asset)
	{
		const bool bIncludesAssetControllerObject = TransactionIncludesObject(AssetController) || TransactionIncludesObject(AssetController->GetAsset());
		const bool bIncludesSourceRig = TransactionIncludesObject(Asset->GetIKRigWriteable(ERetargetSourceOrTarget::Source));
		const bool bIncludesTargetRig = TransactionIncludesObject(Asset->GetIKRigWriteable(ERetargetSourceOrTarget::Target));

		return bIncludesAssetControllerObject || bIncludesSourceRig || bIncludesTargetRig;
	}
		
	return false;
}

void FIKRetargetEditorController::PostUndo(bool bSuccess)
{
	AssetController->CleanAsset();
	HandlePreviewMeshReplaced(ERetargetSourceOrTarget::Source);
	HandleRetargeterNeedsInitialized();
	RefreshAllViews();
}

void FIKRetargetEditorController::PostRedo(bool bSuccess)
{
	AssetController->CleanAsset();
	HandlePreviewMeshReplaced(ERetargetSourceOrTarget::Source);
	HandleRetargeterNeedsInitialized();
	RefreshAllViews();
}

void FIKRetargetEditorController::BindToIKRigAssets()
{
	const UIKRetargeter* Asset = AssetController->GetAsset();
	if (!Asset)
	{
		return;
	}
	
	// unbind previously bound IK Rigs
	for (const FBoundIKRig& BoundIKRig : BoundIKRigs)
	{
		BoundIKRig.UnBind();
	}

	BoundIKRigs.Empty();
	
	if (UIKRigDefinition* SourceIKRig = Asset->GetIKRigWriteable(ERetargetSourceOrTarget::Source))
	{
		BoundIKRigs.Emplace(FBoundIKRig(SourceIKRig, *this));
	}

	if (UIKRigDefinition* TargetIKRig = Asset->GetIKRigWriteable(ERetargetSourceOrTarget::Target))
	{
		BoundIKRigs.Emplace(FBoundIKRig(TargetIKRig, *this));
	}
}

void FIKRetargetEditorController::HandleIKRigNeedsInitialized(UIKRigDefinition* ModifiedIKRig) const
{
	const UIKRetargeter* Retargeter = AssetController->GetAsset();
	check(Retargeter)
	HandleRetargeterNeedsInitialized();
}

void FIKRetargetEditorController::HandleRetargetChainAdded(UIKRigDefinition* ModifiedIKRig) const
{
	check(ModifiedIKRig)
	AssetController->HandleRetargetChainAdded(ModifiedIKRig);
	RefreshAllViews();
}

void FIKRetargetEditorController::HandleRetargetChainRenamed(UIKRigDefinition* ModifiedIKRig, FName OldName, FName NewName) const
{
	check(ModifiedIKRig)
	AssetController->HandleRetargetChainRenamed(ModifiedIKRig, OldName, NewName);
}

void FIKRetargetEditorController::HandleRetargetChainRemoved(UIKRigDefinition* ModifiedIKRig, const FName InChainRemoved) const
{
	check(ModifiedIKRig)
	AssetController->HandleRetargetChainRemoved(ModifiedIKRig, InChainRemoved);
	RefreshAllViews();
}

void FIKRetargetEditorController::HandleRetargeterNeedsInitialized() const
{
	// check for "zero height" pelvis, and prompt user to fix
	FixZeroHeightRetargetRoot(ERetargetSourceOrTarget::Source);
	FixZeroHeightRetargetRoot(ERetargetSourceOrTarget::Target);
	
	ReinitializeRetargeterNoUIRefresh();
}

void FIKRetargetEditorController::ReinitializeRetargeterNoUIRefresh() const
{
	// clear the output log
	ClearOutputLog();

	if(UIKRetargetAnimInstance* AnimInstance = TargetAnimInstance.Get())
	{
		return AnimInstance->ForceInitializeProcessor(GetSkeletalMeshComponent(ERetargetSourceOrTarget::Target));
	}
}

void FIKRetargetEditorController::HandleIKRigReplaced(ERetargetSourceOrTarget SourceOrTarget)
{
	const FText SourceOrTargetText = SourceOrTarget == ERetargetSourceOrTarget::Source ? FText::FromString("Source") : FText::FromString("Target");

	TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("AssignIKRigTitle", "Assign IK Rig to All Ops?")))
		.Content()
		[
			SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("AssignToAllOpsLabel", "Assign new {0} IK Rig to all Ops?"), SourceOrTargetText))
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("AssignLabel", "Assign")),
			SCustomDialog::FButton(LOCTEXT("NoLabel", "No"))
	});

	if (Dialog->ShowModal() == 0)
	{
		const UIKRigDefinition* IKRig = AssetController->GetIKRig(SourceOrTarget);
		AssetController->AssignIKRigToAllOps(SourceOrTarget, IKRig);
	}

	BindToIKRigAssets();
	RefreshAllViews();
}

void FIKRetargetEditorController::HandlePreviewMeshReplaced(ERetargetSourceOrTarget SourceOrTarget)
{
	// pause playback so we can resume after mesh swapped out
	PlaybackManager->PausePlayback();
	
	// set the source and target skeletal meshes on the component
	// NOTE: this must be done AFTER setting the AnimInstance so that the correct root anim node is loaded
	USkeletalMesh* SourceMesh = GetSkeletalMesh(ERetargetSourceOrTarget::Source);
	USkeletalMesh* TargetMesh = GetSkeletalMesh(ERetargetSourceOrTarget::Target);
	SourceSkelMeshComponent->SetSkeletalMesh(SourceMesh);
	TargetSkelMeshComponent->SetSkeletalMesh(TargetMesh);

	// clean bone selections in case of incompatible indices
	CleanSelection(ERetargetSourceOrTarget::Source);
	CleanSelection(ERetargetSourceOrTarget::Target);

	// apply mesh to the preview scene
	TSharedRef<IPersonaPreviewScene> PreviewScene = Editor.Pin()->GetPersonaToolkit()->GetPreviewScene();
	if (PreviewScene->GetPreviewMesh() != SourceMesh)
	{
		PreviewScene->SetPreviewMeshComponent(SourceSkelMeshComponent);
		PreviewScene->SetPreviewMesh(SourceMesh);
		SourceSkelMeshComponent->bCanHighlightSelectedSections = false;
	}

	// re-initializes the anim instances running in the viewport
	if (SourceAnimInstance)
	{
		Editor.Pin()->SetupAnimInstance();	
	}

	// continue playing where we left off
	PlaybackManager->ResumePlayback();
}

UDebugSkelMeshComponent* FIKRetargetEditorController::GetSkeletalMeshComponent(
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceSkelMeshComponent : TargetSkelMeshComponent;
}

UIKRetargetAnimInstance* FIKRetargetEditorController::GetAnimInstance(
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceAnimInstance.Get() : TargetAnimInstance.Get();
}

void FIKRetargetEditorController::UpdateSkeletalMeshComponents() const
{
	auto UpdateMeshComponent = [this](ERetargetSourceOrTarget SourceOrTarget)
	{
		const UIKRetargeter* Asset = AssetController->GetAsset();
		if (!Asset)
		{
			return;
		}
		
		UDebugSkelMeshComponent* SkelMeshComponent = GetSkeletalMeshComponent(SourceOrTarget);
		if (!SkelMeshComponent)
		{
			return;
		}

		// generate component transform
		FTransform ComponentTransform = FTransform::Identity;
		if (SourceOrTarget == ERetargetSourceOrTarget::Source)
		{
			// compose a delta FTransform that scales the source component about the scale pivot
			FIKRetargetProcessor* Processor = GetRetargetProcessor();
			const FRetargetPoseScaleWithPivot& PoseScale = Processor->GetPoseScale(ERetargetSourceOrTarget::Source);
			const FVector TranslationDelta = (1.0 - PoseScale.Factor) * PoseScale.Pivot;
			const FVector Scale(PoseScale.Factor,PoseScale.Factor,PoseScale.Factor);
			FTransform Delta(FQuat::Identity, TranslationDelta, Scale);

			// apply delta to the source component
			ComponentTransform = FTransform(FQuat::Identity, Asset->SourceMeshOffset, FVector::OneVector);
			ComponentTransform = Delta * ComponentTransform;
		}
		else
		{
			const double Scale = Asset->TargetMeshScale;
			ComponentTransform.SetTranslation(Asset->TargetMeshOffset);
			ComponentTransform.SetScale3D(FVector(Scale,Scale,Scale));
		}
			
		// apply component transform
		constexpr bool bSweep = false;
		constexpr FHitResult* OutSweepHitResult = nullptr;
		constexpr ETeleportType Teleport = ETeleportType::TeleportPhysics;
		SkelMeshComponent->SetWorldTransform(ComponentTransform, bSweep, OutSweepHitResult, Teleport);
			
		// toggle mesh visibility
		const bool bIsVisible = SourceOrTarget == ERetargetSourceOrTarget::Source ? Asset->bShowSourceMesh : Asset->bShowTargetMesh;
		SkelMeshComponent->SetVisibility(bIsVisible);	
	};

	UpdateMeshComponent(ERetargetSourceOrTarget::Source);
	UpdateMeshComponent(ERetargetSourceOrTarget::Target);
}

bool FIKRetargetEditorController::GetCameraTargetForSelection(FSphere& OutTarget) const
{
	// center the view on the last selected item
	switch (Selection.LastSelectedType)
	{
	case ERetargetSelectionType::BONE:
		{
			// target the selected bones
			const TArray<FName>& SelectedBones = GetSelectedBones();
			if (SelectedBones.IsEmpty())
			{
				return false;
			}

			TArray<FVector> TargetPoints;
			const UDebugSkelMeshComponent* SkeletalMeshComponent = GetSkeletalMeshComponent(GetSourceOrTarget());
			const FReferenceSkeleton& RefSkeleton = SkeletalMeshComponent->GetReferenceSkeleton();
			TArray<int32> ChildrenIndices;
			for (const FName& SelectedBoneName : SelectedBones)
			{
				const int32 BoneIndex = RefSkeleton.FindBoneIndex(SelectedBoneName);
				if (BoneIndex == INDEX_NONE)
				{
					continue;
				}

				TargetPoints.Add(SkeletalMeshComponent->GetBoneTransform(BoneIndex).GetLocation());
				ChildrenIndices.Reset();
				RefSkeleton.GetDirectChildBones(BoneIndex, ChildrenIndices);
				for (const int32 ChildIndex : ChildrenIndices)
				{
					TargetPoints.Add(SkeletalMeshComponent->GetBoneTransform(ChildIndex).GetLocation());
				}
			}
	
			// create a sphere that contains all the target points
			if (TargetPoints.Num() == 0)
			{
				TargetPoints.Add(FVector::ZeroVector);
			}
			OutTarget = FSphere(&TargetPoints[0], TargetPoints.Num());
			return true;
		}
		
	case ERetargetSelectionType::CHAIN:
		{
			const ERetargetSourceOrTarget SourceOrTarget = GetSourceOrTarget();
			const UIKRigDefinition* IKRig = AssetController->GetIKRig(SourceOrTarget);
			if (!IKRig)
			{
				return false;
			}

			const UDebugSkelMeshComponent* SkeletalMeshComponent = GetSkeletalMeshComponent(GetSourceOrTarget());
			const FReferenceSkeleton& RefSkeleton = SkeletalMeshComponent->GetReferenceSkeleton();

			// get target points from start/end bone of all selected chains on the currently active skeleton (source or target)
			TArray<FVector> TargetPoints;
			const TArray<FName>& SelectedChainNames = GetSelectedChains();
			for (const FName SelectedChainName : SelectedChainNames)
			{
				const FName SourceChain = AssetController->GetSourceChain(SelectedChainName);
				if (SourceChain == NAME_None)
				{
					continue;
				}

				const FName& ChainName = SourceOrTarget == ERetargetSourceOrTarget::Target ? SelectedChainName : SourceChain;
				if (ChainName == NAME_None)
				{
					continue;
				}

				const UIKRigController* RigController = UIKRigController::GetController(IKRig);
				const FBoneChain* BoneChain = RigController->GetRetargetChainByName(ChainName);
				if (!BoneChain)
				{
					continue;
				}

				const int32 StartBoneIndex = RefSkeleton.FindBoneIndex(BoneChain->StartBone.BoneName);
				if (StartBoneIndex != INDEX_NONE)
				{
					TargetPoints.Add(SkeletalMeshComponent->GetBoneTransform(StartBoneIndex).GetLocation());
				}

				const int32 EndBoneIndex = RefSkeleton.FindBoneIndex(BoneChain->EndBone.BoneName);
				if (EndBoneIndex != INDEX_NONE)
				{
					TargetPoints.Add(SkeletalMeshComponent->GetBoneTransform(EndBoneIndex).GetLocation());
				}
			}

			// create a sphere that contains all the target points
			if (TargetPoints.Num() == 0)
			{
				TargetPoints.Add(FVector::ZeroVector);
			}
			OutTarget = FSphere(&TargetPoints[0], TargetPoints.Num());
			return true;	
		}
	case ERetargetSelectionType::ROOT:
	case ERetargetSelectionType::MESH:
	case ERetargetSelectionType::OP:
	case ERetargetSelectionType::NONE:
	default:
		// frame both meshes
		OutTarget = FSphere(0);
		if (const UPrimitiveComponent* SourceComponent = GetSkeletalMeshComponent(ERetargetSourceOrTarget::Source))
		{
			OutTarget += SourceComponent->Bounds.GetSphere();
		}
		if (const UPrimitiveComponent* TargetComponent = GetSkeletalMeshComponent(ERetargetSourceOrTarget::Target))
		{
			OutTarget += TargetComponent->Bounds.GetSphere();
		}
		return true;
	}
}

bool FIKRetargetEditorController::IsEditingPoseWithAnyBoneSelected() const
{
	return IsEditingPose() && !GetSelectedBones().IsEmpty();
}

bool FIKRetargetEditorController::IsBoneRetargeted(const FName& BoneName, ERetargetSourceOrTarget SourceOrTarget) const
{
	// get an initialized processor
	const FIKRetargetProcessor* Processor = GetRetargetProcessor();
	if (!(Processor && Processor->IsInitialized()))
	{
		return false;
	}

	// return true if the bone is mapped to another bone on the source/target
	return Processor->IsBoneMapped(BoneName, SourceOrTarget);
}

FName FIKRetargetEditorController::GetChainDisplayNameForBone(const FName& BoneName, ERetargetSourceOrTarget SourceOrTarget) const
{
	// get an initialized processor
	const FIKRetargetProcessor* Processor = GetRetargetProcessor();
	if (!Processor)
	{
		return NAME_None;
	}

	// special case, retarget pelvis should always display as such
	const FName RootBoneName = Processor->GetPelvisBone(SourceOrTarget, ERetargetOpsToSearch::AssetOps);
	if (BoneName == RootBoneName)
	{
		return FName("Retarget Pelvis");
	}

	auto GetCombinedName = [](const TSet<FName>& Names) -> FName
		{
			if (Names.IsEmpty())
			{
				return NAME_None;
			}
    
			if (Names.Num() == 1)
			{
				return *Names.begin();
			}
    
			FString Result;
			bool bFirst = true;
    
			for (const FName& Name : Names)
			{
				if (!bFirst)
				{
					Result += TEXT(", ");
				}
				Result += Name.ToString();
				bFirst = false;
			}
    
			return FName(Result);
		};
	
	const TSet<FName> ChainsThatContainBone = Processor->GetAllChainNamesForBone(BoneName, SourceOrTarget);
	return GetCombinedName(ChainsThatContainBone);
}

TObjectPtr<UIKRetargetBoneDetails> FIKRetargetEditorController::GetOrCreateBoneDetailsObject(const FName& BoneName)
{
	if (AllBoneDetails.Contains(BoneName))
	{
		return AllBoneDetails[BoneName];
	}

	// create and store a new one
	UIKRetargetBoneDetails* NewBoneDetails = NewObject<UIKRetargetBoneDetails>(AssetController->GetAsset(), FName(BoneName), RF_Transient );
	NewBoneDetails->SelectedBone = BoneName;
	NewBoneDetails->EditorController = SharedThis(this);

	// store it in the map
	AllBoneDetails.Add(BoneName, NewBoneDetails);
	
	return NewBoneDetails;
}

USkeletalMesh* FIKRetargetEditorController::GetSkeletalMesh(const ERetargetSourceOrTarget SourceOrTarget) const
{
	return AssetController ? AssetController->GetPreviewMesh(SourceOrTarget) : nullptr;
}

const USkeleton* FIKRetargetEditorController::GetSkeleton(const ERetargetSourceOrTarget SourceOrTarget) const
{
	if (const USkeletalMesh* Mesh = GetSkeletalMesh(SourceOrTarget))
	{
		return Mesh->GetSkeleton();
	}
	
	return nullptr;
}

UDebugSkelMeshComponent* FIKRetargetEditorController::GetEditedSkeletalMesh() const
{
	return CurrentlyEditingSourceOrTarget == ERetargetSourceOrTarget::Source ? SourceSkelMeshComponent : TargetSkelMeshComponent;
}

const FRetargetSkeleton& FIKRetargetEditorController::GetCurrentlyEditedSkeleton(const FIKRetargetProcessor& Processor) const
{
	return Processor.GetSkeleton(CurrentlyEditingSourceOrTarget);
}

FTransform FIKRetargetEditorController::GetGlobalRetargetPoseOfBone(
	const ERetargetSourceOrTarget SourceOrTarget,
	const int32& BoneIndex,
	const float& Scale,
	const FVector& Offset) const
{
	const UIKRetargetAnimInstance* AnimInstance = GetAnimInstance(SourceOrTarget);
	if (!AnimInstance)
	{
		return FTransform::Identity;
	}
	
	const TArray<FTransform>& GlobalRetargetPose = AnimInstance->GetGlobalRetargetPose();
	if (!GlobalRetargetPose.IsValidIndex(BoneIndex))
	{
		return FTransform::Identity;
	}
	
	// get transform of bone
	FTransform BoneTransform = GlobalRetargetPose[BoneIndex];

	// scale and offset
	BoneTransform.ScaleTranslation(Scale);
	BoneTransform.AddToTranslation(Offset);
	BoneTransform.NormalizeRotation();

	return BoneTransform;
}

void FIKRetargetEditorController::GetGlobalRetargetPoseOfImmediateChildren(
	const FRetargetSkeleton& RetargetSkeleton,
	const int32& BoneIndex,
	const float& Scale,
	const FVector& Offset,
	TArray<int32>& OutChildrenIndices,
	TArray<FVector>& OutChildrenPositions)
{
	OutChildrenIndices.Reset();
	OutChildrenPositions.Reset();
	
	check(RetargetSkeleton.BoneNames.IsValidIndex(BoneIndex))

	// get indices of immediate children
	RetargetSkeleton.GetChildrenIndices(BoneIndex, OutChildrenIndices);

	// get the positions of the immediate children
	const TArray<FTransform>& RetargetPose = RetargetSkeleton.RetargetPoses.GetGlobalRetargetPose();
	for (const int32& ChildIndex : OutChildrenIndices)
	{
		OutChildrenPositions.Emplace(RetargetPose[ChildIndex].GetTranslation());
	}

	// apply scale and offset to positions
	for (FVector& ChildPosition : OutChildrenPositions)
	{
		ChildPosition *= Scale;
		ChildPosition += Offset;
	}
}

FIKRetargetProcessor* FIKRetargetEditorController::GetRetargetProcessor() const
{	
	if(UIKRetargetAnimInstance* AnimInstance = TargetAnimInstance.Get())
	{
		return AnimInstance->GetRetargetProcessor();
	}

	return nullptr;	
}

void FIKRetargetEditorController::OnPlaybackReset() const
{
	if (FIKRetargetProcessor* Processor = GetRetargetProcessor())
	{
		Processor->OnPlaybackReset();
	}
}

void FIKRetargetEditorController::ClearOutputLog() const
{
	if (OutputLogView.IsValid())
	{
		OutputLogView.Get()->ClearLog();
		if (const FIKRetargetProcessor* Processor = GetRetargetProcessor())
		{
			Processor->Log.Clear();
		}
	}
}

bool FIKRetargetEditorController::IsObjectInDetailsView(const UObject* Object)
{
	if (!DetailsView.IsValid())
	{
		return false;
	}

	if (!Object)
	{
		return false;
	}
	
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailsView->GetSelectedObjects();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.Get() == Object)
		{
			return true;
		}
	}

	return false;
}

void FIKRetargetEditorController::RefreshAllViews() const
{
	Editor.Pin()->RegenerateMenusAndToolbars();
	RefreshOpStackView(); // must be before details panel in case details object was deleted
	RefreshDetailsView();
	RefreshAssetBrowserView();
	RefreshHierarchyView();
	RefreshOverrideSetView();
	RefreshVariablesView();
}

void FIKRetargetEditorController::RefreshDetailsView() const
{
	// refresh the details panel, cannot assume tab is not closed
	if (DetailsView.IsValid())
	{
		DetailsView->ForceRefresh();
	}
}

void FIKRetargetEditorController::RefreshAssetBrowserView() const
{
	// refresh the asset browser to ensure it shows compatible sequences
	if (AssetBrowserView.IsValid())
	{
		AssetBrowserView.Get()->RefreshView();
	}
}

void FIKRetargetEditorController::RefreshHierarchyView() const
{
	if (HierarchyView.IsValid())
	{
		HierarchyView.Get()->RefreshTreeView();
	}
}

void FIKRetargetEditorController::RefreshOpStackView() const
{
	if (OpStackView.IsValid())
	{
		OpStackView->RefreshStackView();
	}
}

void FIKRetargetEditorController::RefreshOverrideSetView() const
{
	if (OverrideSetView.IsValid())
	{
		OverrideSetView->RefreshTreeView();
	}
}

void FIKRetargetEditorController::RefreshVariablesView() const
{
	if (VariablesView.IsValid())
	{
		VariablesView->RefreshList();
	}
}

void FIKRetargetEditorController::RefreshPoseList() const
{
	if (HierarchyView.IsValid())
	{
		HierarchyView.Get()->RefreshPoseList();
	}
}

void FIKRetargetEditorController::SetDetailsObject(UObject* DetailsObject) const
{
	if (DetailsView.IsValid())
	{
		DetailsView->SetObject(DetailsObject, true /*forceRefresh*/);
	}
}

void FIKRetargetEditorController::SetDetailsObjects(const TArray<UObject*>& DetailsObjects) const
{
	if (DetailsView.IsValid())
	{
		DetailsView->SetObjects(DetailsObjects);
	}
}

void FIKRetargetEditorController::ShowDetailsForOp(const int32 OpIndex) const
{
	if (AssetController->GetRetargetOpByIndex(OpIndex) == nullptr)
	{
		DetailsView->SetObject(AssetController->GetAsset());
		return;
	}
	
	UIKRetargeter* Asset = AssetController->GetAsset();
	auto MemoryProvider = [Asset, OpIndex]() -> uint8*
	{
		UIKRetargeterController* Controller = UIKRetargeterController::GetController(Asset);
		if (FIKRetargetOpBase* OpToEdit = Controller->GetRetargetOpByIndex(OpIndex))
		{
			return reinterpret_cast<uint8*>(OpToEdit->GetSettings());
		}
		
		return nullptr;
	};

	FIKRetargetOpBase* Op = AssetController->GetRetargetOpByIndex(OpIndex);

	FIKRigStructToView StructToView;
	StructToView.Type = Op->GetSettingsType();
	StructToView.MemoryProvider = MemoryProvider;
	StructToView.Owner = AssetController->GetAsset();
	StructToView.UniqueName = Op->GetName();
	
	UIKRigStructViewer* StructViewer = AssetController->GetStructViewer(ERetargetStructViewerMode::Settings);
	StructViewer->SetStructToView(StructToView);
	
	constexpr bool bForceRefresh = true; // must forcibly refresh because struct viewer is a recycled UObject
	DetailsView->SetObject(StructViewer, bForceRefresh);
}

float FIKRetargetEditorController::GetRetargetPoseAmount() const
{
	return RetargetPosePreviewBlend;
}

void FIKRetargetEditorController::SetRetargetPoseAmount(float InValue)
{
	if (OutputMode==ERetargeterOutputMode::RunRetarget)
	{
		SetRetargeterMode(ERetargeterOutputMode::EditRetargetPose);
	}
	
	RetargetPosePreviewBlend = InValue;
	SourceAnimInstance->SetRetargetPoseBlend(RetargetPosePreviewBlend);
	TargetAnimInstance->SetRetargetPoseBlend(RetargetPosePreviewBlend);
}

void FIKRetargetEditorController::SetSourceOrTargetMode(ERetargetSourceOrTarget NewMode)
{
	// already in this mode, so do nothing
	if (NewMode == CurrentlyEditingSourceOrTarget)
	{
		return;
	}
	
	// store the new skeleton mode
	CurrentlyEditingSourceOrTarget = NewMode;

	// if we switch source/target while in edit mode we need to re-enter that mode
	if (GetRetargeterMode() == ERetargeterOutputMode::EditRetargetPose)
	{
		FEditorModeTools& EditorModeManager = Editor.Pin()->GetEditorModeManager();
		FIKRetargetEditPoseMode* EditMode = EditorModeManager.GetActiveModeTyped<FIKRetargetEditPoseMode>(FIKRetargetEditPoseMode::ModeName);
		if (EditMode)
		{
			// FIKRetargetEditPoseMode::Enter() is reentrant and written so we can switch between editing
			// source / target skeleton without having to enter/exit the mode; just call Enter() again
			EditMode->Enter();
		}
	}

	// make sure details panel updates with selected bone on OTHER skeleton
	if (Selection.LastSelectedType == ERetargetSelectionType::BONE)
	{
		const TArray<FName>& SelectedBones = GetSelectedBones();
		EditBoneSelection(SelectedBones, ESelectionEdit::Replace);
	}
	
	RefreshAllViews();
	RefreshPoseList();
}

void FIKRetargetEditorController::EditBoneSelection(
	const TArray<FName>& InBoneNames,
	ESelectionEdit EditMode,
	const bool bFromHierarchyView)
{
	// must have a skeletal mesh
	UDebugSkelMeshComponent* DebugComponent = GetEditedSkeletalMesh();
	if (!DebugComponent->GetSkeletalMeshAsset())
	{
		return;
	}

	Selection.LastSelectedType = ERetargetSelectionType::BONE;
	
	SetRootSelected(false);
	
	switch (EditMode)
	{
		case ESelectionEdit::Add:
		{
			for (const FName& BoneName : InBoneNames)
			{
				Selection.SelectedBoneNames[CurrentlyEditingSourceOrTarget].AddUnique(BoneName);
			}
			
			break;
		}
		case ESelectionEdit::Remove:
		{
			for (const FName& BoneName : InBoneNames)
			{
				Selection.SelectedBoneNames[CurrentlyEditingSourceOrTarget].Remove(BoneName);
			}
			break;
		}
		case ESelectionEdit::Replace:
		{
			Selection.SelectedBoneNames[CurrentlyEditingSourceOrTarget] = InBoneNames;
			break;
		}
		default:
			checkNoEntry();
	}

	// update hierarchy view
	if (!bFromHierarchyView)
	{
		RefreshHierarchyView();
	}
	else
	{
		// If selection was made from the hierarchy view, the viewport must be invalidated for the
		// new widget hit proxies to be activated. Otherwise user has to click in the viewport first to gain focus.
		Editor.Pin()->GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
	}

	// update details
	if (Selection.SelectedBoneNames[CurrentlyEditingSourceOrTarget].IsEmpty())
	{
		SetDetailsObject(AssetController->GetAsset());
	}
	else
	{
		TArray<UObject*> SelectedBoneDetails;
		for (const FName& SelectedBone : Selection.SelectedBoneNames[CurrentlyEditingSourceOrTarget])
		{
			TObjectPtr<UIKRetargetBoneDetails> BoneDetails = GetOrCreateBoneDetailsObject(SelectedBone);
			SelectedBoneDetails.Add(BoneDetails);
		}
		SetDetailsObjects(SelectedBoneDetails);
	}
}

void FIKRetargetEditorController::EditChainSelection(
	const TArray<FName>& InChainNames,
	ESelectionEdit EditMode,
	const bool bFromChainsView)
{
	// deselect others
	SetRootSelected(false);

	Selection.LastSelectedType = ERetargetSelectionType::CHAIN;
	
	// update selection set based on edit mode
	switch (EditMode)
	{
	case ESelectionEdit::Add:
		{
			for (const FName& ChainName : InChainNames)
			{
				Selection.SelectedChains.AddUnique(ChainName);
			}
			
			break;
		}
	case ESelectionEdit::Remove:
		{
			for (const FName& ChainName : InChainNames)
			{
				Selection.SelectedChains.Remove(ChainName);
			}
			break;
		}
	case ESelectionEdit::Replace:
		{
			Selection.SelectedChains = InChainNames;
			break;
		}
	default:
		checkNoEntry();
	}
}

void FIKRetargetEditorController::SetRootSelected(const bool bIsSelected)
{
	Selection.bIsRootSelected = bIsSelected;
	if (!bIsSelected)
	{
		return;
	}

	Selection.LastSelectedType = ERetargetSelectionType::ROOT;
}

void FIKRetargetEditorController::CleanSelection(ERetargetSourceOrTarget SourceOrTarget)
{
	USkeletalMesh* SkeletalMesh = GetSkeletalMesh(SourceOrTarget);
	if (!SkeletalMesh)
	{
		Selection.SelectedBoneNames[SourceOrTarget].Empty();
		return;
	}

	TArray<FName> CleanedSelection;
	const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
	for (const FName& SelectedBone : Selection.SelectedBoneNames[SourceOrTarget])
	{
		if (RefSkeleton.FindBoneIndex(SelectedBone))
		{
			CleanedSelection.Add(SelectedBone);
		}
	}

	Selection.SelectedBoneNames[SourceOrTarget] = CleanedSelection;
}

void FIKRetargetEditorController::ClearSelection(const bool bKeepBoneSelection)
{
	// clear root and mesh selection
	SetRootSelected(false);
	
	// deselect all chains
	Selection.SelectedChains.Empty();

	// clear bone selection
	if (!bKeepBoneSelection)
	{
		SetRootSelected(false);
		Selection.SelectedBoneNames[ERetargetSourceOrTarget::Source].Reset();
		Selection.SelectedBoneNames[ERetargetSourceOrTarget::Target].Reset();
	}

	Selection.LastSelectedType = ERetargetSelectionType::NONE;

	SetDetailsObject(AssetController->GetAsset());
}

void FIKRetargetEditorController::SetOpSelected(const int32 InOpIndex)
{
	const FName NameOfOp = AssetController->GetOpName(InOpIndex);
	if (!ensureAlways(NameOfOp != NAME_None))
	{
		return;
	}
	Selection.LastSelectedType = ERetargetSelectionType::OP;
	Selection.LastSelectedOpName = NameOfOp;
	ShowDetailsForOp(InOpIndex);
}

FName FIKRetargetEditorController::GetSelectedOpName() const
{
	if (Selection.LastSelectedType == ERetargetSelectionType::OP)
	{
		return Selection.LastSelectedOpName;	
	}
	return NAME_None;
}

FIKRetargetOpBase* FIKRetargetEditorController::GetSelectedOp() const
{
	if (!OpStackView.IsValid())
	{
		return nullptr;
	}
	
	return AssetController->GetRetargetOpByName(Selection.LastSelectedOpName);
}

int32 FIKRetargetEditorController::GetSelectedOpIndex() const
{
	if (!OpStackView.IsValid())
	{
		return INDEX_NONE;
	}

	if (Selection.LastSelectedType != ERetargetSelectionType::OP)
	{
		return INDEX_NONE;
	}
	
	return AssetController->GetIndexOfOpByName(Selection.LastSelectedOpName);
}

FName FIKRetargetEditorController::GetSelectedOverrideSet() const
{
	if (!OverrideSetView.IsValid())
	{
		return NAME_None;
	}

	return OverrideSetView.Get()->GetSelectedOverrideSetName();
}

void FIKRetargetEditorController::SetRetargeterMode(ERetargeterOutputMode Mode)
{
	if (OutputMode == Mode)
	{
		return;
	}
		
	PreviousMode = OutputMode;
	
	FEditorModeTools& EditorModeManager = Editor.Pin()->GetEditorModeManager();
	
	switch (Mode)
	{
		case ERetargeterOutputMode::EditRetargetPose:
			// enter edit mode
			EditorModeManager.DeactivateMode(FIKRetargetDefaultMode::ModeName);
			EditorModeManager.ActivateMode(FIKRetargetEditPoseMode::ModeName);
			OutputMode = ERetargeterOutputMode::EditRetargetPose;
			SourceAnimInstance->SetRetargetMode(ERetargeterOutputMode::EditRetargetPose);
			TargetAnimInstance->SetRetargetMode(ERetargeterOutputMode::EditRetargetPose);
			PlaybackManager->PausePlayback();
			SetRetargetPoseAmount(1.0f);
			break;
		
		case ERetargeterOutputMode::RunRetarget:
			EditorModeManager.DeactivateMode(FIKRetargetEditPoseMode::ModeName);
			EditorModeManager.ActivateMode(FIKRetargetDefaultMode::ModeName);
			OutputMode = ERetargeterOutputMode::RunRetarget;
			SourceAnimInstance->SetRetargetMode(ERetargeterOutputMode::RunRetarget);
			TargetAnimInstance->SetRetargetMode(ERetargeterOutputMode::RunRetarget);
			PlaybackManager->ResumePlayback();
			break;
		
		default:
			checkNoEntry();
	}

	// details view displays differently depending on output mode
	RefreshDetailsView();
}

FText FIKRetargetEditorController::GetRetargeterModeLabel()
{
	switch (GetRetargeterMode())
	{
	case ERetargeterOutputMode::RunRetarget:
		return FText::FromString("Running Retarget");
	case ERetargeterOutputMode::EditRetargetPose:
		return FText::FromString("Editing Retarget Pose");
	default:
		checkNoEntry();
		return FText::FromString("Unknown Mode.");
	}
}

FSlateIcon FIKRetargetEditorController::GetCurrentRetargetModeIcon() const
{
	return GetRetargeterModeIcon(GetRetargeterMode());
}

FSlateIcon FIKRetargetEditorController::GetRetargeterModeIcon(ERetargeterOutputMode Mode) const
{
	switch (Mode)
	{
	case ERetargeterOutputMode::RunRetarget:
		return FSlateIcon(FIKRetargetEditorStyle::Get().GetStyleSetName(), "IKRetarget.RunRetargeter");
	case ERetargeterOutputMode::EditRetargetPose:
		return FSlateIcon(FIKRetargetEditorStyle::Get().GetStyleSetName(), "IKRetarget.EditRetargetPose");
	default:
		checkNoEntry();
		return FSlateIcon(FIKRetargetEditorStyle::Get().GetStyleSetName(), "IKRetarget.ShowRetargetPose");
	}
}

bool FIKRetargetEditorController::IsReadyToRetarget() const
{
	return GetRetargetProcessor()->IsInitialized();
}

bool FIKRetargetEditorController::IsCurrentMeshLoaded() const
{
	return GetSkeletalMesh(GetSourceOrTarget()) != nullptr;
}

bool FIKRetargetEditorController::IsEditingPose() const
{
	return GetRetargeterMode() == ERetargeterOutputMode::EditRetargetPose;
}

void FIKRetargetEditorController::HandleNewPose()
{
	SetRetargeterMode(ERetargeterOutputMode::EditRetargetPose);
	
	// get a unique pose name to use as suggestion
	const FString DefaultNewPoseName = LOCTEXT("NewRetargetPoseName", "CustomRetargetPose").ToString();
	const FName UniqueNewPoseName = AssetController->MakePoseNameUnique(DefaultNewPoseName, GetSourceOrTarget());
	
	SAssignNew(NewPoseWindow, SWindow)
	.Title(LOCTEXT("NewRetargetPoseOptions", "Create New Retarget Pose"))
	.ClientSize(FVector2D(300, 80))
	.HasCloseButton(true)
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(4)
			.AutoHeight()
			[
				SAssignNew(NewPoseEditableText, SEditableTextBox)
				.MinDesiredWidth(275)
				.Text(FText::FromName(UniqueNewPoseName))
			]

			+ SVerticalBox::Slot()
			.Padding(4)
			.HAlign(HAlign_Right)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.Text(LOCTEXT("OkButtonLabel", "Ok"))
					.OnClicked(this, &FIKRetargetEditorController::CreateNewPose)
				]
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
					.OnClicked_Lambda( [this]()
					{
						NewPoseWindow->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
			]
		]
	];

	GEditor->EditorAddModalWindow(NewPoseWindow.ToSharedRef());
	NewPoseWindow.Reset();
}

bool FIKRetargetEditorController::CanCreatePose() const
{
	return !IsEditingPose();
}

FReply FIKRetargetEditorController::CreateNewPose() const
{
	const FName NewPoseName = FName(NewPoseEditableText.Get()->GetText().ToString());
	AssetController->CreateRetargetPose(NewPoseName, GetSourceOrTarget());
	NewPoseWindow->RequestDestroyWindow();
	RefreshPoseList();
	return FReply::Handled();
}

void FIKRetargetEditorController::HandleDuplicatePose()
{
	SetRetargeterMode(ERetargeterOutputMode::EditRetargetPose);
	
	// get a unique pose name to use as suggestion for duplicate
	const FString DuplicateSuffix = LOCTEXT("DuplicateSuffix", "_Copy").ToString();
	FString CurrentPoseName = GetCurrentPoseName().ToString();
	const FString DefaultDuplicatePoseName = CurrentPoseName.Append(*DuplicateSuffix);
	const FName UniqueNewPoseName = AssetController->MakePoseNameUnique(DefaultDuplicatePoseName, GetSourceOrTarget());
	
	SAssignNew(NewPoseWindow, SWindow)
	.Title(LOCTEXT("DuplicateRetargetPoseOptions", "Duplicate Retarget Pose"))
	.ClientSize(FVector2D(300, 80))
	.HasCloseButton(true)
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(4)
			.HAlign(HAlign_Right)
			.AutoHeight()
			[
				SAssignNew(NewPoseEditableText, SEditableTextBox)
				.MinDesiredWidth(275)
				.Text(FText::FromName(UniqueNewPoseName))
			]

			+ SVerticalBox::Slot()
			.Padding(4)
			.HAlign(HAlign_Right)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.Text(LOCTEXT("OkButtonLabel", "Ok"))
					.OnClicked(this, &FIKRetargetEditorController::CreateDuplicatePose)
				]
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4)
				.HAlign(HAlign_Right)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
					.OnClicked_Lambda( [this]()
					{
						NewPoseWindow->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
			]	
		]
	];

	GEditor->EditorAddModalWindow(NewPoseWindow.ToSharedRef());
	NewPoseWindow.Reset();
}

FReply FIKRetargetEditorController::CreateDuplicatePose() const
{
	const FName PoseToDuplicate = AssetController->GetCurrentRetargetPoseName(CurrentlyEditingSourceOrTarget);
	const FName NewPoseName = FName(NewPoseEditableText.Get()->GetText().ToString());
	AssetController->DuplicateRetargetPose(PoseToDuplicate, NewPoseName, GetSourceOrTarget());
	NewPoseWindow->RequestDestroyWindow();
	RefreshPoseList();
	return FReply::Handled();
}

void FIKRetargetEditorController::HandleDeletePose()
{
	SetRetargeterMode(ERetargeterOutputMode::EditRetargetPose);
	
	const ERetargetSourceOrTarget SourceOrTarget = GetSourceOrTarget();
	const FName CurrentPose = AssetController->GetCurrentRetargetPoseName(SourceOrTarget);
	AssetController->RemoveRetargetPose(CurrentPose, SourceOrTarget);
	RefreshPoseList();
}

bool FIKRetargetEditorController::CanDeletePose() const
{	
	// cannot delete default pose
	return AssetController->GetCurrentRetargetPoseName(GetSourceOrTarget()) != UIKRetargeter::GetDefaultPoseName();
}

void FIKRetargetEditorController::HandleResetAllBones() const
{
	const FName CurrentPose = AssetController->GetCurrentRetargetPoseName(CurrentlyEditingSourceOrTarget);
	static TArray<FName> Empty; // empty list will reset all bones
	AssetController->ResetRetargetPose(CurrentPose, Empty, GetSourceOrTarget());
}

void FIKRetargetEditorController::HandleResetSelectedBones() const
{
	const FName CurrentPose = AssetController->GetCurrentRetargetPoseName(CurrentlyEditingSourceOrTarget);
	AssetController->ResetRetargetPose(CurrentPose, GetSelectedBones(), CurrentlyEditingSourceOrTarget);
}

void FIKRetargetEditorController::HandleResetSelectedAndChildrenBones() const
{
	// get all selected bones and their children (recursive)
	const TArray<FName> BonesToReset = GetSelectedBonesAndChildren();
	
	// reset the bones in the current pose
	const FName CurrentPose = AssetController->GetCurrentRetargetPoseName(CurrentlyEditingSourceOrTarget);
	AssetController->ResetRetargetPose(CurrentPose, BonesToReset, GetSourceOrTarget());
}

void FIKRetargetEditorController::HandleAlignBones(
	const bool bIncludeChildren,
	const bool bIncludeAllBones) const
{
	if (bIncludeAllBones)
	{
		AssetController->AutoAlignAllBones(GetSourceOrTarget(), CurrentPoseAlignmentMode);
		return;
	}
	
	const TArray<FName> BonesToAlign = bIncludeChildren ? GetSelectedBonesAndChildren() : GetSelectedBones();
	AssetController->AutoAlignBones(BonesToAlign, CurrentPoseAlignmentMode, GetSourceOrTarget());
}

void FIKRetargetEditorController::HandleSnapToGround() const
{
	const TArray<FName> SelectedBones = GetSelectedBones();
	const FName FirstSelectedBone = SelectedBones.IsEmpty() ? NAME_None : SelectedBones[0];
	AssetController->SnapBoneToGround(FirstSelectedBone, GetSourceOrTarget());
}

void FIKRetargetEditorController::HandleRenamePose()
{
	SAssignNew(RenamePoseWindow, SWindow)
	.Title(LOCTEXT("RenameRetargetPoseOptions", "Rename Retarget Pose"))
	.ClientSize(FVector2D(250, 80))
	.HasCloseButton(true)
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.Padding(4)
			.AutoHeight()
			[
				SAssignNew(NewNameEditableText, SEditableTextBox)
				.Text(GetCurrentPoseName())
			]

			+ SVerticalBox::Slot()
			.Padding(4)
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("OkButtonLabel", "Ok"))
					.IsEnabled_Lambda([this]()
					{
						return !GetCurrentPoseName().EqualTo(NewNameEditableText.Get()->GetText());
					})
					.OnClicked(this, &FIKRetargetEditorController::RenamePose)
				]
				
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
					.OnClicked_Lambda( [this]()
					{
						RenamePoseWindow->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
			]	
		]
	];

	GEditor->EditorAddModalWindow(RenamePoseWindow.ToSharedRef());
	RenamePoseWindow.Reset();
}

FReply FIKRetargetEditorController::RenamePose() const
{
	const FName NewPoseName = FName(NewNameEditableText.Get()->GetText().ToString());
	RenamePoseWindow->RequestDestroyWindow();

	const FName CurrentPoseName = AssetController->GetCurrentRetargetPoseName(GetSourceOrTarget());
	AssetController->RenameRetargetPose(CurrentPoseName, NewPoseName, GetSourceOrTarget());
	RefreshPoseList();
	return FReply::Handled();
}

bool FIKRetargetEditorController::CanRenamePose() const
{
	// cannot rename default pose
	const bool bNotUsingDefaultPose = AssetController->GetCurrentRetargetPoseName(GetSourceOrTarget()) != UIKRetargeter::GetDefaultPoseName();
	return bNotUsingDefaultPose;
}

void FIKRetargetEditorController::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (TTuple<FName, TObjectPtr<UIKRetargetBoneDetails>> Pair : AllBoneDetails)
	{
		Collector.AddReferencedObject(Pair.Value);
	}

	Collector.AddReferencedObject(SourceAnimInstance);
	Collector.AddReferencedObject(TargetAnimInstance);
}

void FIKRetargetEditorController::RenderSkeleton(FPrimitiveDrawInterface* PDI, ERetargetSourceOrTarget InSourceOrTarget) const
{
	const UDebugSkelMeshComponent* MeshComponent = GetSkeletalMeshComponent(InSourceOrTarget);
	if (!MeshComponent->GetSkeletalMeshAsset())
	{
		return;
	}

	const UIKRetargeter* Asset = AssetController->GetAsset();

	// toggle skeleton visibility
	const bool bIsSource = InSourceOrTarget == ERetargetSourceOrTarget::Source;
	const bool bIsSkeletonVisible = bIsSource ? Asset->bShowSourceSkeleton : Asset->bShowTargetSkeleton;
	if (!bIsSkeletonVisible)
	{
		return;
	}
	
	const FTransform ComponentTransform = MeshComponent->GetComponentTransform();
	const FReferenceSkeleton& RefSkeleton = MeshComponent->GetReferenceSkeleton();
	const int32 NumBones = RefSkeleton.GetNum();

	// get world transforms of bones
	TArray<FBoneIndexType> RequiredBones;
	RequiredBones.AddUninitialized(NumBones);
	TArray<FTransform> WorldTransforms;
	WorldTransforms.AddUninitialized(NumBones);
	for (int32 Index=0; Index<NumBones; ++Index)
	{
		RequiredBones[Index] = static_cast<FBoneIndexType>(Index);
		WorldTransforms[Index] = MeshComponent->GetBoneTransform(Index, ComponentTransform);
	}
	
	
	const float BoneDrawSize = Asset->BoneDrawSize;
	const float MaxDrawRadius = static_cast<float>(MeshComponent->Bounds.SphereRadius * 0.01f);
	const float BoneRadius = FMath::Min(1.0f, MaxDrawRadius) * BoneDrawSize;
	const bool bIsSelectable = InSourceOrTarget == GetSourceOrTarget();
	const FLinearColor DefaultColor = bIsSelectable ? GetMutableDefault<UPersonaOptions>()->DefaultBoneColor : GetMutableDefault<UPersonaOptions>()->DisabledBoneColor;
	
	UPersonaOptions* ConfigOption = UPersonaOptions::StaticClass()->GetDefaultObject<UPersonaOptions>();
	
	FSkelDebugDrawConfig DrawConfig;
	DrawConfig.BoneDrawMode = (EBoneDrawMode::Type)ConfigOption->DefaultBoneDrawSelection;
	DrawConfig.BoneDrawSize = BoneRadius;
	DrawConfig.bAddHitProxy = bIsSelectable;
	DrawConfig.bForceDraw = false;
	DrawConfig.DefaultBoneColor = DefaultColor;
	DrawConfig.AffectedBoneColor = GetMutableDefault<UPersonaOptions>()->AffectedBoneColor;
	DrawConfig.SelectedBoneColor = GetMutableDefault<UPersonaOptions>()->SelectedBoneColor;
	DrawConfig.ParentOfSelectedBoneColor = GetMutableDefault<UPersonaOptions>()->ParentOfSelectedBoneColor;

	TArray<TRefCountPtr<HHitProxy>> HitProxies;
	TArray<int32> SelectedBones;
	
	// create hit proxies and selection set only for the currently active skeleton
	if (bIsSelectable)
	{
		HitProxies.Reserve(NumBones);
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			HitProxies.Add(new HIKRetargetEditorBoneProxy(RefSkeleton.GetBoneName(Index), Index, InSourceOrTarget));
		}

		// record selected bone indices
		for (const FName& SelectedBoneName : Selection.SelectedBoneNames[InSourceOrTarget])
		{
			int32 SelectedBoneIndex = RefSkeleton.FindBoneIndex(SelectedBoneName);
			SelectedBones.Add(SelectedBoneIndex);
		}
	}

	// generate bone colors, blue on selected chains
	TArray<FLinearColor> BoneColors;
	{
		// set default colors
		if (GetDefault<UPersonaOptions>()->bShowBoneColors)
		{
			SkeletalDebugRendering::FillWithMultiColors(BoneColors, RefSkeleton.GetNum());
		}
		else
		{
			BoneColors.Init(DefaultColor, RefSkeleton.GetNum());
		}

		auto AddBoneColorsForBonesInChain = [&BoneColors](
			const TArray<FName>& InChainsToFind,
			const TArray<FResolvedBoneChain>& InBoneChains,
			const FLinearColor& InHighlightedColor)
		{
			for (const FResolvedBoneChain& BoneChain : InBoneChains)
			{
				if (!InChainsToFind.Contains(BoneChain.ChainName))
				{
					continue;
				}
				
				for (const int32& BoneIndex : BoneChain.BoneIndices)
				{
					BoneColors[BoneIndex] = InHighlightedColor;
				}
			}
		};

		// highlight selected chains in blue
		const TArray<FName>& SelectedChainNames = GetSelectedChains();
		const FLinearColor HighlightedColor = FLinearColor::Blue;
		if (const FIKRetargetProcessor* Processor = GetRetargetProcessor())
		{
			const FRetargeterBoneChains& AllBoneChains = Processor->GetBoneChains();

			if (InSourceOrTarget == ERetargetSourceOrTarget::Source)
			{
				const TArray<FResolvedBoneChain>* BoneChains = AllBoneChains.GetResolvedBoneChains(ERetargetSourceOrTarget::Source);
				AddBoneColorsForBonesInChain(SelectedChainNames, *BoneChains, HighlightedColor);
			}
			else
			{
				const TMap<const UIKRigDefinition*, TArray<FResolvedBoneChain>>& TargetBoneChainMap = AllBoneChains.GetAllResolvedTargetBoneChains();
				for (const TPair<const UIKRigDefinition*, TArray<FResolvedBoneChain>>& TargetBoneChainPair : TargetBoneChainMap)
				{
					AddBoneColorsForBonesInChain(SelectedChainNames, TargetBoneChainPair.Value, HighlightedColor);
				}
			}
		}
	}

	// override skeleton color
	const bool bOverrideSkeletonColor = bIsSource ? Asset->bOverrideSourceSkeletonColor : Asset->bOverrideTargetSkeletonColor;
	if (bOverrideSkeletonColor)
	{
		const FLinearColor OverrideColor = bIsSource ? Asset->SourceOverideColor : Asset->TargetOverideColor;
		BoneColors.Init(OverrideColor, RefSkeleton.GetNum());
	}

	SkeletalDebugRendering::DrawBones(
		PDI,
		ComponentTransform.GetLocation(),
		RequiredBones,
		RefSkeleton,
		WorldTransforms,
		SelectedBones,
		BoneColors,
		HitProxies,
		DrawConfig
	);
}

TArray<FName> FIKRetargetEditorController::GetSelectedBonesAndChildren() const
{
	// get the reference skeleton we're operating on
	const USkeletalMesh* SkeletalMesh = GetSkeletalMesh(GetSourceOrTarget());
	if (!SkeletalMesh)
	{
		return {};
	}
	const FReferenceSkeleton RefSkeleton = SkeletalMesh->GetRefSkeleton();

	// get list of all children of selected bones
	TArray<int32> AllChildrenIndices;
	for (const FName& SelectedBone : Selection.SelectedBoneNames[CurrentlyEditingSourceOrTarget])
	{
		const int32 SelectedBoneIndex = RefSkeleton.FindBoneIndex(SelectedBone);
		AllChildrenIndices.Add(SelectedBoneIndex);
		
		for (int32 ChildIndex = 0; ChildIndex < RefSkeleton.GetNum(); ++ChildIndex)
		{
			const int32 ParentIndex = RefSkeleton.GetParentIndex(ChildIndex);
			if (ParentIndex != INDEX_NONE && AllChildrenIndices.Contains(ParentIndex))
			{
				AllChildrenIndices.Add(ChildIndex);
			}
		}
	}

	// merge total list of all selected bones and their children
	TArray<FName> BonesToReturn = Selection.SelectedBoneNames[CurrentlyEditingSourceOrTarget];
	for (const int32 ChildIndex : AllChildrenIndices)
	{
		BonesToReturn.AddUnique(RefSkeleton.GetBoneName(ChildIndex));
	}

	return BonesToReturn;
}

void FIKRetargetEditorController::FixZeroHeightRetargetRoot(ERetargetSourceOrTarget SourceOrTarget) const
{
	// is there a mesh to check?
	USkeletalMesh* SkeletalMesh = GetSkeletalMesh(SourceOrTarget);
	if (!SkeletalMesh)
	{
		return;
	}

	// have we already nagged the user about fixing this mesh?
	if (AssetController->GetAskedToFixRootHeightForMesh(SkeletalMesh))
	{
		return;
	}

	FIKRetargetPose& CurrentRetargetPose = AssetController->GetCurrentRetargetPose(SourceOrTarget);
	const FName RetargetRootBoneName = AssetController->GetPelvisBone(SourceOrTarget);
	if (RetargetRootBoneName == NAME_None)
	{
		return;
	}

	const FRetargetPoseScaleWithPivot& PoseScale = GetRetargetProcessor()->GetPoseScale(SourceOrTarget);
	FRetargetSkeleton DummySkeleton;
	DummySkeleton.Initialize(
		SkeletalMesh,
		SourceOrTarget,
		AssetController->GetAsset(),
		RetargetRootBoneName,
		PoseScale);

	const int32 RootBoneIndex = DummySkeleton.FindBoneIndexByName(RetargetRootBoneName);
	if (RootBoneIndex == INDEX_NONE)
	{
		return;
	}

	const FTransform& RootTransform = DummySkeleton.RetargetPoses.GetGlobalRetargetPose()[RootBoneIndex];
	if (RootTransform.GetLocation().Z < 1.0f)
	{
		if (PromptToFixPelvisHeight(SourceOrTarget))
		{
			// move it up based on the height of the mesh
			const double FixedHeight = FMath::Abs(SkeletalMesh->GetBounds().GetBoxExtrema(-1).Z);
			// update the current retarget pose
			CurrentRetargetPose.SetRootTranslationDelta(FVector(0.f, 0.f,FixedHeight));
		}
	}

	AssetController->SetAskedToFixRootHeightForMesh(SkeletalMesh, true);
}

bool FIKRetargetEditorController::PromptToFixPelvisHeight(ERetargetSourceOrTarget SourceOrTarget) const
{
	const FText SourceOrTargetText = SourceOrTarget == ERetargetSourceOrTarget::Source ? FText::FromString("Source") : FText::FromString("Target");

	TSharedRef<SCustomDialog> Dialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("FixRootHeightTitle", "Add Height to Pelvis Pose")))
		.Content()
		[
			SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("FixRootHeightLabel", "The {0} skeleton has a pelvis bone on the ground. Apply a vertical offset to pelvis in the current retarget pose?"), SourceOrTargetText))
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("ApplyOffset", "Apply Offset")),
			SCustomDialog::FButton(LOCTEXT("No", "No"))
	});

	if (Dialog->ShowModal() != 0)
	{
		return false; // cancel button pressed, or window closed
	}

	return true;
}

FText FIKRetargetEditorController::GetCurrentPoseName() const
{
	return FText::FromName(AssetController->GetCurrentRetargetPoseName(GetSourceOrTarget()));
}

void FIKRetargetEditorController::OnPoseSelected(TSharedPtr<FName> InPose, ESelectInfo::Type SelectInfo) const
{
	if (InPose.IsValid())
	{
		AssetController->SetCurrentRetargetPose(*InPose.Get(), GetSourceOrTarget());
	}
}

#undef LOCTEXT_NAMESPACE
