// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetFactory.h"
#include "Retargeter/IKRetargeter.h"
#include "AssetTypeCategories.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "RetargetEditor/IKRetargeterController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetFactory)

#define LOCTEXT_NAMESPACE "IKRetargeterFactory"


UIKRetargetFactory::UIKRetargetFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UIKRetargeter::StaticClass();
}

UObject* UIKRetargetFactory::FactoryCreateNew(
	UClass* Class,
	UObject* InParent,
	FName Name,
	EObjectFlags Flags,
	UObject* Context,
	FFeedbackContext* Warn)
{
	return NewObject<UIKRetargeter>(InParent, Class, Name, Flags);
}

bool UIKRetargetFactory::ShouldShowInNewMenu() const
{
	return true;
}

FText UIKRetargetFactory::GetDisplayName() const
{
	return LOCTEXT("IKRetargeter_DisplayName", "IK Retargeter");
}

uint32 UIKRetargetFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

FText UIKRetargetFactory::GetToolTip() const
{
	return LOCTEXT("IKRetargeter_Tooltip", "Defines a pair of Source/Target Retarget Rigs and the mapping between them.");
}

FString UIKRetargetFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("RTG_NewRetargeter"));
}

#undef LOCTEXT_NAMESPACE

