// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetHierarchyTabSummoner.h"

#include "RetargetEditor/SIKRetargetHierarchy.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RigEditor/IKRigEditorStyle.h"

#include "IDocumentation.h"

#define LOCTEXT_NAMESPACE "IKRetargetHierarchyTabSummoner"

const FName FIKRetargetHierarchyTabSummoner::TabID(TEXT("IKRetargetHierarchy"));

FIKRetargetHierarchyTabSummoner::FIKRetargetHierarchyTabSummoner(const TSharedRef<FIKRetargetEditor>& InEditor)
	: FWorkflowTabFactory(TabID, InEditor)
	, RetargetEditor(InEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRetargetHierarchy_TabLabel", "Hierarchy");
	TabIcon = FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(), "IKRig.Hierarchy");

	ViewMenuDescription = LOCTEXT("IKRetargetHierarchy_ViewMenu_Desc", "Hierarchy");
	ViewMenuTooltip = LOCTEXT("IKRetargetHierarchy_ViewMenu_ToolTip", "Show the Retarget Hierarchy Tab");
}

TSharedPtr<SToolTip> FIKRetargetHierarchyTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRetargetHierarchyTooltip", "The IK Retargeting Hierarchy tab lets you view the skeleton hierarchy."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRigSkeleton_Window"));
}

TSharedRef<SWidget> FIKRetargetHierarchyTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SIKRetargetHierarchy, RetargetEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
