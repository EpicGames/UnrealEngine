// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetOpDetails.h"

#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "Widgets/Input/SComboBox.h"
#include "RetargetEditor/IKRetargeterController.h"
#include "RetargetEditor/SIKRetargetChainMapList.h"
#include "Retargeter/IKRetargeter.h"
#include "Retargeter/RetargetOps/AlignPoleVectorOp.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"
#include "Retargeter/RetargetOps/IKChainsOp.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"
#include "Retargeter/RetargetOps/SpeedPlantingOp.h"
#include "RigEditor/IKRigStructViewer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargetOpDetails)

#define LOCTEXT_NAMESPACE "IKRetargetOpDetails"

bool FIKRetargetOpBaseSettingsCustomization::LoadAndValidateStructToCustomize(
	const TSharedRef<IPropertyHandle>& InStructPropertyHandle,
	const IPropertyTypeCustomizationUtils& InStructCustomizationUtils)
{
	SelectedObjects = InStructCustomizationUtils.GetPropertyUtilities().Get()->GetSelectedObjects();
	if (!ensure(!SelectedObjects.IsEmpty()))
	{
		return false;
	}

	void* StructMemory = nullptr;
	FPropertyAccess::Result Result = InStructPropertyHandle->GetValueData(StructMemory);
	if (!ensure(Result == FPropertyAccess::Success && StructMemory))
	{
		return false;
	}

	FIKRetargetOpSettingsBase* SettingsBeingCustomized = static_cast<FIKRetargetOpSettingsBase*>(StructMemory);
	OpName = SettingsBeingCustomized->OwningOpName;
	if (OpName == NAME_None)
	{
		return false;
	}
	
	StructViewer = Cast<UIKRigStructViewer>(SelectedObjects[0].Get());
	if (StructViewer == nullptr)
	{
		return false;
	}
	RetargetAsset = CastChecked<UIKRetargeter>(StructViewer->GetStructOwner());
	AssetController = UIKRetargeterController::GetController(RetargetAsset);

	return true;
}

void FIKRetargetOpBaseSettingsCustomization::AddPropertyToGroup(
	const TSharedRef<IPropertyHandle>& InPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder)
{
	// get category from metadata
	FString CategoryName = InPropertyHandle->GetMetaData(TEXT("Category"));
	if (CategoryName.IsEmpty())
	{
		// if no category specified, add directly to builder
		ChildBuilder.AddProperty(InPropertyHandle);
		return;
	}

	// get the group
	FName CategoryFName(*CategoryName);
	if (IDetailGroup* Group = ChildBuilder.GetGroup(CategoryFName))
	{
		Group->AddPropertyRow(InPropertyHandle);
		return;
	}

	// didn't already exist, so add one
	FText CategoryText = FText::FromString(CategoryName);
	IDetailGroup& Group = ChildBuilder.AddGroup(CategoryFName, CategoryText);
	// add the property to the new group
	Group.AddPropertyRow(InPropertyHandle);
}

void FIKRetargetOpBaseSettingsCustomization::AddChildPropertiesToCategoryGroups(
	const TSharedRef<IPropertyHandle>& InParentPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder)
{
	uint32 NumChildren;
	InParentPropertyHandle->GetNumChildren(NumChildren);

	// map of category names to property groups
	TMap<FName, IDetailGroup*> CategoryGroups;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = InParentPropertyHandle->GetChildHandle(ChildIndex);
		if (!ChildHandle.IsValid())
		{
			continue;
		}
		
		FProperty* ChildProperty = ChildHandle->GetProperty();
		if (!ChildProperty)
		{
			continue;
		}

		// get category from metadata
		FString CategoryName = ChildProperty->GetMetaData(TEXT("Category"));
		if (CategoryName.IsEmpty())
		{
			// if no category specified, add directly to builder
			ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
			continue;
		}

		// convert to FName for map key
		FName CategoryFName(*CategoryName);

		// Get or create the group for this category
		IDetailGroup* Group;
		if (IDetailGroup** ExistingGroup = CategoryGroups.Find(CategoryFName))
		{
			Group = *ExistingGroup;
		}
		else
		{
			// create new group
			FText CategoryText = FText::FromString(CategoryName);
			Group = &ChildBuilder.AddGroup(CategoryFName, CategoryText);
			CategoryGroups.Add(CategoryFName, Group);
		}

		// sdd the property to its category group
		Group->AddPropertyRow(ChildHandle.ToSharedRef());
	}
}

void FIKRetargetOpBaseSettingsCustomization::AddChildPropertiesInCategory(
	const TSharedRef<IPropertyHandle>& InParentPropertyHandle,
	IDetailChildrenBuilder& InChildBuilder,
	const FName& InCategoryName,
	const TArray<FName>& InPropertiesToIgnore)
{
	uint32 NumChildren;
	InParentPropertyHandle->GetNumChildren(NumChildren);

	IDetailGroup* CategoryGroup = nullptr;
	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = InParentPropertyHandle->GetChildHandle(ChildIndex);
		if (!ChildHandle.IsValid())
		{
			continue;
		}

		FProperty* ChildProperty = ChildHandle->GetProperty();
		if (!ChildProperty)
		{
			continue;
		}

		if (InPropertiesToIgnore.Contains(ChildProperty->GetName()))
		{
			continue; // filtered out by name
		}
		
		FString CategoryName = ChildProperty->GetMetaData(TEXT("Category"));
		if (CategoryName.IsEmpty() || InCategoryName != CategoryName)
		{
			continue;  // not in the category we're looking for
		}

		// create group if not created yet
		if (!CategoryGroup)
		{
			static bool bStartExpanded = true;
			CategoryGroup = &InChildBuilder.AddGroup(InCategoryName, FText::FromName(InCategoryName), bStartExpanded);
		}

		// Add the property to the category group
		IDetailPropertyRow& NewRow = CategoryGroup->AddPropertyRow(ChildHandle.ToSharedRef());
		NewRow.ShouldAutoExpand(true);
	}

	if (CategoryGroup)
	{
		CategoryGroup->ToggleExpansion(true);
	}
}

void FIKRetargetOpBaseSettingsCustomization::AddBaseOpProperties(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder)
{
	UScriptStruct* OpBaseStructType = FIKRetargetOpSettingsBase::StaticStruct();

	for (TFieldIterator<FProperty> It(OpBaseStructType); It; ++It)
	{
		FProperty* BaseProp = *It;

		// the enabled property is displayed as a checkbox on the op itself
		if (BaseProp->GetFName() == GET_MEMBER_NAME_CHECKED(FIKRetargetOpSettingsBase, bEnabled))
		{
			continue;
		}
		
		TSharedPtr<IPropertyHandle> BasePropHandle = StructPropertyHandle->GetChildHandle(BaseProp->GetFName());
		if (BasePropHandle.IsValid())
		{
			AddPropertyToGroup(BasePropHandle.ToSharedRef(), ChildBuilder);
		}
	}
}

void FChainsFKOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}

	// add the IK Rig asset input field
	TSharedPtr<IPropertyHandle> IKRigHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRetargetFKChainsOpSettings, IKRigAsset));
	ChildBuilder.AddProperty(IKRigHandle.ToSharedRef());

	// configure the chain map list view for FK chains
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = OpName;
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = false;
	ChainListConfig.bEnableChainMapping = true;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(URetargetFKChainSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FRetargetFKChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FRetargetFKChainSettings,URetargetFKChainSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);

	// add the chain mapping list
	const FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	// add the debug properties
	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Debug"));

	// add the properties from the base op
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FRunIKRigOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}

	// add the IK Rig asset input field
	TSharedPtr<IPropertyHandle> IKRigHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRetargetFKChainsOpSettings, IKRigAsset));
	ChildBuilder.AddProperty(IKRigHandle.ToSharedRef());

	// show only IK chains by default
	FChainMapFilterOptions ShowOnlyIKFilter;
	ShowOnlyIKFilter.bNeverShowChainsWithoutIK = true;

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = OpName;
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = true;
	ChainListConfig.Filter = ShowOnlyIKFilter;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(URunIKRigSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FIKRetargetRunIKRigChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FIKRetargetRunIKRigChainSettings,URunIKRigSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);

	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	// add the extra properties
	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Debug"));

	// add the properties from the base op
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FIKChainOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}
	
	// show only IK chains
	FChainMapFilterOptions ShowOnlyIKFilter;
	ShowOnlyIKFilter.bNeverShowChainsWithoutIK = true;

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = AssetController->GetParentOpByName(OpName); // use chain mapping from parent
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = false;
	ChainListConfig.Filter = ShowOnlyIKFilter;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(URetargetIKChainSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FRetargetIKChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FRetargetIKChainSettings,URetargetIKChainSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);

	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	// add the debug properties
	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Debug"));

	// add the properties from the base op
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FStrideWarpOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}
	
	// show only IK chains
	FChainMapFilterOptions ShowOnlyIKFilter;
	ShowOnlyIKFilter.bNeverShowChainsWithoutIK = true;

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = AssetController->GetParentOpByName(OpName); // use chain mapping from parent
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = false;
	ChainListConfig.Filter = ShowOnlyIKFilter;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(URetargetStrideWarpSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FRetargetStrideWarpChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FRetargetStrideWarpChainSettings,URetargetStrideWarpSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);

	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Op Settings"));
	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Debug"));
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FSpeedPlantOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}
	
	// show only IK chains
	FChainMapFilterOptions ShowOnlyIKFilter;
	ShowOnlyIKFilter.bNeverShowChainsWithoutIK = true;

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = AssetController->GetParentOpByName(OpName); // use chain mapping from parent
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = false;
	ChainListConfig.Filter = ShowOnlyIKFilter;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(URetargetSpeedPlantSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FRetargetSpeedPlantingSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FRetargetSpeedPlantingSettings,URetargetSpeedPlantSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);

	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];
	
	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Op Settings"));
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FPoleVectorOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}

	// add the IK Rig asset input field
	TSharedPtr<IPropertyHandle> IKRigHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRetargetAlignPoleVectorOpSettings, IKRigAsset));
	ChildBuilder.AddProperty(IKRigHandle.ToSharedRef());

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = OpName;
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = false;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(UPoleVectorSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FRetargetPoleVectorSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FRetargetPoleVectorSettings,UPoleVectorSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);

	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Op Settings"));
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FAdditivePoseOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}

	// store property handles for callbacks
	PoseToApplyProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRetargetAdditivePoseOpSettings, PoseToApply));

	UpdatePoseNameOptions();

	// add dropdown menu to select retarget pose
	ChildBuilder.AddCustomRow(LOCTEXT("CurrentPoseLabel", "Pose To Apply"))
	.NameContent()
	[
		PoseToApplyProperty->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200)
	[
		SNew(SComboBox<TSharedPtr<FName>>)
		.OptionsSource(&PoseNameOptions)
		.InitiallySelectedItem(CurrentPoseOption)
		.OnGenerateWidget_Lambda([](TSharedPtr<FName> InItem)
		{
			return SNew(STextBlock).Text(FText::FromName(*InItem));
		})
		.OnSelectionChanged_Lambda([this](TSharedPtr<FName> NewSelection, ESelectInfo::Type)
		{
			if (NewSelection.IsValid())
			{
				PoseToApplyProperty->SetValue(*NewSelection);
			}
		})
		[
			SNew(STextBlock)
			.Text_Lambda([this]() {
				FName Value;
				PoseToApplyProperty->GetValue(Value);
				return FText::FromName(Value);
			})
		]
	];

	// add the alpha property
	TSharedPtr<IPropertyHandle> AlphaProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRetargetAdditivePoseOpSettings, Alpha));
	ChildBuilder.AddProperty(AlphaProperty.ToSharedRef());

	// add the properties from the base op
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FAdditivePoseOpCustomization::UpdatePoseNameOptions()
{
	// determine currently selected pose
	FName CurrentPoseName;
	PoseToApplyProperty->GetValue(CurrentPoseName);
	
	// get all the retarget poses
	const TMap<FName, FIKRetargetPose>& RetargetPoses = AssetController->GetRetargetPoses(ERetargetSourceOrTarget::Target);
	TArray<FName> PoseNames;
	RetargetPoses.GetKeys(PoseNames);

	// reset list of names
	PoseNameOptions.Reset(PoseNames.Num());

	// add all the other poses
	for (const FName& PoseName : PoseNames)
	{
		TSharedPtr<FName> PostNameOption = MakeShared<FName>(PoseName);
		if (PoseName == CurrentPoseName)
		{
			CurrentPoseOption = PostNameOption;
		}
		PoseNameOptions.Emplace(PostNameOption);
	}

	// default to first pose if stored pose no longer available
	if (!CurrentPoseOption.IsValid())
	{
		CurrentPoseOption = PoseNameOptions[0];
	}
}

void FStretchChainOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}

	// add the IK Rig asset input field
	TSharedPtr<IPropertyHandle> IKRigHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FIKRetargetStretchChainOpSettings, IKRigAsset));
	ChildBuilder.AddProperty(IKRigHandle.ToSharedRef());

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = OpName;
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = true;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(UStretchChainSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FRetargetStretchChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FRetargetStretchChainSettings,UStretchChainSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);

	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Op Settings"));
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FFloorConstraintOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}
	
	// show only IK chains
	FChainMapFilterOptions ShowOnlyIKFilter;
	ShowOnlyIKFilter.bNeverShowChainsWithoutIK = true;

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = AssetController->GetParentOpByName(OpName); // use chain mapping from parent
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = false;
	ChainListConfig.Filter = ShowOnlyIKFilter;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(UFloorConstraintSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FFloorConstraintChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FFloorConstraintChainSettings,UFloorConstraintSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);
	
	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Op Settings"));
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FBlendToSourceOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}
	
	// show only IK chains
	FChainMapFilterOptions ShowOnlyIKFilter;
	ShowOnlyIKFilter.bNeverShowChainsWithoutIK = true;

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = AssetController->GetParentOpByName(OpName); // use chain mapping from parent
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = false;
	ChainListConfig.Filter = ShowOnlyIKFilter;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(UBlendToSourceSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FIKRetargetBlendToSourceChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FIKRetargetBlendToSourceChainSettings,UBlendToSourceSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);
	
	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	// add op properties
	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Op Settings"));
	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Debug"));
	
	// add the properties from the base op
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FOffsetGoalsOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}
	
	// show only IK chains
	FChainMapFilterOptions ShowOnlyIKFilter;
	ShowOnlyIKFilter.bNeverShowChainsWithoutIK = true;

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = AssetController->GetParentOpByName(OpName); // use chain mapping from parent
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = false;
	ChainListConfig.Filter = ShowOnlyIKFilter;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(UOffsetGoalsSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FIKRetargetOffsetGoalsChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FIKRetargetOffsetGoalsChainSettings,UOffsetGoalsSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);
	
	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Op Settings"));
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}

void FScaleGoalsOpCustomization::CustomizeChildren(
	TSharedRef<IPropertyHandle> StructPropertyHandle,
	IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!LoadAndValidateStructToCustomize(StructPropertyHandle, StructCustomizationUtils))
	{
		return;
	}
	
	// show only IK chains
	FChainMapFilterOptions ShowOnlyIKFilter;
	ShowOnlyIKFilter.bNeverShowChainsWithoutIK = true;

	// configure chain map list for the IK op
	FChainMapListConfig ChainListConfig;
	ChainListConfig.OpWithChainSettings = OpName;
	ChainListConfig.OpWithChainMapping = AssetController->GetParentOpByName(OpName); // use chain mapping from parent
	ChainListConfig.Controller = AssetController;
	ChainListConfig.bEnableGoalColumn = true;
	ChainListConfig.bEnableChainMapping = false;
	ChainListConfig.Filter = ShowOnlyIKFilter;
	const FName SettingsPropertyName = GET_MEMBER_NAME_CHECKED(UScaleGoalsSettingsWrapper, Settings);
	const TArray<FName> HiddenSettings = {GET_MEMBER_NAME_CHECKED(FIKRetargetScaleGoalsChainSettings, TargetChainName)};
	ChainListConfig.ChainSettingsGetterFunc =
		TMakeChainSettingsGetter<FIKRetargetScaleGoalsChainSettings,UScaleGoalsSettingsWrapper>(OpName, SettingsPropertyName, HiddenSettings);
	
	// add the chain mapping list
	FText ChainMapRowName = FText::FromString("Chain Map");
	ChildBuilder.AddCustomRow(ChainMapRowName)
	.WholeRowContent()
	[
		SNew(SIKRetargetChainMapList).InChainMapListConfig(ChainListConfig)
	];

	AddChildPropertiesInCategory(StructPropertyHandle, ChildBuilder, FName("Op Settings"));
	AddBaseOpProperties(StructPropertyHandle, ChildBuilder);
}


#undef LOCTEXT_NAMESPACE
