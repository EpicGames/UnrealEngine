// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetOpStackTabSummoner.h"

#include "IDocumentation.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/SRetargetOpStack.h"

#define LOCTEXT_NAMESPACE "IKRetargetOpStackTabSummoner"

const FName FIKRetargetOpStackTabSummoner::TabID(TEXT("IKRetargeterOpStack"));

FIKRetargetOpStackTabSummoner::FIKRetargetOpStackTabSummoner(const TSharedRef<FIKRetargetEditor>& InIKRetargetEditor)
	: FWorkflowTabFactory(TabID, InIKRetargetEditor)
	, RetargetEditor(InIKRetargetEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRetargetOpStackTabLabel", "Op Stack");
	TabIcon = FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(), "IKRig.SolverStack"); // TODO get new icon for this tab

	ViewMenuDescription = LOCTEXT("IKRetargetOpStack_ViewMenu_Desc", "Retargeting Op Stack");
	ViewMenuTooltip = LOCTEXT("IKRetargetOpStack_ViewMenu_ToolTip", "Show the Retargeting Op Stack Tab");
}

TSharedPtr<SToolTip> FIKRetargetOpStackTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("RetargetOpStackTooltip", "A stack of operations executed from top to bottom to transfer animation from the source to the target."), NULL, TEXT("Shared/Editors/Persona"), TEXT("RetargetOpStack_Window"));
}

TSharedRef<SWidget> FIKRetargetOpStackTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SRetargetOpStack, RetargetEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
