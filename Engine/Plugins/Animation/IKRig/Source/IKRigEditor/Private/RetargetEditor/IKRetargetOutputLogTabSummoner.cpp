// Copyright Epic Games, Inc. All Rights Reserved.
#include "RetargetEditor/IKRetargetOutputLogTabSummoner.h"

#include "IDocumentation.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RigEditor/SIKRigOutputLog.h"
#include "Retargeter/IKRetargetProcessor.h"

#define LOCTEXT_NAMESPACE "IKRetargetOutputLogTabSummoner"

const FName FIKRetargetOutputLogTabSummoner::TabID(TEXT("RetargetOutputLog"));

FIKRetargetOutputLogTabSummoner::FIKRetargetOutputLogTabSummoner(
	const TSharedRef<FIKRetargetEditor>& InRetargetEditor)
	: FWorkflowTabFactory(TabID, InRetargetEditor),
	IKRetargetEditor(InRetargetEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRetargetOutputLogTabLabel", "Retarget Output Log");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.CompilerResults");

	ViewMenuDescription = LOCTEXT("IKRetargetOutputLog_ViewMenu_Desc", "Retarget Output Log");
	ViewMenuTooltip = LOCTEXT("IKRetargetOutputLog_ViewMenu_ToolTip", "Show the Retargeting Output Log Tab");
}

TSharedPtr<SToolTip> FIKRetargetOutputLogTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRetargetOutputLogTooltip", "View warnings and errors while retargeting."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRetargetOutputLog_Window"));
}

TSharedRef<SWidget> FIKRetargetOutputLogTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	// at this point the processor's log may not be bound to the asset, so forcibly do it now
	const TSharedRef<FIKRetargetEditorController>& Controller = IKRetargetEditor.Pin()->GetController();
	FIKRetargetProcessor* Processor = Controller->GetRetargetProcessor();
	Processor->Log.SetLogTarget(Controller->AssetController->GetAsset());
	// now bind the log UI to the processor's log target
	const FName LogName = Processor->Log.GetLogTarget();
	TSharedRef<SIKRigOutputLog> LogView = SNew(SIKRigOutputLog, LogName);
	Controller->SetOutputLogView(LogView);
	return LogView;
}

#undef LOCTEXT_NAMESPACE 
