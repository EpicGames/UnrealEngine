// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetOverrideCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "RetargetEditor/IKRetargeterController.h"
#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetOverrides.h"
#include "RigEditor/IKRigStructViewer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RetargetOverrideCustomization"

void FRetargetOverrideCustomization::CustomizeHeader(
    TSharedRef<IPropertyHandle> PropertyHandle,
    FDetailWidgetRow& HeaderRow,
    IPropertyTypeCustomizationUtils& CustomizationUtils)
{
}

void FRetargetOverrideCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> RetargetOverrideSetHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	// NOTE:
	// The top object being customized here is an FRetargetOverrideSet.
	// FRetargetOverrideSet contains a TArray<FRetargetOpOverrides> with an entry for each op that has property overrides in this set.
	// The FRetargetOpOverrides has a TArray<FRetargetOpPropertyOverride> which contains the override value for a single property in the op settings
	// This customization lists all the property overrides with each op in its own category

	// get the controller for the set being edited
	AssetController = GetAssetController(CustomizationUtils);
	if (!AssetController.IsValid())
	{
		// the struct is being viewed outside the editor, so display the default view
		uint32 NumChildren;
		RetargetOverrideSetHandle->GetNumChildren(NumChildren);
		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = RetargetOverrideSetHandle->GetChildHandle(ChildIndex);
			if (!ChildHandle.IsValid())
			{
				continue;
			}

			const FProperty* Property = ChildHandle->GetProperty();
			if (!Property)
			{
				continue;
			}

			FString Category = Property->GetMetaData(TEXT("Category"));
			if (Category.Equals(TEXT("Deprecated")))
			{
				continue; // skip if it belongs to the "Deprecated" category
			}

			ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}

		// skip editor-side customization
		return;
	}

	// get the name of this set
	FRetargetOverrideSet* SetPtr = nullptr;
	void* RawData = nullptr;
	if (RetargetOverrideSetHandle->GetValueData(RawData) == FPropertyAccess::Success)
	{
		SetPtr = static_cast<FRetargetOverrideSet*>(RawData);
	}
	const FName SetName = AssetController->GetOverrideSetName(SetPtr);
	if (!ensure(SetName != NAME_None))
	{
		return;
	}

	// get handle to the RetargetOpOverrides array property
	TSharedPtr<IPropertyHandle> OpOverridesArrayHandle = RetargetOverrideSetHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRetargetOverrideSet, OpOverrides));
	if (!OpOverridesArrayHandle.IsValid())
	{
		return;
	}

	// how many ops are stored in this override set?
	uint32 NumOpOverrides = 0;
	OpOverridesArrayHandle->GetNumChildren(NumOpOverrides);

	// make copy of the weak ptr for lambda capture
	TWeakObjectPtr<UIKRetargeterController> LocalController = AssetController;

	// for each op that has overrides in this set...
	for (uint32 OpIdx = 0; OpIdx < NumOpOverrides; ++OpIdx)
	{
		TSharedPtr<IPropertyHandle> OpOverridesHandle = OpOverridesArrayHandle->GetChildHandle(OpIdx);

		// get the reflection data for the settings struct associated with this op
		const UScriptStruct* OpSettingsType = GetSettingsStruct(OpOverridesHandle);
		if (!ensure(OpSettingsType))
		{
			continue;
		}

		// get the op name
		TSharedPtr<IPropertyHandle> OpNameHandle = OpOverridesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRetargetOpOverrides, OpName));
		if (!ensure(OpNameHandle.IsValid()))
		{
			continue;
		}
		FName OpName;
		OpNameHandle->GetValue(OpName);

		// generate a scoped struct to display
		TSharedPtr<FStructOnScope> OpSettingsScopedStruct = GetCachedScopedStruct(OpName, OpSettingsType);
		if (!ensure(OpSettingsScopedStruct.IsValid()))
		{
			continue;
		}

		// create a group for this specific op
		IDetailGroup& OpGroup = ChildBuilder.AddGroup(OpName, FText::FromName(OpName));

		// get a handle to the array of property overrides
		TSharedPtr<IPropertyHandle> OverridesArrayHandle = OpOverridesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRetargetOpOverrides, PropertyOverrides));
		uint32 NumOverrides = 0;
		OverridesArrayHandle->GetNumChildren(NumOverrides);

		// update temp struct with all property override values
		// NOTE: I had to do this BEFORE AddExternalStructure because otherwise the EditConditions evaluated with the wrong values
		for (uint32 OverrideIndex = 0; OverrideIndex < NumOverrides; ++OverrideIndex)
		{
			FString PropertyPath;
			GetOverridePropertyPath(OpOverridesHandle, OverrideIndex, PropertyPath);
			TSharedPtr<IPropertyHandle> ValueHandle = GetOverrideValueHandle(OpOverridesHandle, OverrideIndex);
			CopyStoredValueToScopedStruct(OpSettingsScopedStruct, OpSettingsType, PropertyPath, ValueHandle);
		}

		// add the struct as a child (this registers the memory with the details system)
		IDetailPropertyRow* ExternalRoot = ChildBuilder.AddExternalStructure(OpSettingsScopedStruct.ToSharedRef());
		if (!ensure(ExternalRoot))
		{
			continue;
		}

		// hide the whole struct so we can manually add only the overridden properties below
		ExternalRoot->Visibility(EVisibility::Hidden);

		// split properties into groups so that we can distinguish between different elements of the same array
		TMap<FString, TArray<int32>> PropertyGroups = MapOverridesToGroups(OpOverridesHandle);

		for (TTuple<FString, TArray<int>>& Group : PropertyGroups)
		{
			const FString& GroupName = Group.Key;
			const TArray<int32>& OverrideIndices = Group.Value;

			// resolve the group display label: if the group is an array element like
			// "ChainsToAlign[3]", try to substitute "[3]" with the element's TitleProperty
			// value (e.g. the TargetChainName) read from live op settings. Falls back to
			// the raw GroupName on any parse/lookup failure.
			FText GroupDisplayLabel = FText::FromString(GroupName);
			{
				int32 BracketOpen = INDEX_NONE;
				int32 BracketClose = INDEX_NONE;
				if (GroupName.FindLastChar(TEXT('['), BracketOpen)
					&& GroupName.FindLastChar(TEXT(']'), BracketClose)
					&& BracketClose > BracketOpen + 1)
				{
					const FString ArrayName = GroupName.Left(BracketOpen);
					const FString IndexStr = GroupName.Mid(BracketOpen + 1, BracketClose - BracketOpen - 1);
					int32 ElementIndex = INDEX_NONE;
					if (LexTryParseString(ElementIndex, *IndexStr) && ElementIndex >= 0)
					{
						if (FArrayProperty* ArrayProp = FindFProperty<FArrayProperty>(OpSettingsType, FName(*ArrayName)))
						{
							const FString TitleMeta = ArrayProp->GetMetaData(TEXT("TitleProperty"));
							FStructProperty* InnerStructProp = !TitleMeta.IsEmpty() ? CastField<FStructProperty>(ArrayProp->Inner) : nullptr;
							if (InnerStructProp && LocalController.IsValid())
							{
								if (const FIKRetargetOpBase* Op = LocalController->GetRetargetOpByName(OpName))
								{
									if (const uint8* OpSettingsData = reinterpret_cast<const uint8*>(Op->GetSettingsConst()))
									{
										FScriptArrayHelper ArrayHelper(ArrayProp, ArrayProp->ContainerPtrToValuePtr<void>(OpSettingsData));
										if (ElementIndex < ArrayHelper.Num())
										{
											if (FProperty* TitleProp = FindFProperty<FProperty>(InnerStructProp->Struct, FName(*TitleMeta)))
											{
												FString Exported;
												TitleProp->ExportTextItem_Direct(
													Exported,
													TitleProp->ContainerPtrToValuePtr<void>(ArrayHelper.GetRawPtr(ElementIndex)),
													nullptr, nullptr, PPF_None);
												if (!Exported.IsEmpty() && Exported != TEXT("None"))
												{
													GroupDisplayLabel = FText::FromString(Exported);
												}
											}
										}
									}
								}
							}
						}
					}
				}
			}

			// add the group
			IDetailGroup& PropertyGroup = OpGroup.AddGroup(FName(GroupName), GroupDisplayLabel);

			// add all the property overrides in this group
			for (int32 OverrideIndex : OverrideIndices)
			{
				// get the path to this property
				FString PropertyPath;
				GetOverridePropertyPath(OpOverridesHandle, OverrideIndex, PropertyPath);

				// resolve the final leaf property so we can use its widget generation
				TSharedPtr<IPropertyHandle> PropertyHandle = ResolveHandleFromPath(ExternalRoot->GetPropertyHandle(), PropertyPath);
				if (!ensure(PropertyHandle.IsValid()))
				{
					continue;
				}

				// inject the asset into the active transaction
				auto PreChangeAction = [LocalController]()
				{
					if (LocalController.IsValid())
					{
						if (UIKRetargeter* Asset = LocalController->GetAsset())
						{
							Asset->Modify();
						}
					}
				};

				// marshal the data back into the asset
				auto PostChangeAction = [LocalController, SetName, OpName, PropertyPath, OpSettingsScopedStruct]()
				{
					if (LocalController.IsValid())
					{
						constexpr bool bShouldTransact = false;
						LocalController->UpdateOverrideValue(
							SetName,
							OpName,
							PropertyPath,
							*OpSettingsScopedStruct.Get(),
							bShouldTransact);
					}
				};

				// bind the PRE-change delegates
				PropertyHandle->SetOnPropertyValuePreChange(FSimpleDelegate::CreateLambda(PreChangeAction));
				PropertyHandle->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateLambda(PreChangeAction));

				// bind the POST-change delegates
				PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda(PostChangeAction));
				PropertyHandle->SetOnChildPropertyValueChanged(FSimpleDelegate::CreateLambda(PostChangeAction));

				// add the widget for the property to the group
				AddRowForPropertyOverride(PropertyHandle, PropertyGroup, SetName, OpName, PropertyPath);
			}
		}
	}
}

TObjectPtr<UIKRetargeterController> FRetargetOverrideCustomization::GetAssetController(
    const IPropertyTypeCustomizationUtils& CustomizationUtils)
{
    TArray<TWeakObjectPtr<UObject>> SelectedObjects = CustomizationUtils.GetPropertyUtilities()->GetSelectedObjects();
    if (!ensure(!SelectedObjects.IsEmpty()))
    {
       return nullptr;
    }

    UIKRigStructViewer* StructViewer = Cast<UIKRigStructViewer>(SelectedObjects[0].Get());
    if (!StructViewer)
    {
       return nullptr; // this can happen if an FRetargetOverrideSet is viewed in a blueprint details panel
    }
    
    UObject* StructOwner = StructViewer->GetStructOwner();
    if (!ensure(StructOwner))
    {
       return nullptr;
    }
    
    UIKRetargeter* Retargeter = Cast<UIKRetargeter>(StructOwner);
    if (!ensure(Retargeter))
    {
       return nullptr;
    }
    
    return UIKRetargeterController::GetController(Retargeter);
}

void FRetargetOverrideCustomization::AddRowForPropertyOverride(
	TSharedPtr<IPropertyHandle> InPropertyHandle,
	IDetailGroup& InPropertyGroup,
	const FName& InOverrideSetName,
	const FName& InOpName,
	const FString& InPropertyPath)
{
	// add the property row
	// NOTE: cannot just use AddWidgetRow() here because it won't use struct customizations
	IDetailPropertyRow& PropertyRow = InPropertyGroup.AddPropertyRow(InPropertyHandle.ToSharedRef());
	TSharedPtr<SWidget> DefaultNameWidget;
	TSharedPtr<SWidget> DefaultValueWidget;
	PropertyRow.GetDefaultWidgets(DefaultNameWidget, DefaultValueWidget);

	// copy of controller ptr for lambda
	TWeakObjectPtr<UIKRetargeterController> LocalController = AssetController;
	
	PropertyRow.CustomWidget()
	.NameContent()
	[
		DefaultNameWidget.ToSharedRef()
	]
	.ValueContent()
	[
		SNew(SBox) // wraps the default value widget so we can dynamically control its enabled state
		.IsEnabled_Lambda([LocalController, InOverrideSetName, InOpName, InPropertyPath]()
		{
			if (LocalController.IsValid())
			{
				// enable the widget only if it does NOT have a binding
				return !LocalController->GetPropertyOverrideHasBinding(InOverrideSetName, InOpName, InPropertyPath);
			}
			return false;
		})
		[
			DefaultValueWidget.ToSharedRef()
		]
	]
	.ExtensionContent()
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.Padding(0.0f, 2.0f)
		[
			SNew(SComboButton)
			.ComboButtonStyle(FAppStyle::Get(), "ComboButton") 
			.OnGetMenuContent(this, &FRetargetOverrideCustomization::GetBindingMenuContent, InPropertyHandle, InOverrideSetName, InOpName, InPropertyPath)
			.HasDownArrow(true)
			.ContentPadding(FMargin(6.0f, 2.0f))
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::ClipToBounds)
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text_Lambda([LocalController, InOverrideSetName, InOpName, InPropertyPath]()
					{
						if (!LocalController.IsValid())
						{
							return FText::GetEmpty();
						}
							
						const FName BoundCurve = LocalController->GetCurveBoundToPropertyOverride(InOverrideSetName, InOpName, InPropertyPath);
						if (!BoundCurve.IsNone())
						{
							return FText::FromName(BoundCurve);
						}
							
						const FName BoundVariable = LocalController->GetVariableBoundToPropertyOverride(InOverrideSetName, InOpName, InPropertyPath);
						if (!BoundVariable.IsNone())
						{
							return FText::FromName(BoundVariable);
						}
							
						return LOCTEXT("NotBoundLabel", "Not Bound");
					})
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
		
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked_Lambda([LocalController, InOverrideSetName, InOpName, InPropertyPath]()
			{
				if (LocalController.IsValid())
				{
					LocalController->RemovePropertyOverrideFromOp(InOverrideSetName, InOpName, InPropertyPath);
				}
				return FReply::Handled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Delete"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	];
}

TSharedRef<SWidget> FRetargetOverrideCustomization::GetBindingMenuContent(
	TSharedPtr<IPropertyHandle> InPropertyHandle, 
	FName InOverrideSetName, 
	FName InOpName, 
	FString InPropertyPath)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TWeakObjectPtr<UIKRetargeterController> LocalController = AssetController;
	if (!LocalController.IsValid())
	{
		return MenuBuilder.MakeWidget();
	}
	
	// SECTION 1: UNBIND
	MenuBuilder.BeginSection("UnbindSection", FText::FromString("Unbind"));
	{
		MenuBuilder.AddMenuEntry(
			FText::FromString("None"),
			FText::FromString("Clear the current variable binding"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Cross"),
			FUIAction(FExecuteAction::CreateLambda([LocalController, InOverrideSetName, InOpName, InPropertyPath]()
			{
				if (LocalController.IsValid())
				{
					LocalController->ClearPropertyOverrideBinding(InOverrideSetName, InOpName, InPropertyPath);
				}
			}))
		);
	}
	MenuBuilder.EndSection();
	
	// SECTION 2: BIND TO CURVE
	MenuBuilder.BeginSection("CurveSection", LOCTEXT("CurveBindHeaderLabel", "Bind to Curve"));
	{
		TSharedRef<SWidget> CurveEntryWidget = SNew(SBox)
		.Padding(FMargin(16.0f, 4.0f, 16.0f, 4.0f))
		.MinDesiredWidth(150.0f)
		[
			SNew(SEditableTextBox)
			.HintText(LOCTEXT("EnterCurveNameLabel", "Enter Curve Name..."))
			.IsEnabled_Lambda([LocalController, InOpName, InPropertyPath]()
				{
					if (!LocalController.IsValid())
					{
						return false;
					}
					return LocalController->CanPropertyOverrideBindToCurve(InOpName, InPropertyPath);
				})
			.Text_Lambda([LocalController, InOverrideSetName, InOpName, InPropertyPath]()
				{
					if (!LocalController.IsValid())
					{
						return FText::FromString("");
					}

					if (!LocalController->CanPropertyOverrideBindToCurve(InOpName, InPropertyPath))
					{
						return LOCTEXT("CannotBindToCurveLabel", "Type incompatible with curves.");
					}
					return FText::FromName(LocalController->GetCurveBoundToPropertyOverride(InOverrideSetName, InOpName, InPropertyPath));
				})
			.OnTextCommitted_Lambda([LocalController, InOverrideSetName, InOpName, InPropertyPath](const FText& NewText, ETextCommit::Type CommitInfo)
			{
				if (CommitInfo == ETextCommit::OnEnter && LocalController.IsValid())
				{
					FName CurveName = FName(*NewText.ToString());
					LocalController->BindPropertyOverrideToCurve(CurveName, InOverrideSetName, InOpName, InPropertyPath);
					FSlateApplication::Get().DismissAllMenus();
				}
			})
		];

		MenuBuilder.AddWidget(CurveEntryWidget, FText::GetEmpty(), false, false);
	}
	MenuBuilder.EndSection();
	
	// SECTION 3: BIND TO VARIABLE
	MenuBuilder.BeginSection("VariableSection", LOCTEXT("VariableBindHeaderLabel", "Bind to Variable"));
	{
		const FProperty* PropertyToBind = InPropertyHandle->GetProperty();
		const FInstancedPropertyBag& VariablePropertyBag = AssetController->GetVariables().Bag;
		const UPropertyBag* VariablePropertyBagStruct = VariablePropertyBag.GetPropertyBagStruct();

		if (VariablePropertyBagStruct)
		{
			int32 NumAddedProperties = 0;
			const UEnum* PropertyTypeEnum = StaticEnum<EPropertyBagPropertyType>();

			for (const FPropertyBagPropertyDesc& VariablePropDesc : VariablePropertyBagStruct->GetPropertyDescs())
			{
				if (!VariablePropDesc.CachedProperty)
				{
					continue;
				}
				if (!AssetController->IsPropertyCompatibleForBinding(VariablePropDesc.CachedProperty, PropertyToBind))
				{
					continue;
				}

				NumAddedProperties++;
				
				const FName VariableName = VariablePropDesc.Name;
				FString TypeString = PropertyTypeEnum->GetNameStringByValue((int64)VariablePropDesc.ValueType).ToLower();
				FText ItemLabel = FText::Format(
					FText::FromString("{0} ({1})"), 
					FText::FromName(VariableName), 
					FText::FromString(TypeString)
				);

				MenuBuilder.AddMenuEntry(
					ItemLabel,
					FText::Format(LOCTEXT("BindToVariableTooltip", "Bind to variable: {0}"), FText::FromName(VariableName)),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateLambda([LocalController, VariableName, InOverrideSetName, InOpName, InPropertyPath]()
					{
						if (LocalController.IsValid())
						{
							LocalController->BindPropertyOverrideToVariable(VariableName, InOverrideSetName, InOpName, InPropertyPath);
						}
					}))
				);
			}
			
			if (NumAddedProperties == 0)
			{
				MenuBuilder.AddWidget(
					SNew(SBox)
					.Padding(FMargin(16.0f, 4.0f))
					[
						SNew(STextBlock)
						.Text(LOCTEXT("NoCompatibleVariablesLabel", "No compatible variables found."))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Font(IDetailLayoutBuilder::GetDetailFont())
					], 
					FText::GetEmpty(), false, false
				);
			}
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TMap<FString, TArray<int32>> FRetargetOverrideCustomization::MapOverridesToGroups(const TSharedPtr<IPropertyHandle>& OpOverridesHandle) const
{
    TMap<FString, TArray<int32>> GroupMap;

    if (!ensure(OpOverridesHandle.IsValid()))
    {
       return GroupMap;
    }

    // get the handle to the array of property overrides
    TSharedPtr<IPropertyHandle> OverridesArrayHandle = OpOverridesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRetargetOpOverrides, PropertyOverrides));
    if (!ensure(OverridesArrayHandle.IsValid()))
    {
       return GroupMap;
    }

    // iterate over all the property overrides
    uint32 NumOverrides = 0;
    OverridesArrayHandle->GetNumChildren(NumOverrides);
    for (uint32 OverrideIndex = 0; OverrideIndex < NumOverrides; ++OverrideIndex)
    {
       FString FullPath;
       GetOverridePropertyPath(OpOverridesHandle, OverrideIndex, FullPath);

       FString GroupLabel;
       FString RemainingPath;

       // check if the path contains an array element (e.g., "ChainsToAlign[3]->bEnabled")
       // we split at the last "]->" to ensure we group by the deepest array index
       if (FullPath.Split(TEXT("]->"), &GroupLabel, &RemainingPath, ESearchCase::IgnoreCase, ESearchDir::FromEnd))
       {
          // restore the bracket that was consumed by Split
          GroupLabel += TEXT("]");
       }
       else
       {
          // if there's no array index, it's a "loose" property, so just put these in a common group
          GroupLabel = TEXT("Op Settings");
       }

       // add the index to the corresponding group
       GroupMap.FindOrAdd(GroupLabel).Add(static_cast<int32>(OverrideIndex));
    }

    return MoveTemp(GroupMap);
}

void FRetargetOverrideCustomization::GetOverridePropertyPath(
    const TSharedPtr<IPropertyHandle>& InOpOverridesHandle,
    const int32 InOverrideIndex,
    FString& OutPath) const
{
    TSharedPtr<IPropertyHandle> OverridesArrayHandle = InOpOverridesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRetargetOpOverrides, PropertyOverrides));
    TSharedPtr<IPropertyHandle> OverrideHandle = OverridesArrayHandle->GetChildHandle(InOverrideIndex);
    TSharedPtr<IPropertyHandle> PathHandle =  OverrideHandle->GetChildHandle(FRetargetOpPropertyOverride::GetPropertyPathName());
    PathHandle->GetValue(OutPath);
}

TSharedPtr<IPropertyHandle> FRetargetOverrideCustomization::GetOverrideValueHandle(
    const TSharedPtr<IPropertyHandle>& InOpOverridesHandle,
    const int32 InOverrideIndex) const
{
    TSharedPtr<IPropertyHandle> OverridesArrayHandle = InOpOverridesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRetargetOpOverrides, PropertyOverrides));
    TSharedPtr<IPropertyHandle> OverrideHandle = OverridesArrayHandle->GetChildHandle(InOverrideIndex);
    return OverrideHandle->GetChildHandle(FRetargetOpPropertyOverride::GetValeAsStringName());
}

TSharedPtr<FStructOnScope>& FRetargetOverrideCustomization::GetCachedScopedStruct(const FName& InOpName, const UScriptStruct* OpSettingsType)
{
    TSharedPtr<FStructOnScope>& ScopedStruct = ScopedInstances.FindOrAdd(InOpName);

    if (!ScopedStruct.IsValid())
    {
       // initialize the struct with the override value
       ScopedStruct = MakeShared<FStructOnScope>(OpSettingsType);
    }
            
    return ScopedStruct;
}

bool FRetargetOverrideCustomization::CopyStoredValueToScopedStruct(
    const TSharedPtr<FStructOnScope>& InScopedStruct,
    const UScriptStruct* OpSettingsType,
    const FString& InPropertyPath,
    const TSharedPtr<IPropertyHandle>& InRawValueHandle)
{
    if (!ensure(InScopedStruct.IsValid()))
    {
       return false;
    }

    TArray<FRetargetOpPropertyOverride::FPropertySegment> Segments;
    if (!ensure(FRetargetOpPropertyOverride::GetSegmentsFromProperyPath(InPropertyPath, OpSettingsType, Segments)))
    {
       return false;
    }
    
    uint8* ValuePtr = FRetargetOpPropertyOverride::GetDataPointerFromPathSegments(InScopedStruct->GetStructMemory(), Segments, true);
    FProperty* LeafProp = FRetargetOpPropertyOverride::GetLeafProperty(Segments);
    FString CurrentValue;
    InRawValueHandle->GetValue(CurrentValue);
    if (!ensure(ValuePtr && LeafProp && !CurrentValue.IsEmpty()))
    {
       return false;
    }
    
    LeafProp->ImportText_Direct(*CurrentValue, ValuePtr, nullptr, PPF_None);
    
    return true;
}

const UScriptStruct* FRetargetOverrideCustomization::GetSettingsStruct(TSharedPtr<IPropertyHandle>& OpOverridesHandle) const
{
    TSharedPtr<IPropertyHandle> SettingsHandle = OpOverridesHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRetargetOpOverrides, ScriptStruct));
        
    const UScriptStruct* SettingsStruct = nullptr;
    void* StructData = nullptr;
    if (SettingsHandle->GetValueData(StructData) == FPropertyAccess::Success)
    {
       if (TObjectPtr<UScriptStruct>* StructPtr = static_cast<TObjectPtr<UScriptStruct>*>(StructData))
       {
          SettingsStruct = StructPtr->Get();
       }
    }

    return SettingsStruct;
}

TSharedPtr<IPropertyHandle> FRetargetOverrideCustomization::ResolveHandleFromPath(TSharedPtr<IPropertyHandle> InRootHandle, const FString& InPath)
{
	// validation
	if (!InRootHandle.IsValid() || InPath.IsEmpty())
	{
		return nullptr;
	}

	// tokenize the property path into steps
	TArray<FPropertyPathParser::FPropertyPathStep> Steps = FPropertyPathParser::ParsePath(InPath);

	// follow each step in the path by traversing the property handles
	TSharedPtr<IPropertyHandle> CurrentHandle = InRootHandle;
	for (const FPropertyPathParser::FPropertyPathStep& Step : Steps)
	{
		// navigate to the member property by name
		CurrentHandle = CurrentHandle->GetChildHandle(*Step.PropertyName);
        
		// if valid and this step requires an array index, dive deeper
		if (CurrentHandle.IsValid() && Step.IsArray())
		{
			CurrentHandle = CurrentHandle->GetChildHandle(Step.ArrayIndex);
		}

		// early out if any step in the chain fails
		if (!CurrentHandle.IsValid())
		{
			return nullptr;
		}
	}

	return CurrentHandle;
}


#undef LOCTEXT_NAMESPACE