// Copyright Epic Games, Inc. All Rights Reserved.
#include "RetargetEditor/IKRetargetOverridesTabSummoner.h"

#include "IDocumentation.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/SIKRetargetOverrideManager.h"

#define LOCTEXT_NAMESPACE "IKRetargetOverrideTabSummoner"

const FName FIKRetargetOverridesTabSummoner::TabID(TEXT("OverrideSets"));

FIKRetargetOverridesTabSummoner::FIKRetargetOverridesTabSummoner(
	const TSharedRef<FIKRetargetEditor>& InRetargetEditor)
	: FWorkflowTabFactory(TabID, InRetargetEditor),
	IKRetargetEditor(InRetargetEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRetargetOverridesTabLabel", "Override Sets");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.CompilerResults");

	ViewMenuDescription = LOCTEXT("IKRetargetOverrides_ViewMenu_Desc", "Override Sets");
	ViewMenuTooltip = LOCTEXT("IKRetargetOverrides_ViewMenu_ToolTip", "Show the Retargeting Overrides Tab");
}

TSharedPtr<SToolTip> FIKRetargetOverridesTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRetargetOverridesTooltip", "Create and edit property override sets."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRetargetProfiles_Window"));
}

TSharedRef<SWidget> FIKRetargetOverridesTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SIKRetargetOverrideManager, IKRetargetEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
