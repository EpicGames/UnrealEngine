// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargetPoseExporter.h"

#include "Animation/AnimMontage.h"
#include "AnimPose.h"
#include "AssetToolsModule.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "RetargetEditor/IKRetargetAnimInstance.h"
#include "RetargetEditor/IKRetargetEditorController.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/SavePackage.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Animation/AnimSequence.h"

#define LOCTEXT_NAMESPACE "IKRetargetExporter"


void FIKRetargetPoseExporter::Initialize(TSharedPtr<FIKRetargetEditorController> InController)
{
	Controller = InController;
}

void FIKRetargetPoseExporter::HandleImportFromPoseAsset()
{
	FIKRetargetEditorController* ControllerPtr = Controller.Pin().Get();
	
	ControllerPtr->SetRetargeterMode(ERetargeterOutputMode::EditRetargetPose);
	
	RetargetPoseToImport = nullptr;
	PosesInSelectedAsset.Empty();
	SelectedPose.Reset();

	// load the content browser module to display an asset picker
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	// the asset picker will only show pose assets
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.Filter.ClassPaths.Add(UPoseAsset::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &FIKRetargetPoseExporter::OnShouldFilterPoseToImport);
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &FIKRetargetPoseExporter::OnPoseAssetSelected);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = true;
	AssetPickerConfig.DefaultFilterMenuExpansion = EAssetTypeCategories::Animation;
	AssetPickerConfig.bAllowNullSelection = false;

	// hide all asset registry columns by default (we only really want the name and path)
	UObject* PoseAssetDefaultObject = UPoseAsset::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(PoseAssetDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	PoseAssetDefaultObject->GetAssetRegistryTags(TagsContext);
	const FName ColumnToKeep = FName("Number of Frames");
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		if (TagPair.Key != ColumnToKeep)
		{
			AssetPickerConfig.HiddenColumnNames.Add(TagPair.Key.ToString());
		}
	}

	// also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("HasVirtualizedData"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Disk Size"));

	ImportPoseWindow = SNew(SWindow)
	.Title(LOCTEXT("ImportRetargetPose_Label", "Import Retarget Pose"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.Padding(4)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				[
					SNew(SComboBox<TSharedPtr<FName> >) // pose list combo
					.OptionsSource(&PosesInSelectedAsset)
					.OnGenerateWidget(this, &FIKRetargetPoseExporter::OnGeneratePoseComboWidget)
					.OnSelectionChanged(this, &FIKRetargetPoseExporter::OnSelectPoseFromPoseAsset)
					.OnComboBoxOpening(this, &FIKRetargetPoseExporter::RefreshPoseList)
					[
						SNew(STextBlock).Text_Lambda([this]()
						{
							if (SelectedPose.IsValid())
							{
								return FText::FromName(*SelectedPose.Get());
							}

							return FText::FromString("No Pose Selected");
						})
					]
				]
			]
			
			+ SVerticalBox::Slot()
			.Padding(4)
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("ImportNewRetargetPoseButtonLabel", "Import New Retarget Pose"))
					.OnClicked(this, &FIKRetargetPoseExporter::ImportPoseAsset)
				]
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
					.OnClicked_Lambda( [this]()
					{
						ImportPoseWindow->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
			]	
		]
	];

	GEditor->EditorAddModalWindow(ImportPoseWindow.ToSharedRef());
	ImportPoseWindow.Reset();
}

void FIKRetargetPoseExporter::OnPoseAssetSelected(const FAssetData& SelectedAsset)
{
	RetargetPoseToImport = SelectedAsset.ToSoftObjectPath();

	RefreshPoseList();

	// set the selected pose to the first pose in the asset (or null if empty)
	if (PosesInSelectedAsset.IsEmpty())
	{
		SelectedPose = nullptr;
	}
	else
	{
		SelectedPose = PosesInSelectedAsset[0];
	}
}

TSharedRef<SWidget> FIKRetargetPoseExporter::OnGeneratePoseComboWidget(TSharedPtr<FName> Item) const
{
	return SNew(STextBlock).Text(FText::FromName(*Item.Get()));
}

void FIKRetargetPoseExporter::OnSelectPoseFromPoseAsset(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		SelectedPose = Item;
	}
}

void FIKRetargetPoseExporter::RefreshPoseList()
{
	// get all poses in the selected asset
	PosesInSelectedAsset.Empty();
	const TObjectPtr<UPoseAsset> PoseAsset = Cast<UPoseAsset>(RetargetPoseToImport.TryLoad());
	if (!PoseAsset)
	{
		return;
	}

	// store pose names in list used by combobox
	for (const FName& PoseName : PoseAsset->GetPoseFNames())
	{
		PosesInSelectedAsset.Add(MakeShared<FName>(PoseName));
	}
}

FReply FIKRetargetPoseExporter::ImportPoseAsset() const
{
	// close the import window
	ImportPoseWindow->RequestDestroyWindow();

	// check that user actually selected an asset to import
	if (RetargetPoseToImport.IsNull())
	{
		const FText Message = LOCTEXT("NoPoseToImport", "No pose asset specified. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}

	// load the selected pose asset
	const TObjectPtr<UPoseAsset> PoseAsset = Cast<UPoseAsset>(RetargetPoseToImport.TryLoad());
	if (!PoseAsset)
	{
		const FText Message = LOCTEXT("PoseLoadFailed", "Pose asset could not be loaded. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}

	// get the requested pose out of the pose asset
	if (!SelectedPose.IsValid())
	{
		const FText Message = LOCTEXT("PoseNotSelected", "No pose was specified to load. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}

	// validate that pose exists within the selected pose asset
	const FName NameOfPoseToImport = *SelectedPose.Get();
	const int32 IndexOfPoseToImport = PoseAsset->GetPoseIndexByName(NameOfPoseToImport);
	if (IndexOfPoseToImport == INDEX_NONE)
	{
		const FText Message = LOCTEXT("PoseNotInAsset", "Specified pose not found in the pose asset. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}

	// validate mesh is available to load sequence on
	FIKRetargetEditorController* ControllerPtr = Controller.Pin().Get();
	ERetargetSourceOrTarget SourceOrTarget = ControllerPtr->GetSourceOrTarget();
	USkeletalMesh* Mesh = ControllerPtr->GetSkeletalMesh(SourceOrTarget);
	if (!Mesh)
	{
		const FText Message = LOCTEXT("ImportFailedNoMesh", "Skeletal Mesh not found. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}

	// validate IK Rig is available to sort the pose offsets
	const UIKRigDefinition* IKRig = ControllerPtr->AssetController->GetIKRig(SourceOrTarget);
	if (!IKRig)
	{
		const FText Message = LOCTEXT("ImportFailedNoIKRig", "IK Rig not found. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}

	// get pelvis bone name
	const FName PelvisBoneName = IKRig->GetPelvis();

	// get transforms of loaded pose
	TArray<FTransform> LocalBoneTransformFromPose;
	PoseAsset->GetFullPose(IndexOfPoseToImport, LocalBoneTransformFromPose);

	// create a new retarget pose to store the data from the selected retarget pose asset
	const FName NewPoseName = ControllerPtr->AssetController->CreateRetargetPose(FName(PoseAsset->GetName()), SourceOrTarget);
	FIKRetargetPose& NewPose = ControllerPtr->AssetController->GetRetargetPoses(SourceOrTarget)[NewPoseName];
	
	// iterate over all bones in the reference skeleton and compare against bone in the pose asset
	// store sparse set of deltas in the new retarget pose 
	FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();
	const TArray<FTransform>& RefPose = RefSkeleton.GetRefBonePose();
	for (int32 BoneIndex=0; BoneIndex<RefSkeleton.GetNum(); ++BoneIndex)
	{
		const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
		const int32 PoseTrackIndex = PoseAsset->GetTrackIndexByName(BoneName);
		if (PoseTrackIndex == INDEX_NONE)
		{
			continue;
		}
		
		// record a global space translation offset for the root bone
		if (BoneName == PelvisBoneName)
		{
			FTransform GlobalRefTransform = FAnimationRuntime::GetComponentSpaceTransform(RefSkeleton, RefPose, BoneIndex);
			FTransform GlobalImportedTransform = PoseAsset->GetComponentSpaceTransform(BoneName, LocalBoneTransformFromPose);
			NewPose.SetRootTranslationDelta(GlobalImportedTransform.GetLocation() - GlobalRefTransform.GetLocation());
		}
		
		// record a local space delta rotation (if there is one)
		const FTransform& LocalRefTransform = RefPose[BoneIndex];
		const FTransform& LocalImportedTransform = LocalBoneTransformFromPose[PoseTrackIndex];
		const FQuat DeltaRotation = LocalRefTransform.GetRotation().Inverse() * LocalImportedTransform.GetRotation();
		const double DeltaAngle = FMath::RadiansToDegrees(DeltaRotation.GetAngle());
		constexpr double MinAngleThreshold = 0.05;
		if (DeltaAngle >= MinAngleThreshold)
		{
			NewPose.SetDeltaRotationForBone(BoneName, DeltaRotation);
		}
	}

	// update view with new pose
	ControllerPtr->RefreshAllViews();
	
	return FReply::Unhandled();
}

bool FIKRetargetPoseExporter::OnShouldFilterPoseToImport(const FAssetData& AssetData) const
{
	// is this a pose asset
	if (!AssetData.IsInstanceOf(UPoseAsset::StaticClass()))
	{
		return true;
	}

	// get currently edited skeleton
	const USkeleton* DesiredSkeleton = Controller.Pin()->GetSkeleton(Controller.Pin()->GetSourceOrTarget());
	if (!DesiredSkeleton)
	{
		return true;
	}

	return !DesiredSkeleton->IsCompatibleForEditor(AssetData);
}

void FIKRetargetPoseExporter::HandleImportFromSequenceAsset()
{
	FIKRetargetEditorController* ControllerPtr = Controller.Pin().Get();
	
	ControllerPtr->SetRetargeterMode(ERetargeterOutputMode::EditRetargetPose);
	
	SequenceToImportAsPose = nullptr;

	// get a unique pose name to use as suggestion
	const FString DefaultImportedPoseName = LOCTEXT("ImportedRetargetPoseName", "ImportedRetargetPose").ToString();
	const FName UniqueNewPoseName = ControllerPtr->AssetController->MakePoseNameUnique(DefaultImportedPoseName, ControllerPtr->GetSourceOrTarget());
	ImportedPoseName = FText::FromName(UniqueNewPoseName);

	// load the content browser module to display an asset picker
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	// the asset picker will only show animation sequences compatible with the preview mesh
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = true;
	AssetPickerConfig.DefaultFilterMenuExpansion = EAssetTypeCategories::Animation;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &FIKRetargetPoseExporter::OnShouldFilterSequenceToImport);
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &FIKRetargetPoseExporter::OnSequenceSelectedForPose);
	AssetPickerConfig.bAllowNullSelection = false;

	// hide all asset registry columns by default (we only really want the name and path)
	UObject* AnimSequenceDefaultObject = UAnimSequence::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(AnimSequenceDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	AnimSequenceDefaultObject->GetAssetRegistryTags(TagsContext);
	FName ColumnToKeep = FName("Number of Frames");
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		if (TagPair.Key != ColumnToKeep)
		{
			AssetPickerConfig.HiddenColumnNames.Add(TagPair.Key.ToString());
		}
	}

	// also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("HasVirtualizedData"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Disk Size"));

	// create pop-up window for user to select animation sequence asset to import as a retarget pose]
	ImportPoseFromSequenceWindow = SNew(SWindow)
	.Title(LOCTEXT("ImportRetargetPoseFromSequenceAsset_Label", "Import Retarget Pose from Sequence Asset"))
	.ClientSize(FVector2D(500, 600))
	.SupportsMinimize(false) .SupportsMaximize(false)
	[
		SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			SNew(SVerticalBox)
			
			+ SVerticalBox::Slot()
			.Padding(4)
			[
				ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
			]

			+ SVerticalBox::Slot()
			.Padding(4)
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ImportFrame_Label", "Sequence Frame: "))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Right)
				.Padding(2.0f, 0.0f)
				[
					SNew(SNumericEntryBox<int32>)
					.ToolTipText(LOCTEXT("ArrayIndex", "Frame of sequence to import pose from."))
					.AllowSpin(true)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.MinValue(0)
					.Value_Lambda([this]
					{
						return FrameOfSequenceToImport;
					})
					.OnValueChanged(SNumericEntryBox<int32>::FOnValueChanged::CreateLambda([this](int32 Value)
					{
						FrameOfSequenceToImport = Value;
					}))
				]
			]

			+ SVerticalBox::Slot()
			.Padding(4)
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ImportName_Label", "Pose Name: "))
				]
				
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SEditableTextBox)
					.Text(FText::FromName(UniqueNewPoseName))
					.OnTextChanged(FOnTextChanged::CreateLambda([this](const FText InText)
					{
						ImportedPoseName = InText;
					}))
				]
			]
			
			+ SVerticalBox::Slot()
			.Padding(4)
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("ImportAsRetargetPoseButtonLabel", "Import New Retarget Pose"))
					.OnClicked(this, &FIKRetargetPoseExporter::OnImportPoseFromSequence)
				]
				
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "Button")
					.TextStyle( FAppStyle::Get(), "DialogButtonText" )
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
					.OnClicked_Lambda( [this]()
					{
						ImportPoseFromSequenceWindow->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
			]	
		]
	];

	GEditor->EditorAddModalWindow(ImportPoseFromSequenceWindow.ToSharedRef());
	ImportPoseFromSequenceWindow.Reset();
}

bool FIKRetargetPoseExporter::OnShouldFilterSequenceToImport(const FAssetData& AssetData) const
{
	// is this an animation sequence?
	if (!AssetData.IsInstanceOf(UAnimSequence::StaticClass()))
	{
		return true;
	}

	// get currently edited skeleton
	const USkeleton* DesiredSkeleton = Controller.Pin()->GetSkeleton(Controller.Pin()->GetSourceOrTarget());
	if (!DesiredSkeleton)
	{
		return true;
	}

	return !DesiredSkeleton->IsCompatibleForEditor(AssetData);
}

FReply FIKRetargetPoseExporter::OnImportPoseFromSequence()
{
	// close the pop-up window
	ImportPoseFromSequenceWindow->RequestDestroyWindow();

	// check that a sequence was selected to import
	if (SequenceToImportAsPose.IsNull())
	{
		const FText Message = LOCTEXT("ImportFailedNoSequence", "No sequence selected. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}

	// load sequence to read pose from
	const TObjectPtr<UAnimSequence> AnimSequence = Cast<UAnimSequence>(SequenceToImportAsPose.TryLoad());
	if (!AnimSequence)
	{
		const FText Message = LOCTEXT("ImportFailedSeqNotLoaded", "Sequence could not be loaded. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}

	// validate mesh is available to load sequence on
	FIKRetargetEditorController* ControllerPtr = Controller.Pin().Get();
	ERetargetSourceOrTarget SourceOrTarget = ControllerPtr->GetSourceOrTarget();
	USkeletalMesh* Mesh = ControllerPtr->GetSkeletalMesh(SourceOrTarget);
	if (!Mesh)
	{
		const FText Message = LOCTEXT("ImportFailedNoMesh", "Skeletal Mesh not found. Import aborted.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return FReply::Handled();
	}
	
	// ensure we evaluate the source animation using the skeletal mesh proportions that were evaluated in the viewport
	FAnimPoseEvaluationOptions EvaluationOptions = FAnimPoseEvaluationOptions();
	EvaluationOptions.OptionalSkeletalMesh = Mesh;
	
	// evaluate the sequence at the desired frame
	FAnimPose ImportedPose;
	FrameOfSequenceToImport = FMath::Clamp(FrameOfSequenceToImport, 0, AnimSequence->GetNumberOfSampledKeys());
	UAnimPoseExtensions::GetAnimPoseAtFrame(AnimSequence, FrameOfSequenceToImport, EvaluationOptions, ImportedPose);

	// create a new retarget pose to hold imported data
	const FName NewPose = ControllerPtr->AssetController->CreateRetargetPose(FName(ImportedPoseName.ToString()), SourceOrTarget);

	// record delta pose for all bones being retargeted
	FIKRetargetPose& ImportedRetargetPose = ControllerPtr->AssetController->GetRetargetPoses(SourceOrTarget)[NewPose];
	
	// get all imported bone transforms and record them in the retarget pose
	const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();
	const TArray<FTransform>& RefPose = RefSkeleton.GetRefBonePose();
	int32 NumBones = RefSkeleton.GetNum();
	const FName PelvisBoneName = ControllerPtr->AssetController->GetPelvisBone(ControllerPtr->GetSourceOrTarget());
	for (int32 BoneIndex = 0; BoneIndex < NumBones; BoneIndex++)
	{
		const FName& BoneName = RefSkeleton.GetBoneName(BoneIndex);
		
		// if this is the pelvis, we want to record the translation delta as well
		if (BoneName == PelvisBoneName)
		{
			const FTransform GlobalTransformImported = UAnimPoseExtensions::GetBonePose(ImportedPose, BoneName, EAnimPoseSpaces::World);
			const FTransform GlobalTransformReference = UAnimPoseExtensions::GetRefBonePose(ImportedPose, BoneName, EAnimPoseSpaces::World);
			const FVector TranslationDelta = GlobalTransformImported.GetLocation() - GlobalTransformReference.GetLocation();
			ImportedRetargetPose.SetRootTranslationDelta(TranslationDelta);

			// rotation offsets are interpreted as relative to the parent (local), but in the case of the pelvis bone,
			// when we generate the retarget pose, it's parents will be left at ref pose, so we need to generate a local
			// rotation offset relative to the ref pose parent, NOT the (potentially) posed parent transform from the animation.
			FTransform GlobalParentTransformInRefPose = FTransform::Identity;
			const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			if (ParentIndex != INDEX_NONE)
			{
				const FName& ParentBoneName = RefSkeleton.GetBoneName(ParentIndex);
				GlobalParentTransformInRefPose = UAnimPoseExtensions::GetRefBonePose(ImportedPose, ParentBoneName, EAnimPoseSpaces::World);
			}

			// this is a bit crazy, but we have to generate a delta rotation in the local space of the pelvis
			// bone while treating the root bone as being in global space since the retarget pose does not consider any
			// bones above it.
			const FQuat GlobalDeltaRotation = GlobalTransformImported.GetRotation() * GlobalTransformReference.GetRotation().Inverse();
			const FQuat BoneGlobalOrig = GlobalTransformReference.GetRotation();
			const FQuat BoneGlobalPlusOffset = GlobalDeltaRotation * BoneGlobalOrig;
			const FQuat ParentInv = GlobalParentTransformInRefPose.GetRotation().Inverse();
			const FQuat BoneLocal = ParentInv * BoneGlobalOrig;
			const FQuat BoneLocalPlusOffset = ParentInv * BoneGlobalPlusOffset;
			const FQuat BoneLocalOffset = BoneLocal * BoneLocalPlusOffset.Inverse();
			
			ImportedRetargetPose.SetDeltaRotationForBone(BoneName, BoneLocalOffset.Inverse());
		}
		else
		{
			// record the delta rotation
			const FTransform LocalTransformImported = UAnimPoseExtensions::GetBonePose(ImportedPose, BoneName, EAnimPoseSpaces::Local);
			const FTransform LocalTransformReference = RefPose[BoneIndex];
			const FQuat DeltaRotation = LocalTransformReference.GetRotation().Inverse() * LocalTransformImported.GetRotation();
			// only if it's different than the ref pose
			const double DeltaAngle = FMath::RadiansToDegrees(DeltaRotation.GetAngle());
			constexpr double MinAngleThreshold = 0.05;
			if (DeltaAngle >= MinAngleThreshold)
			{
				ImportedRetargetPose.SetDeltaRotationForBone(BoneName, DeltaRotation);
			}
		}
	}

	// notify user of new pose
	const FText Message = FText::Format(LOCTEXT("ImportSuccess", "Imported new retarget pose: {0}"), FText::FromName(NewPose));
	NotifyUser(Message, SNotificationItem::CS_Success);
	return FReply::Unhandled();
}

void FIKRetargetPoseExporter::OnSequenceSelectedForPose(const FAssetData& SelectedAsset)
{
	SequenceToImportAsPose = SelectedAsset.ToSoftObjectPath();
}

void FIKRetargetPoseExporter::HandleExportPoseAsset()
{
	const FIKRetargetEditorController* ControllerPtr = Controller.Pin().Get();

	// get mesh to export pose
	const ERetargetSourceOrTarget SourceOrTarget = ControllerPtr->GetSourceOrTarget();
	USkeletalMesh* Mesh = ControllerPtr->GetSkeletalMesh(SourceOrTarget);
	UDebugSkelMeshComponent* MeshComponent = ControllerPtr->GetSkeletalMeshComponent(SourceOrTarget);
	if (!(Mesh && MeshComponent))
	{
		const FText Message = LOCTEXT("IKRigExportPoseFailed", "Export pose failed. No mesh found.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return;
	}

	FString PoseName = ControllerPtr->GetCurrentPoseName().ToString();
	PoseName.RemoveSpacesInline();
	
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DefaultPath = ControllerPtr->AssetController->GetAsset()->GetPackage()->GetPathName();
	SaveAssetDialogConfig.DefaultAssetName = PoseName;
	SaveAssetDialogConfig.AssetClassNames.Add(UPoseAsset::StaticClass()->GetClassPathName());
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("ExportRetargetPoseDialogTitle", "Export Retarget Pose");

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
	if (SaveObjectPath.IsEmpty())
	{
		const FText Message = LOCTEXT("IKRigExportPoseCancelled", "Export pose cancelled.");
		NotifyUser(Message, SNotificationItem::CS_Fail);
		return;
	}

	// create new pose asset
	const FString PackagePath = FPackageName::ObjectPathToPackageName(SaveObjectPath);
	const FString AssetName = FPaths::GetBaseFilename(PackagePath, true);
	UPackage *Package = CreatePackage(*PackagePath);
	UPoseAsset* NewPoseAsset = NewObject<UPoseAsset>(Package, UPoseAsset::StaticClass(), FName(AssetName), RF_Public | RF_Standalone);

	// fill new pose asset with current pose
	NewPoseAsset->SetSkeleton(const_cast<USkeleton*>(Mesh->GetSkeleton()));
	NewPoseAsset->SetPreviewMesh(Mesh);
	const FName NewPoseName = NewPoseAsset->AddPoseWithUniqueName(MeshComponent);

	// mark asset dirty 
	FAssetRegistryModule::AssetCreated(NewPoseAsset);
	NewPoseAsset->MarkPackageDirty();

	// show in content browser
	TArray<UObject*> ObjectsToSync;
	ObjectsToSync.Add(NewPoseAsset);
	GEditor->SyncBrowserToObjects(ObjectsToSync);
	
	// notify user of newly created asset
	const FText Message = FText::Format(LOCTEXT("IKRigExportPoseSuccess", "Exported Pose Asset: {0} with pose, '{1}'."), FText::FromString(AssetName), FText::FromName(NewPoseName));
	NotifyUser(Message, SNotificationItem::CS_Success);
}

void FIKRetargetPoseExporter::NotifyUser(const FText& Message, SNotificationItem::ECompletionState NotificationType)
{
	FNotificationInfo NotifyInfo(Message);
	NotifyInfo.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(NotifyInfo)->SetCompletionState(NotificationType);
}

#undef LOCTEXT_NAMESPACE
