// Copyright Epic Games, Inc. All Rights Reserved.
#include "RetargetEditor/IKRetargetVariablesTabSummoner.h"

#include "IDocumentation.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/SIKRetargetVariablesEditor.h"

#define LOCTEXT_NAMESPACE "IKRetargetVariablesTabSummoner"

const FName FIKRetargetVariablesTabSummoner::TabID(TEXT("Variables"));

FIKRetargetVariablesTabSummoner::FIKRetargetVariablesTabSummoner(
	const TSharedRef<FIKRetargetEditor>& InRetargetEditor)
	: FWorkflowTabFactory(TabID, InRetargetEditor),
	IKRetargetEditor(InRetargetEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRetargetVariablesTabLabel", "Variables");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.CompilerResults");

	ViewMenuDescription = LOCTEXT("IKRetargetVariables_ViewMenu_Desc", "Variables");
	ViewMenuTooltip = LOCTEXT("IKRetargetVariables_ViewMenu_ToolTip", "Show the Retargeting Variables Tab");
}

TSharedPtr<SToolTip> FIKRetargetVariablesTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRetargetVariablesTooltip", "Create and edit variables to drive overrides."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRetargetVariables_Window"));
}

TSharedRef<SWidget> FIKRetargetVariablesTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SIKRetargetVariablesEditor, IKRetargetEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
