// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargeterController.h"

#include "ScopedTransaction.h"
#include "Engine/SkeletalMesh.h"
#include "Retargeter/IKRetargeter.h"
#include "RigEditor/IKRigController.h"
#include "IKRigEditor.h"
#include "PropertyBagDetails.h"
#include "RetargetEditor/IKRetargeterPoseGenerator.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/RetargetOps/CurveRemapOp.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"
#include "Retargeter/RetargetOps/IKChainsOp.h"
#include "Retargeter/RetargetOps/PelvisMotionOp.h"
#include "Retargeter/RetargetOps/RootMotionGeneratorOp.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "RigEditor/IKRigStructViewer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargeterController)

#define LOCTEXT_NAMESPACE "IKRetargeterController"


UIKRetargeterController* UIKRetargeterController::GetController(const UIKRetargeter* InRetargeterAsset)
{
	if (!InRetargeterAsset)
	{
		return nullptr;
	}

	if (!InRetargeterAsset->Controller)
	{
		UIKRetargeterController* Controller = NewObject<UIKRetargeterController>();
		Controller->Asset = const_cast<UIKRetargeter*>(InRetargeterAsset);
		Controller->Asset->Controller = Controller;
	}

	return Cast<UIKRetargeterController>(InRetargeterAsset->Controller);
}

UIKRetargeterController::UIKRetargeterController()
{
	// initialize Op Settings viewer
	UIKRigStructViewer* SettingsViewer = CreateDefaultSubobject<UIKRigStructViewer>(TEXT("RetargetSettingsViewer"));
	SettingsViewer->OnNeedsReinitialized().AddUObject(this, &UIKRetargeterController::OnOpPropertyChanged);
	StructViewers.Add(ERetargetStructViewerMode::Settings, SettingsViewer);

	// initialize Override Sets viewer
	UIKRigStructViewer* OverrideViewer = CreateDefaultSubobject<UIKRigStructViewer>(TEXT("RetargetOverrideSetsViewer"));
	OverrideViewer->OnNeedsReinitialized().AddUObject(this, &UIKRetargeterController::OnOpPropertyChanged);
	StructViewers.Add(ERetargetStructViewerMode::OverrideSets, OverrideViewer);
}

void UIKRetargeterController::PostInitProperties()
{
	Super::PostInitProperties();
	AutoPoseGenerator = MakeUnique<FRetargetAutoPoseGenerator>(this);
}

UIKRetargeter* UIKRetargeterController::GetAsset() const
{
	return Asset;
}

void UIKRetargeterController::CleanAsset() const
{
	FScopeLock Lock(&ControllerLock);

	CleanChainMaps();
	CleanPoseList(ERetargetSourceOrTarget::Source);
	CleanPoseList(ERetargetSourceOrTarget::Target);
}

void UIKRetargeterController::SetIKRig(const ERetargetSourceOrTarget SourceOrTarget, UIKRigDefinition* IKRig) const
{
	FScopeLock Lock(&ControllerLock);
	
	FScopedReinitializeIKRetargeter Reinitialize(this);
	
	if (SourceOrTarget == ERetargetSourceOrTarget::Source)
	{
		Asset->SourceIKRigAsset = IKRig;
		Asset->SourcePreviewMesh = IKRig ? IKRig->GetPreviewMesh() : Asset->SourcePreviewMesh;
	}
	else
	{
		Asset->TargetIKRigAsset = IKRig;
		Asset->TargetPreviewMesh = IKRig ? IKRig->GetPreviewMesh() : Asset->TargetPreviewMesh;
	}
	
	// re-ask to fix root height for this mesh
	if (IKRig)
	{
		SetAskedToFixRootHeightForMesh(GetPreviewMesh(SourceOrTarget), false);
	}

	// update ops with new source
	// NOTE: we do NOT auto-update the target IK rig as this may be overridden
	if (SourceOrTarget == ERetargetSourceOrTarget::Source)
	{
		const int32 NumOps = GetNumRetargetOps();
		for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
		{
			FIKRetargetOpBase* Op = GetRetargetOpByIndex(OpIndex);
			FRetargetChainMapping* ChainMapping = Op->GetChainMapping();
			if (!ChainMapping)
			{
				continue;
			}

			const UIKRigDefinition* TargetIKRig = GetTargetIKRigForOp(Op->GetName());
			ChainMapping->ReinitializeWithIKRigs(Asset->SourceIKRigAsset, TargetIKRig);
		}
	}

	// update any editors attached to this asset
	IKRigReplaced.Broadcast(SourceOrTarget);
	PreviewMeshReplaced.Broadcast(SourceOrTarget);
}

const UIKRigDefinition* UIKRetargeterController::GetIKRig(const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	return Asset->GetIKRig(SourceOrTarget);
}

TArray<UIKRigDefinition*> UIKRetargeterController::GetAllTargetIKRigs() const
{
	TArray<UIKRigDefinition*> AllTargetIKRigs;
	
	const int32 NumOps = GetNumRetargetOps();
	for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
	{
		FIKRetargetOpBase* Op = GetRetargetOpByIndex(OpIndex);
		if (UIKRigDefinition* TargetIKRig = const_cast<UIKRigDefinition*>(Op->GetCustomTargetIKRig()))
		{
			AllTargetIKRigs.AddUnique(TargetIKRig);
		}
	}
	
	return MoveTemp(AllTargetIKRigs);
}

UIKRigDefinition* UIKRetargeterController::GetIKRigWriteable(const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	return Asset->GetIKRigWriteable(SourceOrTarget);
}

void UIKRetargeterController::SetPreviewMesh(
	const ERetargetSourceOrTarget SourceOrTarget,
	USkeletalMesh* PreviewMesh) const
{
	FScopeLock Lock(&ControllerLock);

	FScopedTransaction Transaction(LOCTEXT("SetPreviewMesh_Transaction", "Set Preview Mesh"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	
	if (SourceOrTarget == ERetargetSourceOrTarget::Source)
	{
		Asset->SourcePreviewMesh = PreviewMesh;
	}
	else
	{
		Asset->TargetPreviewMesh = PreviewMesh;
	}
	
	// re-ask to fix root height for this mesh
	SetAskedToFixRootHeightForMesh(PreviewMesh, false);
	
	// update any editors attached to this asset
	PreviewMeshReplaced.Broadcast(SourceOrTarget);
}

USkeletalMesh* UIKRetargeterController::GetPreviewMesh(const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	
	// return the preview mesh if one is provided
	const TSoftObjectPtr<USkeletalMesh> PreviewMesh = SourceOrTarget == ERetargetSourceOrTarget::Source ? Asset->SourcePreviewMesh : Asset->TargetPreviewMesh;
	if (!PreviewMesh.IsNull())
	{
		return PreviewMesh.LoadSynchronous();
	}

	// fallback to preview mesh from IK Rig asset
	if (const UIKRigDefinition* IKRig = GetIKRig(SourceOrTarget))
	{
		return IKRig->GetPreviewMesh();
	}
	
	return nullptr;
}

int32 UIKRetargeterController::AddRetargetOp(const FString InIKRetargetOpType) const
{
	UScriptStruct* OpType = FindObject<UScriptStruct>(nullptr, *InIKRetargetOpType);
	if (!OpType)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Specified retarget op type was not found, %ls.", *InIKRetargetOpType);
		return INDEX_NONE;
	}

	return AddRetargetOp(OpType);
}

bool UIKRetargeterController::RemoveRetargetOp(const int32 InOpIndex) const
{
	FIKRetargetOpBase* Op = GetRetargetOpByIndex(InOpIndex);
	if (!Op)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget Op not removed. Invalid index, %d.", InOpIndex);
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveRetargetOp_Label", "Remove Retarget Op"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();

	// get full list of ops to remove (including children, if any)
	const TArray<int32> ChildrenOpIndices = GetChildOpIndices(InOpIndex);
	TArray<int32> IndicesToRemove = {InOpIndex};
	IndicesToRemove.Append(ChildrenOpIndices);
	TSet<FName> NamesToRemove;
	for (int32 OpIndex : IndicesToRemove)
	{
		const FName OpNameToRemove = GetRetargetOpByIndex(OpIndex)->GetName();
		NamesToRemove.Add(OpNameToRemove);
	}

	// remove any overrides associated with this op
	TMap<FName, FRetargetOverrideSet>& OverrideSets = Asset->GetOverrideSetsWriteable();
	for (TTuple<FName, FRetargetOverrideSet>& OverrideSetPair : OverrideSets)
	{
		FRetargetOverrideSet& OverrideSet = OverrideSetPair.Value;
		OverrideSet.OpOverrides.RemoveAll([&NamesToRemove](const FRetargetOpOverrides& Element)
		{
			return NamesToRemove.Contains(Element.OpName);
		});
	}

	// remove the op itself
	Algo::Sort(IndicesToRemove); 
	Algo::Reverse(IndicesToRemove); // high to low (children are always before parent)
	for (const int32 OpToRemove : IndicesToRemove)
	{
		Asset->RetargetOps.RemoveAt(OpToRemove);
	}

	OverrideSetsModified.Broadcast();
	return true;
}

bool UIKRetargeterController::RemoveAllOps() const
{
	if (Asset->RetargetOps.IsEmpty())
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveAllRetargetOps_Label", "Remove All Retarget Ops"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();
	Asset->RetargetOps.Empty();
	return true;
}

FName UIKRetargeterController::SetOpName(const FName InName, const int32 InOpIndex) const
{
	FIKRetargetOpBase* Op = GetRetargetOpByIndex(InOpIndex);
	if (!Op)
	{
		return NAME_None;
	}
	
	FScopedTransaction Transaction(LOCTEXT("SetOpName_Label", "Rename Retarget Op"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();

	const FName OldOpName = GetOpName(InOpIndex);
	FName UniqueName = GetUniqueOpName(InName, InOpIndex);
	Op->SetName(UniqueName);
	
	// update any children pointing at the old name
	if (OldOpName != NAME_None)
	{
		for (FInstancedStruct& OpStruct : Asset->RetargetOps)
		{
			FIKRetargetOpBase& OtherOp = OpStruct.GetMutable<FIKRetargetOpBase>();
			if (OtherOp.GetParentOpName() == OldOpName)
			{
				OtherOp.SetParentOpName(UniqueName);
			}
		}
	}

	// update any profiles pointing at the old name
	for (TTuple<FName, FRetargetProfile>& ProfilePair : Asset->Profiles)
	{
		if (FRetargetOpProfile* OpProfile = ProfilePair.Value.GetOpProfileByName(OldOpName))
		{
			OpProfile->OpToApplySettingsTo = UniqueName;
		}
	}

	// update any override sets pointing to the old name
	for (TTuple<FName, FRetargetOverrideSet>& OverrideSetPair : Asset->OverrideSets)
	{
		if (FRetargetOpOverrides* OpOverride = OverrideSetPair.Value.FindOpOverrideByOpName(OldOpName))
		{
			OpOverride->OpName = UniqueName;
		}
	}
	
	return UniqueName;
}

FName UIKRetargeterController::GetOpName(const int32 InOpIndex) const
{
	const FIKRetargetOpBase* Op = GetRetargetOpByIndex(InOpIndex);
	if (!Op)
	{
		return NAME_None;
	}
	
	return Op->GetName();
}

bool UIKRetargeterController::SetParentOpByName(
	const FName InChildOpName,
	const FName InParentOpName) const
{
	FIKRetargetOpBase* ChildOp = GetRetargetOpByName(InChildOpName);
	if (!ChildOp)
	{
		return false; // child not found
	}

	FIKRetargetOpBase* ParentOp = GetRetargetOpByName(InParentOpName);
	if (!ParentOp)
	{
		return false; // parent not found
	}

	if (ChildOp->GetParentOpType() != ParentOp->GetType())
	{
		return false; // wrong type of parent
	}

	FScopedTransaction Transaction(LOCTEXT("SetOpParent_Label", "Set Op Parent"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();
	
	ChildOp->SetParentOpName(InParentOpName);
	
	Asset->CleanOpStack();
	
	return true;
}

FName UIKRetargeterController::GetParentOpByName(const FName InOpName) const
{
	FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
	if (!Op)
	{
		return NAME_None; // child not found
	}
	
	return Op->GetParentOpName();
}

int32 UIKRetargeterController::GetIndexOfOpByName(const FName InOpName) const
{
	return Asset->RetargetOps.IndexOfByPredicate([&](const FInstancedStruct& InOpStruct)
	{
		const FIKRetargetOpBase* Op = InOpStruct.GetPtr<FIKRetargetOpBase>();
		return Op->GetName().IsEqual(InOpName, ENameCase::IgnoreCase); 
	});
}

void UIKRetargeterController::AddDefaultOps() const
{
	FScopedTransaction Transaction(LOCTEXT("AddDefaultOps_Label", "Add Default Ops"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();

	// add set of default ops for basic retargeting
	AddRetargetOp(FIKRetargetPelvisMotionOp::StaticStruct());
	AddRetargetOp(FIKRetargetFKChainsOp::StaticStruct());
	const int32 RunIKIndex = AddRetargetOp(FIKRetargetRunIKRigOp::StaticStruct());
	const FName RunIKOpName = GetOpName(RunIKIndex);
	AddRetargetOp(FIKRetargetRootMotionOp::StaticStruct());
	AddRetargetOp(FIKRetargetCurveRemapOp::StaticStruct());
}

void UIKRetargeterController::RunOpInitialSetup(const int32 InOpIndex) const
{
	FIKRetargetOpBase* Op = GetRetargetOpByIndex(InOpIndex);
	if (!Op)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget Op initial setup skipped. Invalid index, %d.", InOpIndex);
		return;
	}
	
	FScopedTransaction Transaction(LOCTEXT("RunOpInitialSetup_Label", "Run Op Initial Setup"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();

	FIKRetargetOpBase* ParentOp = GetRetargetOpByIndex(GetParentOpIndex(InOpIndex));
	Op->OnAddedToStack(GetAsset(), ParentOp);
}

void UIKRetargeterController::AssignIKRigToAllOps(const ERetargetSourceOrTarget InSourceOrTarget, const UIKRigDefinition* InIKRig) const
{
	FScopedTransaction Transaction(LOCTEXT("RunOpInitialSetup_Label", "Run Op Initial Setup"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();

	// assign ik rig in reverse order so parent ops consume it before children
	for (int32 OpIndex = Asset->RetargetOps.Num() - 1; OpIndex >= 0; --OpIndex)
	{
		FInstancedStruct& OpStruct = Asset->RetargetOps[OpIndex];
		FIKRetargetOpBase& Op = OpStruct.GetMutable<FIKRetargetOpBase>();
		FIKRetargetOpBase* ParentOp = GetRetargetOpByName(GetParentOpByName(Op.GetName()));
		Op.OnAssignIKRig(InSourceOrTarget, InIKRig, ParentOp);
	}
}

FIKRetargetOpBase* UIKRetargeterController::GetRetargetOpByIndex(const int32 InOpIndex) const
{
	if (Asset->RetargetOps.IsValidIndex(InOpIndex))
	{
		return Asset->RetargetOps[InOpIndex].GetMutablePtr<FIKRetargetOpBase>();
	}
	
	return nullptr;
}

FIKRetargetOpBase* UIKRetargeterController::GetRetargetOpByName(const FName InOpName) const
{
	return GetRetargetOpByIndex(GetIndexOfOpByName(InOpName));
}

TArray<int32> UIKRetargeterController::GetChildOpIndices(const int32 InOpIndex) const
{
	if (!Asset->RetargetOps.IsValidIndex(InOpIndex))
	{
		return TArray<int32>{};
	}

	const FName InOpName = GetOpName(InOpIndex);

	TArray<int32> ChildrenOpIndices;
	for (int32 OtherOpIndex = 0; OtherOpIndex < Asset->RetargetOps.Num(); OtherOpIndex++)
	{
		const FIKRetargetOpBase* OtherOp = Asset->RetargetOps[OtherOpIndex].GetPtr<FIKRetargetOpBase>();
		if (OtherOp->GetParentOpName() == InOpName)
		{
			ChildrenOpIndices.Add(OtherOpIndex);
		}
	}
	
	return MoveTemp(ChildrenOpIndices);
}

bool UIKRetargeterController::GetCanOpHaveChildren(const int32 InOpIndex) const
{
	if (FIKRetargetOpBase* Op = GetRetargetOpByIndex(InOpIndex))
	{
		return Op->CanHaveChildOps();
	}
	return false;
}

int32 UIKRetargeterController::GetParentOpIndex(const int32 InOpIndex) const
{
	if (FIKRetargetOpBase* Op = GetRetargetOpByIndex(InOpIndex))
	{
		return GetIndexOfOpByName(Op->GetParentOpName());
	}
	
	return INDEX_NONE;	
}

FName UIKRetargeterController::GetUniqueOpName(const FName InNameToMakeUnique, int32 InOpIndexToIgnore) const
{
	// handle the "None" case by providing a default base name
	const FName BaseName = InNameToMakeUnique.IsNone() ? FName(TEXT("NewOp")) : InNameToMakeUnique;

	// check if any other op using name
	auto OpNameInUse = [this, InOpIndexToIgnore](const FName InOpNameToCheck)
	{
		for (int32 OpIndex=0; OpIndex<Asset->RetargetOps.Num(); ++OpIndex)
		{
			if (OpIndex == InOpIndexToIgnore)
			{
				continue;
			}
   
			const FIKRetargetOpBase* Op = Asset->RetargetOps[OpIndex].GetPtr<FIKRetargetOpBase>();
			if (Op && Op->GetName() == InOpNameToCheck)
			{
				return true;
			}
		}
		return false;
	};

	// check if the (potentially corrected) base name is available
	if (!OpNameInUse(BaseName))
	{
		return BaseName;
	}
    
	// increment the internal FName number suffix until we hit a unique one
	int32 Number = BaseName.GetNumber() + 1;
	while(OpNameInUse(FName(BaseName, Number)))
	{
		Number++;
	}

	return FName(BaseName, Number);
}

int32 UIKRetargeterController::GetIndexOfRetargetOp(FIKRetargetOpBase* RetargetOp) const
{
	if (!RetargetOp)
	{
		return INDEX_NONE;
	}
	
	for (int32 OpIndex=0; OpIndex<Asset->RetargetOps.Num(); ++OpIndex)
	{
		const FIKRetargetOpBase* Op = Asset->RetargetOps[OpIndex].GetPtr<FIKRetargetOpBase>();
		if (Op == RetargetOp)
		{
			return OpIndex;
		}
	}
	
	return INDEX_NONE;
}

void UIKRetargeterController::OnOpPropertyChanged(const FName& InOpName, const FPropertyChangedEvent& InPropertyChangedEvent) const
{
	const int32 OpIndex = GetIndexOfOpByName(InOpName);
	if (!ensure(OpIndex != INDEX_NONE))
	{
		return; // should not get callback for unknown op
	}

	// notify op of property edit
	FIKRetargetOpBase* Op = GetRetargetOpByIndex(OpIndex);
	Op->OnReinitPropertyEdited(&InPropertyChangedEvent);

	// notify children of parent property edit
	TArray<int32> ChildOps = GetChildOpIndices(OpIndex);
	for (int32 ChildOpIndex : ChildOps)
	{
		if (FIKRetargetOpBase* ChildOp = GetRetargetOpByIndex(ChildOpIndex))
		{
			ChildOp->OnParentReinitPropertyEdited(*Op, &InPropertyChangedEvent);
		}
	}

	// reinitialize
	RetargeterNeedsInitialized.Broadcast();
}

uint8* UIKRetargeterController::GetChainSettingsMemory(const FName& InOpName, const FName InChainName) const
{
	FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
	if (!Op)
	{
		return nullptr;
	}

	return Op->GetChainSettingsMemory(InChainName);
}

FInstancedStruct* UIKRetargeterController::GetRetargetOpStructAtIndex(int32 Index) const
{
	if (Asset->RetargetOps.IsValidIndex(Index))
	{
		return &Asset->RetargetOps[Index];
	}
	
	return nullptr;
}

int32 UIKRetargeterController::GetNumRetargetOps() const
{
	return Asset->RetargetOps.Num();
}

bool UIKRetargeterController::MoveRetargetOpInStack(int32 OpToMoveIndex, int32 TargetIndex) const
{
	// ensure target is within range
	TargetIndex = FMath::Clamp(TargetIndex, 0, Asset->RetargetOps.Num() - 1);
	
	if (!Asset->RetargetOps.IsValidIndex(OpToMoveIndex))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget Op not moved. Invalid source index, %d.", OpToMoveIndex);
		return false;
	}

	if (!Asset->RetargetOps.IsValidIndex(TargetIndex))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget Op not moved. Invalid target index, %d.", TargetIndex);
		return false;
	}

	if (OpToMoveIndex == TargetIndex)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget Op not moved. Source and target index cannot be the same.");
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("ReorderRetargetOps_Label", "Reorder Retarget Ops"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();

	// extract the element to move
	FInstancedStruct MovedOp = MoveTemp(Asset->RetargetOps[OpToMoveIndex]);
	// Remove the element without shrinking the array
	Asset->RetargetOps.RemoveAt(OpToMoveIndex, 1, EAllowShrinking::No);
	// Insert at the corrected TargetIndex
	Asset->RetargetOps.Insert(MoveTemp(MovedOp), TargetIndex);

	// enforce ordering constraints
	Asset->CleanOpStack();
	
	return true;
}

bool UIKRetargeterController::SetRetargetOpEnabled(int32 RetargetOpIndex, bool bIsEnabled) const
{
	if (!Asset->RetargetOps.IsValidIndex(RetargetOpIndex))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget op not found. Invalid index, %d.", RetargetOpIndex);
		return false;
	}
	
	FScopedTransaction Transaction(LOCTEXT("SetRetargetOpEnabled_Label", "Enable/Disable Op"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();
	GetRetargetOpByIndex(RetargetOpIndex)->SetEnabled(bIsEnabled);
	return true;
}

bool UIKRetargeterController::GetRetargetOpEnabled(int32 RetargetOpIndex) const
{
	if (!Asset->RetargetOps.IsValidIndex(RetargetOpIndex))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Invalid retarget op index, %d.", RetargetOpIndex);
		return false;
	}

	return GetRetargetOpByIndex(RetargetOpIndex)->IsEnabled();
}

UIKRetargetOpControllerBase* UIKRetargeterController::GetOpController(int32 OpIndex)
{
	if (FIKRetargetOpBase* OpAtIndex = GetRetargetOpByIndex(OpIndex))
	{
		return OpAtIndex->GetSettings()->GetController(this);
	}
	
	return nullptr;
}

void UIKRetargeterController::ResetChainSettingsInAllOps(const FName InTargetChainName) const
{
	FScopedTransaction Transaction(LOCTEXT("ResetChainSettings_Label", "Reset Settings for Chain"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();
	for (int32 OpIndex=0; OpIndex<Asset->RetargetOps.Num(); ++OpIndex)
	{
		FIKRetargetOpBase& Op = Asset->RetargetOps[OpIndex].GetMutable<FIKRetargetOpBase>();
		Op.ResetChainSettingsToDefault(InTargetChainName);
	}
}

const FRetargetChainMapping& UIKRetargeterController::GetChainMapping() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Asset->GetChainMapping();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UIKRetargeterController::CleanChainMaps(const FName InOpName) const
{
	const UIKRigDefinition* SourceIKRig = GetIKRig(ERetargetSourceOrTarget::Source);
	
	auto CleanChainMapInOp = [SourceIKRig](FIKRetargetOpBase* InOp)
	{
		if (!ensure(InOp))
		{
			return;
		}

		FRetargetChainMapping* ChainMapping = InOp->GetChainMapping();
		if (!ChainMapping)
		{
			return; // not all ops maintain their own chain mapping
		}

		// clean the mapping
		ChainMapping->ReinitializeWithIKRigs(SourceIKRig, InOp->GetCustomTargetIKRig());

		// force the op to regenerate chain settings if needed
		InOp->OnReinitPropertyEdited(nullptr);
	};

	// single op
	if (InOpName != NAME_None)
	{
		CleanChainMapInOp(GetRetargetOpByName(InOpName));
		return;
	}

	// all ops
	const int32 NumOps = GetNumRetargetOps();
	for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
	{
		CleanChainMapInOp(GetRetargetOpByIndex(OpIndex));
	}
}

bool UIKRetargeterController::AreChainSettingsAtDefault(const FName InTargetChainName, const FName InOpName) const
{
	FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
	if (!Op)
	{
		return false;
	}

	return Op->AreChainSettingsAtDefault(InTargetChainName);
}

void UIKRetargeterController::ResetChainSettingsToDefault(const FName InTargetChainName, const FName InOpName) const
{
	FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
	if (!Op)
	{
		return;
	}
	
	FScopeLock Lock(&ControllerLock);
	FScopedTransaction Transaction(LOCTEXT("ResetChainSettings", "Reset Chain Settings"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();
	
	Op->ResetChainSettingsToDefault(InTargetChainName);
}

const UIKRigDefinition* UIKRetargeterController::GetTargetIKRigForOp(const FName InOpName) const
{
	FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
	if (!Op)
	{
		return nullptr;
	}

	if (const UIKRigDefinition* TargetIKRig = Op->GetCustomTargetIKRig())
	{
		return TargetIKRig;
	}

	if (FIKRetargetOpBase* ParentOp = GetRetargetOpByName(Op->GetParentOpName()))
	{
		return ParentOp->GetCustomTargetIKRig();
	}

	return nullptr;
}

int32 UIKRetargeterController::AddRetargetOp(const UScriptStruct* InRetargetOpType, const FName InParentOpName) const
{
	if (!InRetargetOpType)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Could not add retarget operation. No type specified.");
		return INDEX_NONE;
	}

	if (!InRetargetOpType->IsChildOf(FIKRetargetOpBase::StaticStruct()))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Could not add retarget operations. Invalid op-type specified. Must be child of FIKRetargetOpBase.");
		return INDEX_NONE;
	}
	
	// check if the op is a singleton and the stack already contains an op of that type
	FInstancedStruct NewOpStruct = FInstancedStruct(InRetargetOpType);
	if (NewOpStruct.GetPtr<FIKRetargetOpBase>()->IsSingleton())
	{
		for (int32 OpIndex=0; OpIndex<Asset->RetargetOps.Num(); ++OpIndex)
		{
			const FInstancedStruct& Op = Asset->RetargetOps[OpIndex];
			if (Op.GetScriptStruct() == InRetargetOpType)
			{
				UE_LOGF(LogIKRigEditor, Warning, "Op not added. It is a singleton and the stack already contains an op of that type.");
				return INDEX_NONE;
			}
		}
	}

	FScopedTransaction Transaction(LOCTEXT("AddRetargetOp_Label", "Add Retarget Op"));
	FScopedReinitializeIKRetargeter Reinitialize(this, ERetargetRefreshMode::ProcessorAndOpStack);
	Asset->Modify();

	// add the op
	int32 NewOpIndex = Asset->RetargetOps.Emplace(MoveTemp(NewOpStruct));

	// give a unique default name
	FIKRetargetOpBase* NewOp = GetRetargetOpByIndex(NewOpIndex);
	const FName NewOpName = GetUniqueOpName(NewOp->GetDefaultName(), NewOpIndex);
	NewOp->SetName(NewOpName);

	// assign to parent
	NewOp->SetParentOpName(InParentOpName);

	// enforce correct execution order
	Asset->CleanOpStack();

	// in the unlikely event that cleaning the op stack reshuffled the indices, get the index again
	NewOpIndex = GetIndexOfOpByName(NewOpName);
	
	// run the initial setup
	FIKRetargetOpBase* ParentOp = GetRetargetOpByIndex(GetParentOpIndex(NewOpIndex));
	NewOp->OnAddedToStack(GetAsset(), ParentOp);
	
	return NewOpIndex;
}

bool UIKRetargeterController::GetAskedToFixRootHeightForMesh(USkeletalMesh* Mesh) const
{
	return GetAsset()->MeshesAskedToFixRootHeightFor.Contains(Mesh);
}

void UIKRetargeterController::SetAskedToFixRootHeightForMesh(USkeletalMesh* Mesh, bool InAsked) const
{
	FScopeLock Lock(&ControllerLock);
	if (InAsked)
	{
		GetAsset()->MeshesAskedToFixRootHeightFor.Add(Mesh);
	}
	else
	{
		GetAsset()->MeshesAskedToFixRootHeightFor.Remove(Mesh);
	}
}

FName UIKRetargeterController::GetPelvisBone(const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	const UIKRigDefinition* IKRig = GetIKRig(SourceOrTarget);
	return IKRig ? IKRig->GetPelvis() : FName("None");
}

void UIKRetargeterController::CleanPoseList(const ERetargetSourceOrTarget SourceOrTarget) const
{
	USkeletalMesh* SkeletalMesh = GetPreviewMesh(SourceOrTarget);
	if (!SkeletalMesh)
	{
		return;
	}

	const TWeakPtr<const FIKRigSkeleton> IKRigSkeleton = GetCachedIKRigSkeleton(SkeletalMesh);
	if (!IKRigSkeleton.IsValid())
	{
		return;
	}
	
	// remove all bone offsets that are no longer part of the skeleton
	TMap<FName, FIKRetargetPose>& RetargetPoses = GetRetargetPoses(SourceOrTarget);
	for (TTuple<FName, FIKRetargetPose>& Pose : RetargetPoses)
	{
		// find bone offsets no longer in target skeleton
		TArray<FName> BonesToRemove;
		for (TTuple<FName, FQuat>& BoneOffset : Pose.Value.BoneRotationOffsets)
		{
			if (!IKRigSkeleton.Pin()->BoneNames.Contains(BoneOffset.Key))
			{
				BonesToRemove.Add(BoneOffset.Key);
			}
		}
		
		// remove bone offsets
		for (const FName& BoneToRemove : BonesToRemove)
		{
			Pose.Value.BoneRotationOffsets.Remove(BoneToRemove);
		}

		// sort the pose offset from leaf to root
		Pose.Value.SortHierarchically(*IKRigSkeleton.Pin().Get());
	}
}

UIKRigStructViewer* UIKRetargeterController::GetStructViewer(ERetargetStructViewerMode InMode)
{
	TObjectPtr<UIKRigStructViewer> StructViewer = StructViewers[InMode];
	StructViewer->Reset();
	return StructViewer;
}

void UIKRetargeterController::AutoMapChains(const EAutoMapChainType AutoMapType, const bool bForceRemap, const FName InOpName) const
{
	FScopeLock Lock(&ControllerLock);
	FScopedTransaction Transaction(LOCTEXT("AutoMapRetargetChains", "Auto-Map Retarget Chains"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();

	auto AutoMapChainsInOp = [AutoMapType, bForceRemap](FIKRetargetOpBase* InRetargetOp)
	{
		if (InRetargetOp)
		{
			if (FRetargetChainMapping* ChainMapping = InRetargetOp->GetChainMapping())
			{
				ChainMapping->AutoMapChains(AutoMapType, bForceRemap);
			}
		}
	};

	// single op
	if (InOpName != NAME_None)
	{
		AutoMapChainsInOp(GetRetargetOpByName(InOpName));
		return;
	}

	// all ops
	const int32 NumOps = GetNumRetargetOps();
	for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
	{
		AutoMapChainsInOp(GetRetargetOpByIndex(OpIndex));
	}
}

void UIKRetargeterController::HandleRetargetChainAdded(UIKRigDefinition* IKRig) const
{
	const bool bIsTargetRig = IKRig == Asset->GetIKRig(ERetargetSourceOrTarget::Target);
	if (!bIsTargetRig)
	{
		// if a source chain is added, it will simply be available as a new option, no need to reinitialize until it's used
		return;
	}
	
	FScopedReinitializeIKRetargeter Reinitialize(this);
	
	// clean the chain map (this will add the new chain automatically)
	CleanChainMaps();
}

void UIKRetargeterController::HandleRetargetChainRenamed(UIKRigDefinition* InIKRig, FName OldChainName, FName NewChainName) const
{
	FScopedTransaction Transaction(LOCTEXT("RetargetChainRenamed_Label", "Retarget Chain Renamed"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();

	int32 NumOps = GetNumRetargetOps();
	for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
	{
		FIKRetargetOpBase* Op = GetRetargetOpByIndex(OpIndex);
		FRetargetChainMapping* ChainMapping = Op->GetChainMapping();
		if (!ChainMapping)
		{
			continue;
		}
		
		const bool bIsSourceRig = InIKRig == ChainMapping->GetIKRig(ERetargetSourceOrTarget::Source);
		const bool bIsTargetRig = InIKRig == GetTargetIKRigForOp(Op->GetName());
		if (!(bIsSourceRig || bIsTargetRig))
		{
			continue;
		}

		// maintain mappings to old name with new name
		for (FRetargetChainPair& ChainPair : ChainMapping->GetChainPairs())
		{
			FName& ChainNameToUpdate = bIsSourceRig ? ChainPair.SourceChainName : ChainPair.TargetChainName;
			if (ChainNameToUpdate == OldChainName)
			{
				ChainNameToUpdate = NewChainName;
				break;
			}
		}

		// allow op to retain the old chain settings under the new name
		// NOTE: this is only called for ops that use the target IK rig
		if (bIsTargetRig)
		{
			Op->OnTargetChainRenamed(OldChainName, NewChainName);	
		}
	}
}

void UIKRetargeterController::HandleRetargetChainRemoved(UIKRigDefinition* IKRig, const FName& InChainRemoved) const
{
	FScopedTransaction Transaction(LOCTEXT("RetargetChainRemoved_Label", "Retarget Chain Removed"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();

	int32 NumOps = GetNumRetargetOps();
	for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
	{
		FIKRetargetOpBase* Op = GetRetargetOpByIndex(OpIndex);
		FRetargetChainMapping* ChainMapping = Op->GetChainMapping();
		if (!ChainMapping)
		{
			continue;
		}
		
		const bool bIsSourceRig = IKRig == ChainMapping->GetIKRig(ERetargetSourceOrTarget::Source);
		const bool bIsTargetRig = IKRig == Op->GetCustomTargetIKRig();
		if (!(bIsSourceRig || bIsTargetRig))
		{
			continue;
		}

		TArray<FRetargetChainPair>& ChainPairs = ChainMapping->GetChainPairs();
	
		// set source chain name to NONE if it has been deleted 
		if (bIsSourceRig)
		{
			for (FRetargetChainPair& ChainPair : ChainPairs)
			{
				if (ChainPair.SourceChainName == InChainRemoved)
				{
					ChainPair.SourceChainName = NAME_None;
					return;
				}
			}
			return;
		}
	
		// remove target mapping if the target chain has been removed
		const int32 ChainIndex = ChainPairs.IndexOfByPredicate([&InChainRemoved](const FRetargetChainPair& ChainPair)
		{
			return ChainPair.TargetChainName == InChainRemoved;
		});
	
		if (ChainIndex != INDEX_NONE)
		{
			ChainPairs.RemoveAt(ChainIndex);
		}

		// force regeneration of chain settings
		Op->OnReinitPropertyEdited(nullptr);
	}
}

bool UIKRetargeterController::SetSourceChain(FName SourceChainName, FName TargetChainName, const FName InOpName) const
{
	FScopeLock Lock(&ControllerLock);
	FScopedTransaction Transaction(LOCTEXT("SetRetargetChainSource", "Set Retarget Chain Source"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();

	bool bModifiedChainMap = false;
	const int32 NumOps = GetNumRetargetOps();
	for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
	{
		FIKRetargetOpBase* Op = GetRetargetOpByIndex(OpIndex);
		if (InOpName != NAME_None && Op->GetName() != InOpName)
		{
			continue;
		}
		
		FRetargetChainMapping* ChainMapping = Op->GetChainMapping();
		if (!ChainMapping)
		{
			continue;
		}
		
		if (!ChainMapping->HasChain(TargetChainName, ERetargetSourceOrTarget::Target))
		{
			continue;
		}
		
		ChainMapping->SetChainMapping(TargetChainName, SourceChainName);
		bModifiedChainMap = true;
	}
	
	return bModifiedChainMap;
}

FName UIKRetargeterController::GetSourceChain(const FName& TargetChainName, const FName InOpName) const
{
	const int32 NumOps = GetNumRetargetOps();
	for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
	{
		FIKRetargetOpBase* Op = GetRetargetOpByIndex(OpIndex);
		if (InOpName != NAME_None && Op->GetName() != InOpName)
		{
			continue;
		}
		
		FRetargetChainMapping* ChainMapping = Op->GetChainMapping();
		if (!ChainMapping)
		{
			continue;
		}
		
		if (!ChainMapping->HasChain(TargetChainName, ERetargetSourceOrTarget::Target))
		{
			continue;
		}
		
		return ChainMapping->GetChainMappedTo(TargetChainName, ERetargetSourceOrTarget::Target);
	}

	return NAME_None;
}

const FRetargetChainMapping* UIKRetargeterController::GetChainMapping(const FName InOpName) const
{
	const int32 NumOps = GetNumRetargetOps();
	for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
	{
		FIKRetargetOpBase* Op = GetRetargetOpByIndex(OpIndex);
		if (InOpName != NAME_None && Op->GetName() != InOpName)
		{
			continue;
		}
		
		return Op->GetChainMapping();
	}
	
	return nullptr;
}

bool UIKRetargeterController::IsChainGoalConnectedToASolver(const FName& GoalName) const
{
	TArray<UIKRigDefinition*> AllTargetIKRigs = GetAllTargetIKRigs();
	if (AllTargetIKRigs.IsEmpty())
	{
		return false;
	}

	for (const UIKRigDefinition* TargetIKRig : AllTargetIKRigs)
	{
		const UIKRigController* RigController = UIKRigController::GetController(TargetIKRig);
		if (!RigController)
		{
			continue;
		}
		
		if (RigController->IsGoalConnectedToAnySolver(GoalName))
		{
			return true;
		}
	}
	
	return false; 
}

FName UIKRetargeterController::CreateRetargetPose(const FName& NewPoseName, const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	FScopedTransaction Transaction(LOCTEXT("CreateRetargetPose", "Create Retarget Pose"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();

	// create a new pose with a unique name 
	const FName UniqueNewPoseName = MakePoseNameUnique(NewPoseName.ToString(), SourceOrTarget);
	GetRetargetPoses(SourceOrTarget).Add(UniqueNewPoseName);

	// set new pose as the current pose
	FName& CurrentRetargetPoseName = SourceOrTarget == ERetargetSourceOrTarget::Source ? Asset->CurrentSourceRetargetPose : Asset->CurrentTargetRetargetPose;
	CurrentRetargetPoseName = UniqueNewPoseName;

	return UniqueNewPoseName;
}

bool UIKRetargeterController::RemoveRetargetPose(const FName& PoseToRemove, const ERetargetSourceOrTarget SourceOrTarget) const
{
	if (PoseToRemove == Asset->GetDefaultPoseName())
	{
		return false; // cannot remove default pose
	}

	TMap<FName, FIKRetargetPose>& Poses = GetRetargetPoses(SourceOrTarget);
	if (!Poses.Contains(PoseToRemove))
	{
		return false; // cannot remove pose that doesn't exist
	}

	FScopeLock Lock(&ControllerLock);
	FScopedTransaction Transaction(LOCTEXT("RemoveRetargetPose", "Remove Retarget Pose"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();

	Poses.Remove(PoseToRemove);

	// did we remove the currently used pose?
	if (GetCurrentRetargetPoseName(SourceOrTarget) == PoseToRemove)
	{
		SetCurrentRetargetPose(UIKRetargeter::GetDefaultPoseName(), SourceOrTarget);
	}
	
	return true;
}

FName UIKRetargeterController::DuplicateRetargetPose( const FName PoseToDuplicate, const FName NewPoseName, const ERetargetSourceOrTarget SourceOrTarget) const
{
	TMap<FName, FIKRetargetPose>& Poses = GetRetargetPoses(SourceOrTarget);
	if (!Poses.Contains(PoseToDuplicate))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to duplicate pose that does not exist, %ls.", *PoseToDuplicate.ToString());
		return NAME_None; // cannot duplicate pose that doesn't exist
	}

	FScopeLock Lock(&ControllerLock);
	FScopedTransaction Transaction(LOCTEXT("DuplicateRetargetPose", "Duplicate Retarget Pose"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();

	// create a new pose with a unique name
	const FName UniqueNewPoseName = MakePoseNameUnique(NewPoseName.ToString(), SourceOrTarget);
	FIKRetargetPose& NewPose = Poses.Add(UniqueNewPoseName);
	// duplicate the pose data
	NewPose.RootTranslationOffset = Poses[PoseToDuplicate].RootTranslationOffset;
	NewPose.BoneRotationOffsets = Poses[PoseToDuplicate].BoneRotationOffsets;

	// set duplicate to be the current pose
	FName& CurrentRetargetPoseName = SourceOrTarget == ERetargetSourceOrTarget::Source ? Asset->CurrentSourceRetargetPose : Asset->CurrentTargetRetargetPose;
	CurrentRetargetPoseName = UniqueNewPoseName;
	return UniqueNewPoseName;
}

bool UIKRetargeterController::RenameRetargetPose(const FName OldPoseName, const FName NewPoseName, const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	
	// does the old pose exist?
	if (!GetRetargetPoses(SourceOrTarget).Contains(OldPoseName))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to rename pose that does not exist, %ls.", *OldPoseName.ToString());
		return false;
	}

	// do not allow renaming the default pose (this is disallowed from the UI, but must be done here as well for API usage)
	if (OldPoseName == UIKRetargeter::GetDefaultPoseName())
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to rename the default pose. This is not allowed.");
    	return false;
	}

	// check if we're renaming the current pose
	const bool bWasCurrentPose = GetCurrentRetargetPoseName(SourceOrTarget) == OldPoseName;
	
	FScopedTransaction Transaction(LOCTEXT("RenameRetargetPose", "Rename Retarget Pose"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();

	// make sure new name is unique
	const FName UniqueNewPoseName = MakePoseNameUnique(NewPoseName.ToString(), SourceOrTarget);

	// replace key in the map
	TMap<FName, FIKRetargetPose>& Poses = GetRetargetPoses(SourceOrTarget);
	const FIKRetargetPose OldPoseData = Poses[OldPoseName];
	Poses.Remove(OldPoseName);
	Poses.Shrink();
	Poses.Add(UniqueNewPoseName, OldPoseData);

	// make this the current retarget pose, iff the old one was
	if (bWasCurrentPose)
	{
		SetCurrentRetargetPose(UniqueNewPoseName, SourceOrTarget);
	}
	return true;
}

void UIKRetargeterController::ResetRetargetPose(
	const FName& PoseToReset,
	const TArray<FName>& BonesToReset,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	
	TMap<FName, FIKRetargetPose>& Poses = GetRetargetPoses(SourceOrTarget);
	if (!Poses.Contains(PoseToReset))
	{
		return; // cannot reset pose that doesn't exist
	}
	
	FIKRetargetPose& PoseToEdit = Poses[PoseToReset];

	FScopedReinitializeIKRetargeter Reinitialize(this);
	
	if (BonesToReset.IsEmpty())
	{
		FScopedTransaction Transaction(LOCTEXT("ResetRetargetPose", "Reset Retarget Pose"));
		Asset->Modify();

		PoseToEdit.BoneRotationOffsets.Reset();
		PoseToEdit.RootTranslationOffset = FVector::ZeroVector;
	}
	else
	{
		FScopedTransaction Transaction(LOCTEXT("ResetRetargetBonePose", "Reset Bone Pose"));
		Asset->Modify();
		
		const FName RootBoneName = GetPelvisBone(SourceOrTarget);
		for (const FName& BoneToReset : BonesToReset)
		{
			if (PoseToEdit.BoneRotationOffsets.Contains(BoneToReset))
			{
				PoseToEdit.BoneRotationOffsets.Remove(BoneToReset);
			}

			if (BoneToReset == RootBoneName)
			{
				PoseToEdit.RootTranslationOffset = FVector::ZeroVector;	
			}
		}
	}
}

FName UIKRetargeterController::GetCurrentRetargetPoseName(const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? GetAsset()->CurrentSourceRetargetPose : GetAsset()->CurrentTargetRetargetPose;
}

bool UIKRetargeterController::SetCurrentRetargetPose(FName NewCurrentPose, const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	
	const TMap<FName, FIKRetargetPose>& Poses = GetRetargetPoses(SourceOrTarget);
	if (!Poses.Contains(NewCurrentPose))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to set current pose to a pose that does not exist, %ls.", *NewCurrentPose.ToString());
		return false;
	}
	
	FScopedTransaction Transaction(LOCTEXT("SetCurrentPose", "Set Current Pose"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	Asset->Modify();
	FName& CurrentPose = SourceOrTarget == ERetargetSourceOrTarget::Source ? Asset->CurrentSourceRetargetPose : Asset->CurrentTargetRetargetPose;
	CurrentPose = NewCurrentPose;
	return true;
}

TMap<FName, FIKRetargetPose>& UIKRetargeterController::GetRetargetPoses(const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	return SourceOrTarget == ERetargetSourceOrTarget::Source ? GetAsset()->SourceRetargetPoses : GetAsset()->TargetRetargetPoses;
}

FIKRetargetPose& UIKRetargeterController::GetCurrentRetargetPose(const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	return GetRetargetPoses(SourceOrTarget)[GetCurrentRetargetPoseName(SourceOrTarget)];
}

void UIKRetargeterController::SetRotationOffsetForRetargetPoseBone(
	const FName& BoneName,
	const FQuat& RotationOffset,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);

	// add bone offset to pose
	FIKRetargetPose& Pose = GetCurrentRetargetPose(SourceOrTarget);
	Pose.SetDeltaRotationForBone(BoneName, RotationOffset);

	// sort the pose hierarchically based on skeleton
	if (USkeletalMesh* SkeletalMesh = GetAsset()->GetPreviewMesh(SourceOrTarget))
	{
		TWeakPtr<const FIKRigSkeleton> IKRigSkeleton = GetCachedIKRigSkeleton(SkeletalMesh);
		if (IKRigSkeleton.IsValid())
		{
			Pose.SortHierarchically(*IKRigSkeleton.Pin().Get());
		}
	}
}

FQuat UIKRetargeterController::GetRotationOffsetForRetargetPoseBone(
	const FName& BoneName,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	
	TMap<FName, FQuat>& BoneOffsets = GetCurrentRetargetPose(SourceOrTarget).BoneRotationOffsets;
	if (!BoneOffsets.Contains(BoneName))
	{
		return FQuat::Identity;
	}
	
	return BoneOffsets[BoneName];
}

void UIKRetargeterController::SetRootOffsetInRetargetPose(
	const FVector& TranslationOffset,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	GetCurrentRetargetPose(SourceOrTarget).AddToRootTranslationDelta(TranslationOffset);
}

FVector UIKRetargeterController::GetRootOffsetInRetargetPose(
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	FScopeLock Lock(&ControllerLock);
	return GetCurrentRetargetPose(SourceOrTarget).GetRootTranslationDelta();
}

void UIKRetargeterController::AutoAlignAllBones(ERetargetSourceOrTarget SourceOrTarget, const ERetargetAutoAlignMethod Method) const
{
	// undo transaction
	constexpr bool bShouldTransact = true;
	FScopedTransaction Transaction(LOCTEXT("AutoAlignAllBones", "Auto Align All Bones"), bShouldTransact);
	Asset->Modify();
	
	FScopedReinitializeIKRetargeter Reinitialize(this);

	// first reset the entire retarget pose
	ResetRetargetPose(GetCurrentRetargetPoseName(SourceOrTarget), TArray<FName>(), SourceOrTarget);

	// suppress warnings about bones that cannot be aligned when aligning ALL bones
	constexpr bool bSuppressWarnings = true;
	AutoPoseGenerator.Get()->AlignBones(
		TArray<FName>(), // empty list means "all bones"
		Method,
		SourceOrTarget,
		bSuppressWarnings);
}

void UIKRetargeterController::AutoAlignBones(
	const TArray<FName>& BonesToAlign,
	const ERetargetAutoAlignMethod Method,
	ERetargetSourceOrTarget SourceOrTarget) const
{
	// undo transaction
	constexpr bool bShouldTransact = true;
	FScopedTransaction Transaction(LOCTEXT("AutoAlignBones", "Auto Align Bones"), bShouldTransact);
	Asset->Modify();

	FScopedReinitializeIKRetargeter Reinitialize(this);
	
	// allow warnings about bones that cannot be aligned when bones are explicitly specified by user
	constexpr bool bSuppressWarnings = false;
	AutoPoseGenerator.Get()->AlignBones(
		BonesToAlign,
		Method,
		SourceOrTarget,
		bSuppressWarnings);
}

void UIKRetargeterController::SnapBoneToGround(FName ReferenceBone, ERetargetSourceOrTarget SourceOrTarget)
{
	// undo transaction
	constexpr bool bShouldTransact = true;
	FScopedTransaction Transaction(LOCTEXT("SnapBoneToGround", "Snap Bone to Ground"), bShouldTransact);
	Asset->Modify();

	AutoPoseGenerator.Get()->SnapToGround(ReferenceBone, SourceOrTarget);
}

FName UIKRetargeterController::MakePoseNameUnique(const FString& PoseName, const ERetargetSourceOrTarget SourceOrTarget) const
{
	FString UniqueName = PoseName;
	
	if (UniqueName.IsEmpty())
	{
		UniqueName = Asset->GetDefaultPoseName().ToString();
	}
	
	int32 Suffix = 1;
	const TMap<FName, FIKRetargetPose>& Poses = GetRetargetPoses(SourceOrTarget);
	while (Poses.Contains(FName(UniqueName)))
	{
		UniqueName = PoseName + "_" + FString::FromInt(Suffix);
		++Suffix;
	}
	return FName(UniqueName);
}

TWeakPtr<const FIKRigSkeleton> UIKRetargeterController::GetCachedIKRigSkeleton(const USkeletalMesh* InSkeletalMesh) const
{
	if (!InSkeletalMesh)
	{
		return nullptr;
	}

	if (TSharedPtr<FIKRigSkeleton>* IKRigSkeleton = CachedSkeletons.Find(InSkeletalMesh))
	{
		return IKRigSkeleton->ToWeakPtr();
	}

	TSharedPtr<FIKRigSkeleton>& NewIKRigSkeleton = CachedSkeletons.Add(InSkeletalMesh);
	NewIKRigSkeleton = MakeShared<FIKRigSkeleton>();
	NewIKRigSkeleton->SetInputSkeleton(InSkeletalMesh, TArray<FName>() /*ExcludedBones*/);
	return NewIKRigSkeleton.ToWeakPtr();
}

FName UIKRetargeterController::AddNewRetargetOverrideSet(FName InOverrideSetName)
{
    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("AddNewRetargetOverrideSet", "Add New Retarget Override Set"), bShouldTransact);
    Asset->Modify();

    const FName UniqueName = GetUniqueOverrideSetName(InOverrideSetName);
    FRetargetOverrideSet& NewSet = Asset->OverrideSets.Add(UniqueName);
    NewSet.Name = UniqueName;

	// Place at the bottom of root-level sets by assigning the next available display order.
	int32 MaxDisplayOrder = -1;
	for (const TTuple<FName, FRetargetOverrideSet>& Entry : Asset->GetOverrideSets())
	{
		if (Entry.Value.ParentName == NAME_None && Entry.Value.DisplayOrder > MaxDisplayOrder)
		{
			MaxDisplayOrder = Entry.Value.DisplayOrder;
		}
	}
	NewSet.DisplayOrder = MaxDisplayOrder + 1;

	PostEditOverrideSets();
	
    return UniqueName;
}

FName UIKRetargeterController::DuplicateRetargetOverrideSet(FName InOverrideSetName)
{
	const FRetargetOverrideSet* SourceSet = FindOverrideSet(InOverrideSetName);
	if (!SourceSet)
	{
		return NAME_None;
	}

	// copy source data before Add() which may rehash the TMap and invalidate SourceSet
	const FName SourceParentName = SourceSet->ParentName;
	const bool bSourceActiveByDefault = SourceSet->bActiveByDefault;
	const TArray<FRetargetOpOverrides> SourceOpOverrides = SourceSet->OpOverrides;

	constexpr bool bShouldTransact = true;
	FScopedTransaction Transaction(LOCTEXT("DuplicateRetargetOverrideSet", "Duplicate Retarget Override Set"), bShouldTransact);
	Asset->Modify();

	const FName UniqueName = GetUniqueOverrideSetName(InOverrideSetName);
	FRetargetOverrideSet& NewSet = Asset->OverrideSets.Add(UniqueName);
	NewSet.Name = UniqueName;
	NewSet.ParentName = SourceParentName;
	NewSet.bActiveByDefault = bSourceActiveByDefault;
	NewSet.OpOverrides = SourceOpOverrides;

	// Place the duplicate at the bottom of its sibling group.
	int32 MaxDisplayOrder = -1;
	for (const TTuple<FName, FRetargetOverrideSet>& Entry : Asset->GetOverrideSets())
	{
		if (Entry.Value.ParentName == SourceParentName && Entry.Value.DisplayOrder > MaxDisplayOrder)
		{
			MaxDisplayOrder = Entry.Value.DisplayOrder;
		}
	}
	NewSet.DisplayOrder = MaxDisplayOrder + 1;

	PostEditOverrideSets();

	return UniqueName;
}

void UIKRetargeterController::ReorderOverrideSet(FName InSetToMove, FName InTargetSet, bool bInsertAfter)
{
	if (InSetToMove == InTargetSet)
	{
		return;
	}

	FRetargetOverrideSet* SetToMove = FindOverrideSet(InSetToMove);
	FRetargetOverrideSet* TargetSet = FindOverrideSet(InTargetSet);
	if (!SetToMove || !TargetSet)
	{
		return;
	}

	constexpr bool bShouldTransact = true;
	FScopedTransaction Transaction(LOCTEXT("ReorderOverrideSet", "Reorder Override Set"), bShouldTransact);
	Asset->Modify();

	// make the moved set a sibling of the target (same parent)
	const FName TargetParent = TargetSet->ParentName;
	SetToMove->ParentName = TargetParent;

	// collect all siblings (same parent) sorted by current DisplayOrder, excluding the moved set
	TArray<FName> SiblingOrder;
	for (const TTuple<FName, FRetargetOverrideSet>& Entry : Asset->GetOverrideSets())
	{
		if (Entry.Value.ParentName == TargetParent && Entry.Key != InSetToMove)
		{
			SiblingOrder.Add(Entry.Key);
		}
	}
	SiblingOrder.Sort([this](const FName& A, const FName& B)
	{
		const FRetargetOverrideSet* SetA = FindOverrideSet(A);
		const FRetargetOverrideSet* SetB = FindOverrideSet(B);
		return SetA && SetB && SetA->DisplayOrder < SetB->DisplayOrder;
	});

	// find the target's position in the sorted list and insert the moved set before or after
	const int32 TargetIdx = SiblingOrder.IndexOfByKey(InTargetSet);
	const int32 InsertIdx = bInsertAfter ? TargetIdx + 1 : TargetIdx;
	SiblingOrder.Insert(InSetToMove, FMath::Clamp(InsertIdx, 0, SiblingOrder.Num()));

	// reassign sequential display orders
	for (int32 Index = 0; Index < SiblingOrder.Num(); ++Index)
	{
		if (FRetargetOverrideSet* Set = FindOverrideSet(SiblingOrder[Index]))
		{
			Set->DisplayOrder = Index;
		}
	}

	PostEditOverrideSets();
}

bool UIKRetargeterController::RemoveRetargetOverrideSet(FName InOverrideSetName)
{
    if (!Asset->OverrideSets.Contains(InOverrideSetName))
    {
       return false;
    }
    
    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("RemoveRetargetOverrideSet", "Remove Retarget Override Set"), bShouldTransact);
    Asset->Modify();
    
    Asset->OverrideSets.Remove(InOverrideSetName);

	PostEditOverrideSets();
	
    return true;
}

bool UIKRetargeterController::RenameRetargetOverrideSet(FName InOverrideSetName, FName InNewName)
{
    FString CleanString = InNewName.ToString().TrimStartAndEnd();
    CleanString = SlugStringForValidName(CleanString);
    InNewName = FName(CleanString);
    
    if (InOverrideSetName == InNewName || InNewName == NAME_None)
    {
       return false;
    }
    
    FRetargetOverrideSet* SetToRename = Asset->OverrideSets.Find(InOverrideSetName);
    if (!SetToRename || Asset->OverrideSets.Contains(InNewName))
    {
       return false;
    }

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("RenameRetargetOverrideSet", "Rename Retarget Override Set"), bShouldTransact);
    Asset->Modify();

    FRetargetOverrideSet TempValue = MoveTemp(*SetToRename);
    Asset->OverrideSets.Remove(InOverrideSetName);
    
    const FName NewName = GetUniqueOverrideSetName(InNewName);
    TempValue.Name = NewName;
    Asset->OverrideSets.Add(NewName, MoveTemp(TempValue));

    // update parent/child references
	for (TTuple<FName, FRetargetOverrideSet>& It : Asset->OverrideSets)
	{
		if (It.Value.ParentName == InOverrideSetName)
		{
			It.Value.ParentName = NewName;
		}
	}

	PostEditOverrideSets();
    return true;
}

bool UIKRetargeterController::SetParentOverrideSet(FName InOverrideSetName, FName InParentName)
{
    if (InOverrideSetName == InParentName)
    {
       return false;
    }

    FRetargetOverrideSet* SetToReparent = Asset->OverrideSets.Find(InOverrideSetName);
    if (!SetToReparent)
    {
       return false;
    }

    if (InParentName != NAME_None && !Asset->OverrideSets.Contains(InParentName))
    {
       return false;
    }  

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("ReparentRetargetOverrideSet", "Reparent Retarget Override Set"), bShouldTransact);
    Asset->Modify();

    // prevent circular dependency by moving children of the current set to the current set's parent
    const TArray<FName> AllChildren = GetAllChildrenOverrideSets(InOverrideSetName);
    if (AllChildren.Contains(InParentName))
    {
       const TArray<FName> ChildrenToMove = GetDirectChildrenOverrideSets(InOverrideSetName);
       const FName GrandParent = GetParentOverrideSet(InOverrideSetName);
       for (const FName& ChildName : ChildrenToMove)
       {
          Asset->OverrideSets[ChildName].ParentName = GrandParent;
       }
    }

    SetToReparent->ParentName = InParentName;
	PostEditOverrideSets();
    
    return true;
}

FName UIKRetargeterController::GetParentOverrideSet(FName InOverrideSetName)
{
    const FRetargetOverrideSet* OverrideSet = Asset->OverrideSets.Find(InOverrideSetName);
    return OverrideSet ? OverrideSet->ParentName : NAME_None;
}

FName UIKRetargeterController::GetUniqueOverrideSetName(const FName InNameToMakeUnique) const
{
    auto NameInUse = [this](const FName& Name) { return Asset->OverrideSets.Contains(Name); };

    if (!NameInUse(InNameToMakeUnique))
    {
       return InNameToMakeUnique;
    }
    
    int32 Number = InNameToMakeUnique.GetNumber() + 1;
    while(NameInUse(FName(InNameToMakeUnique, Number)))
    {
       Number++;
    }

    return FName(InNameToMakeUnique, Number);
}

const TMap<FName, FRetargetOverrideSet>& UIKRetargeterController::GetAllOverrideSets() const
{
    return Asset->OverrideSets;
}

FRetargetOverrideSet* UIKRetargeterController::FindOverrideSet(const FName InOverrideSetName) const
{
    return Asset->OverrideSets.Find(InOverrideSetName);
}

const TArray<FName> UIKRetargeterController::GetAllChildrenOverrideSets(const FName InOverrideSetName) const
{
    TArray<FName> AllChildren;
	
    TArray<FName> Stack;
    Stack.Push(InOverrideSetName);
    while (!Stack.IsEmpty())
    {
       FName CurrentParent = Stack.Pop(EAllowShrinking::No);
       for (const TTuple<FName, FRetargetOverrideSet>& OverrideSetPair : Asset->OverrideSets)
       {
          if (OverrideSetPair.Value.ParentName == CurrentParent)
          {
             AllChildren.Add(OverrideSetPair.Key);
             Stack.Push(OverrideSetPair.Key);
          }
       }
    }

    return AllChildren;
}

const TArray<FName> UIKRetargeterController::GetDirectChildrenOverrideSets(const FName InOverrideSetName) const
{
    TArray<FName> Children;
    for (const TTuple<FName, FRetargetOverrideSet>& OverrideSetPair : Asset->OverrideSets)
    {
       if (OverrideSetPair.Value.ParentName == InOverrideSetName)
       {
          Children.Add(OverrideSetPair.Key);
       }
    }
    return Children;
}

FRetargetOpOverrides* UIKRetargeterController::FindOpOverrides(const FName InOverrideSetName, const FName InOpName) const
{
    FRetargetOverrideSet* OverrideSet = FindOverrideSet(InOverrideSetName);
    if (!OverrideSet)
    {
	    return nullptr;
    }

    for (FRetargetOpOverrides& OpOverrides : OverrideSet->OpOverrides)
    {
       if (OpOverrides.OpName == InOpName)
       {
          return &OpOverrides;
       }
    }
    return nullptr;
}

FRetargetOpPropertyOverride* UIKRetargeterController::FindPropertyOverride(
	const FName InOverrideSetName,
	const FName InOpName,
	const FString& InPropertyPath) const
{
	FRetargetOpOverrides* OpOverrides = FindOpOverrides(InOverrideSetName, InOpName);
	if (!OpOverrides)
	{
		return nullptr;
	}

	return OpOverrides->FindPropertyOverride(InPropertyPath);
}

FRetargetOpOverrides* UIKRetargeterController::AddOpOverrides(const FName InOverrideSetName, const FName InOpName) const
{
    FRetargetOverrideSet* OverrideSet = FindOverrideSet(InOverrideSetName);
    if (!OverrideSet)
    {
	    return nullptr;
    }

    if (FRetargetOpOverrides* Existing = FindOpOverrides(InOverrideSetName, InOpName))
    {
        return Existing;
    }

    FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
    if (!Op)
    {
	    return nullptr;
    }

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("AddOpOverrides", "Added Op Overrides to Set"), bShouldTransact);
    Asset->Modify();

    FRetargetOpOverrides& NewOpOverrides = OverrideSet->OpOverrides.AddDefaulted_GetRef();
    NewOpOverrides.OpName = InOpName;
    NewOpOverrides.ScriptStruct = const_cast<UScriptStruct*>(Op->GetSettingsType());

	PostEditOverrideSets();
    return &NewOpOverrides;
}

bool UIKRetargeterController::RemoveOpOverrides(const FName InOverrideSetName, const FName InOpName) const
{
    FRetargetOverrideSet* OverrideSet = FindOverrideSet(InOverrideSetName);
    if (!OverrideSet)
    {
	    return false;
    }

    int32 Index = OverrideSet->OpOverrides.IndexOfByPredicate([&](const FRetargetOpOverrides& O) { return O.OpName == InOpName; });
    if (Index == INDEX_NONE)
    {
	    return false;
    }

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("RemoveOpOverrides", "Removed Op Overrides from Set"), bShouldTransact);
    Asset->Modify();
    
    OverrideSet->OpOverrides.RemoveAt(Index);

	PostEditOverrideSets();
    return true;
}

bool UIKRetargeterController::UpdateOverrideValue(
	const FName InOverrideSetName,
	const FName InOpName,
	const FString& InPropertyPath,
	const FStructOnScope& InNewValue,
	const bool bShouldTransact) const
{
	FRetargetOpOverrides* OpOverrides = FindOpOverrides(InOverrideSetName, InOpName);
	if (!OpOverrides)
	{
		return false;
	}

	FRetargetOpPropertyOverride* PropertyOverride = OpOverrides->FindPropertyOverride(InPropertyPath);
	if (!PropertyOverride)
	{
		return false;
	}

	if (bShouldTransact)
	{
		FScopedTransaction Transaction(LOCTEXT("UpdateOverrideValue", "Edit Override Value"), bShouldTransact);
		Asset->Modify();
	}
	
	const UStruct* BaseStruct = InNewValue.GetStruct();
	const UScriptStruct* ScriptStruct = CastChecked<UScriptStruct>(BaseStruct);
	OpOverrides->UpdateOverrideValue(*PropertyOverride, InNewValue.GetStructMemory(), ScriptStruct);

	if (bShouldTransact)
	{
		PostEditOverrideSets();	
	}
	
	return true;
}

bool UIKRetargeterController::BindPropertyOverrideToCurve(
	const FName& InCurveName,
	const FName InOverrideSetName,
	const FName InOpName,
	const FString& InPropertyPath) const
{
	FRetargetOpPropertyOverride* PropertyOverride = FindPropertyOverride(InOverrideSetName, InOpName, InPropertyPath);
	if (!PropertyOverride)
	{
		return false;
	}
	
	if (!CanPropertyOverrideBindToCurve(InOpName, InPropertyPath))
	{
		return false; // override type not compatible with curves
	}

	constexpr bool bShouldTransact = true;
	FScopedTransaction Transaction(LOCTEXT("BindCurveToProperty", "Edited Property Override Curve Binding"), bShouldTransact);
	Asset->Modify();

	PropertyOverride->SetBoundCurveName(InCurveName);
	PropertyOverride->SetBoundVariableName(NAME_None); // mutually exclusive binding

	PostEditOverrideSets();
	return true;
}

bool UIKRetargeterController::BindPropertyOverrideToVariable(
	const FName InVariableName,
	const FName InOverrideSetName,
	const FName InOpName,
	const FString& InPropertyPath) const
{
	FProperty* OverrideProperty = GetPropertyForOverride(InOpName, InPropertyPath);
	if (!OverrideProperty)
	{
		return false;
	}

	// user can pass in Name_None to "unbind" a property
	// unless we are unbinding, we must validate compatibility between the overridden property type and the variable type
	if (InVariableName != NAME_None)
	{
		const FProperty* VariableProperty = GetVariableProperty(InVariableName);
		if (!VariableProperty)
		{
			return false; // variable not found with that name
		}

		if (!IsPropertyCompatibleForBinding(OverrideProperty, VariableProperty))
		{
			return false; // variable type not compatible with override property type
		}
	}

	FRetargetOpPropertyOverride* PropertyOverride = FindPropertyOverride(InOverrideSetName, InOpName, InPropertyPath);
	if (!PropertyOverride)
	{
		return false;
	}
	
	constexpr bool bShouldTransact = true;
	FScopedTransaction Transaction(LOCTEXT("BindVariableToProperty", "Edited Property Override Variable Binding"), bShouldTransact);
	Asset->Modify();
	
	PropertyOverride->SetBoundVariableName(InVariableName);
	PropertyOverride->SetBoundCurveName(NAME_None); // mutually exclusive binding

	PostEditOverrideSets();
	return true;
}

bool UIKRetargeterController::ClearPropertyOverrideBinding(const FName InOverrideSetName, const FName InOpName, const FString& InPropertyPath) const
{
	FRetargetOpPropertyOverride* PropertyOverride = FindPropertyOverride(InOverrideSetName, InOpName, InPropertyPath);
	if (!PropertyOverride)
	{
		return false; // not found
	}

	if (PropertyOverride->GetBoundCurveName() == NAME_None && PropertyOverride->GetBoundVariableName() == NAME_None)
	{
		return false; // not bound
	}
	
	constexpr bool bShouldTransact = true;
	FScopedTransaction Transaction(LOCTEXT("ClearPropertyBinding", "Clear Override Binding"), bShouldTransact);
	Asset->Modify();
	
	PropertyOverride->SetBoundVariableName(NAME_None);
	PropertyOverride->SetBoundCurveName(NAME_None);

	PostEditOverrideSets();
	return true;
}

bool UIKRetargeterController::CanPropertyOverrideBindToCurve(
	const FName InOpName,
	const FString& InPropertyPath) const
{
	FProperty* OverrideProperty = GetPropertyForOverride(InOpName, InPropertyPath);
	if (!OverrideProperty)
	{
		return false;
	}

	return OverrideProperty->IsA<FNumericProperty>(); // can only bind numeric properties to curves
}

bool UIKRetargeterController::AddPropertyOverrideToOp(const FName InOverrideSetName, const FName InOpName, const FString& InPropertyPath) const
{
    FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
    if (!Op)
    {
	    return false;
    }
    
    FRetargetOpOverrides* OpOverrides = AddOpOverrides(InOverrideSetName, InOpName);
    if (!OpOverrides)
    {
	    return false;
    }

    const uint8* SettingsPtr = reinterpret_cast<const uint8*>(Op->GetSettings());
    const UScriptStruct* SettingsStruct = Op->GetSettingsType();

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("AddPropertyOverride", "Added Property Override"), bShouldTransact);
    Asset->Modify();
    
    const bool bWasAdded = OpOverrides->AddPropertyOverride(InPropertyPath, SettingsPtr, SettingsStruct, false);

	PostEditOverrideSets();
    return bWasAdded;
}

bool UIKRetargeterController::RemovePropertyOverrideFromOp(const FName InOverrideSetName, const FName InOpName, const FString& InPropertyPath) const
{
    FRetargetOpOverrides* OpOverrides = FindOpOverrides(InOverrideSetName, InOpName);
    if (!OpOverrides)
    {
	    return false;
    }

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("RemovePropertyOverride", "Removed Property Override"), bShouldTransact);
    Asset->Modify();
    
    const bool bWasRemoved = OpOverrides->RemovePropertyOverride(InPropertyPath);

    // clean up Op entry if no overrides remain
    if (OpOverrides->GetNumPropertyOverrides() == 0)
    {
		RemoveOpOverrides(InOverrideSetName, InOpName);
    }

	PostEditOverrideSets();
    return bWasRemoved;
}

bool UIKRetargeterController::RemoveAllPropertyOverridesFromSet(const FName InOverrideSetName) const
{
    FRetargetOverrideSet* OverrideSet = FindOverrideSet(InOverrideSetName);
    if (!OverrideSet)
    {
	    return false;
    }

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("ClearOverrideSet", "Clear Override Set"), bShouldTransact);
    Asset->Modify();

    OverrideSet->OpOverrides.Empty();
    
	PostEditOverrideSets();
    return true;
}

bool UIKRetargeterController::SetOverrideSetActiveByDefault(const FName InOverrideSetName, const bool bActiveByDefault) const
{
    FRetargetOverrideSet* OverrideSet = FindOverrideSet(InOverrideSetName);
    if (!OverrideSet)
    {
	    return false;
    }

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("SetOverrideSetActive", "Set Override Set Active By Default"), bShouldTransact);
    Asset->Modify();

    OverrideSet->bActiveByDefault = bActiveByDefault;
    
	PostEditOverrideSets();
    return true;
}

bool UIKRetargeterController::GetOverrideSetActiveByDefault(const FName InOverrideSetName) const
{
    FRetargetOverrideSet* OverrideSet = FindOverrideSet(InOverrideSetName);
    return OverrideSet ? OverrideSet->bActiveByDefault : false;
}

bool UIKRetargeterController::OverrideAllProperties(const FName InOverrideSetName, const FName InOpName) const
{
    FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
    if (!Op)
    {
	    return false;
    }

    TSharedPtr<FRetargetPropertyOverrideNode> RootNode;
    Op->GetOverrideableProperties(RootNode);
    if (!RootNode.IsValid())
    {
	    return false;
    }

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("OverrideAllProperties", "Override All Properties"), bShouldTransact);
    Asset->Modify();
    
    TArray<TSharedPtr<FRetargetPropertyOverrideNode>> NodeStack;
    NodeStack.Push(RootNode);
    while (!NodeStack.IsEmpty())
	{
		TSharedPtr<FRetargetPropertyOverrideNode> CurrentNode = NodeStack.Pop();
		for (const FRetargetPropertyOverrideNode::FRetargetPropertyOverrideEntry& Leaf : CurrentNode->LeafProperties)
		{
			AddPropertyOverrideToOp(InOverrideSetName, InOpName, Leaf.PropertyPath);
		}
		for (TTuple<FName, TSharedPtr<FRetargetPropertyOverrideNode>>& It : CurrentNode->SubNodes)
		{
			NodeStack.Push(It.Value);
		}
	}

	PostEditOverrideSets();
    return true;
}

bool UIKRetargeterController::OverrideAllEditedProperties(const FName InOverrideSetName, const FName InOpName) const
{
    FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
    if (!Op)
    {
	    return false;
    }

    TSharedPtr<FRetargetPropertyOverrideNode> RootNode;
    Op->GetOverrideableProperties(RootNode);
    if (!RootNode.IsValid())
    {
	    return false;
    }

    const UScriptStruct* SettingsStruct = Op->GetSettingsType();
    uint8* SettingsData = reinterpret_cast<uint8*>(Op->GetSettings());

    constexpr bool bShouldTransact = true;
    FScopedTransaction Transaction(LOCTEXT("OverrideAllEditedProperties", "Override All Edited Properties"), bShouldTransact);
    Asset->Modify();
    
    TArray<TSharedPtr<FRetargetPropertyOverrideNode>> NodeStack;
    NodeStack.Push(RootNode);
	while (!NodeStack.IsEmpty())
	{
		TSharedPtr<FRetargetPropertyOverrideNode> CurrentNode = NodeStack.Pop();
		for (const FRetargetPropertyOverrideNode::FRetargetPropertyOverrideEntry& Leaf : CurrentNode->LeafProperties)
		{
			if (!FRetargetOpPropertyOverride::IsStructPropertyAtDefault(SettingsStruct, SettingsData, Leaf.PropertyPath))
			{
				AddPropertyOverrideToOp(InOverrideSetName, InOpName, Leaf.PropertyPath);    
			}
		}
		for (TTuple<FName, TSharedPtr<FRetargetPropertyOverrideNode>>& It : CurrentNode->SubNodes)
		{
			NodeStack.Push(It.Value);
		}
	}

	PostEditOverrideSets();
    return true;
}

bool UIKRetargeterController::HasPropertyOverride(const FName InOverrideSetName, const FName InOpName, const FString& InPropertyPath) const
{
    FRetargetOpOverrides* OpOverrides = FindOpOverrides(InOverrideSetName, InOpName);
    return OpOverrides ? OpOverrides->HasPropertyOverride(InPropertyPath) : false;
}

FName UIKRetargeterController::GetOverrideSetName(const FRetargetOverrideSet* InOverrideSet) const
{
    if (!InOverrideSet)
    {
	    return NAME_None;
    }
    
    for (const TTuple<FName, FRetargetOverrideSet>& It : Asset->OverrideSets)
    {
       if (&It.Value == InOverrideSet)
       {
	       return It.Key;
       }
    }
    return NAME_None;
}

FProperty* UIKRetargeterController::GetPropertyForOverride(const FName InOpName, const FString& InPropertyPath) const
{
	const FIKRetargetOpBase* Op = GetRetargetOpByName(InOpName);
	if (!Op)
	{
		return nullptr; // op not found in stack
	}

	// try to parse the path into segments
	const UScriptStruct* SettingsStruct = Op->GetSettingsType();
	TArray<FRetargetOpPropertyOverride::FPropertySegment> Segments;
	if (!FRetargetOpPropertyOverride::GetSegmentsFromProperyPath(InPropertyPath, SettingsStruct, Segments))
	{
		return nullptr; // the path is broken (ie, that property was not found on the given op settings)
	}

	// get the leaf property
	return FRetargetOpPropertyOverride::GetLeafProperty(Segments);
}

const FRetargetVariableContainer& UIKRetargeterController::GetVariables() const
{
	return Asset->GetVariables();
}

const FProperty* UIKRetargeterController::GetVariableProperty(const FName InVariableName) const
{
	const FRetargetVariableContainer& Variables = Asset->GetVariables();
	const FPropertyBagPropertyDesc* PropertyDesc = Variables.Bag.FindPropertyDescByName(InVariableName);
	if (!PropertyDesc)
	{
		return nullptr;
	}
	
	return PropertyDesc->CachedProperty;
}

bool UIKRetargeterController::IsPropertyCompatibleForBinding(const FProperty* Source, const FProperty* Target)
{
	if (!Source || !Target)
	{
		return false;
	}

	// exact match (handles structs, objects, enums)
	if (Source->SameType(Target))
	{
		return true;
	}

	// numeric cross-compatibility (float <-> double)
	if (Source->IsA<FNumericProperty>() && Target->IsA<FNumericProperty>())
	{
		const FNumericProperty* SourceNumeric = CastField<FNumericProperty>(Source);
		const FNumericProperty* TargetNumeric = CastField<FNumericProperty>(Target);

		// allow any float type to any float type
		if (SourceNumeric->IsFloatingPoint() && TargetNumeric->IsFloatingPoint())
		{
			return true;
		}

		// allow any int type to any int type (int, int32, uint32 etc)
		if (SourceNumeric->IsInteger() && TargetNumeric->IsInteger())
		{
			return true;
		}
	}

	return false;
};

FName UIKRetargeterController::GetVariableBoundToPropertyOverride(
	const FName InOverrideSetName,
	const FName InOpName,
	const FString& InPropertyPath) const
{
	FRetargetOpPropertyOverride* PropertyOverride = FindPropertyOverride(InOverrideSetName, InOpName, InPropertyPath);
	if (!PropertyOverride)
	{
		return NAME_None; // no override found
	}

	return PropertyOverride->GetBoundVariableName();
}

FName UIKRetargeterController::GetCurveBoundToPropertyOverride(
	const FName InOverrideSetName,
	const FName InOpName,
	const FString& InPropertyPath) const
{
	FRetargetOpPropertyOverride* PropertyOverride = FindPropertyOverride(InOverrideSetName, InOpName, InPropertyPath);
	if (!PropertyOverride)
	{
		return NAME_None; // no override found
	}
	
	return PropertyOverride->GetBoundCurveName();
}

bool UIKRetargeterController::GetPropertyOverrideHasBinding(
	const FName InOverrideSetName,
	const FName InOpName,
	const FString& InPropertyPath) const
{
	FRetargetOpPropertyOverride* PropertyOverride = FindPropertyOverride(InOverrideSetName, InOpName, InPropertyPath);
	if (!PropertyOverride)
	{
		return false; // no override found
	}

	return PropertyOverride->GetBoundVariableName() != NAME_None || PropertyOverride->GetBoundCurveName() != NAME_None;
}

FName UIKRetargeterController::AddNewVariable(const FName InVariableName) const
{
	FRetargetVariableContainer& Container = GetAsset()->GetVariables();
	
	auto GetSafeNewVariableName = [Container, InVariableName]() -> FName
	{
		// sanitize
		FName SanitizedBase = FInstancedPropertyBag::SanitizePropertyName(InVariableName);
		if (SanitizedBase == NAME_None)
		{
			SanitizedBase = TEXT("NewVar");
		}

		// make unique
		FName NewName = SanitizedBase;
		int32 Suffix = 0;
		while (Container.Bag.FindPropertyDescByName(NewName) != nullptr)
		{
			Suffix++;
			NewName = FName(*FString::Printf(TEXT("%s_%d"), *SanitizedBase.ToString(), Suffix));
		}

		return NewName;
	};

	FScopedTransaction Transaction(LOCTEXT("AddVariable", "Add Retarget Variable"));
	Asset->Modify();

	// get a unique and sanitized name
	FName NewName = GetSafeNewVariableName();
	// add as float by default
	Container.Bag.AddProperty(NewName, EPropertyBagPropertyType::Float);

	Asset->OverridesVersion++;
	VariablesModified.Broadcast();
	return NewName;
}

bool UIKRetargeterController::DeleteVariable(const FName InVariableName) const
{
	if (!GetVariableExists(InVariableName))
	{
		return false;
	}
	
	FScopedTransaction Transaction(LOCTEXT("DeleteVar", "Delete Variable"));
	Asset->Modify();

	EPropertyBagAlterationResult Result = Asset->GetVariables().Bag.RemovePropertyByName(InVariableName);

	// remove any bindings
	IterateAllOverrides([InVariableName](FRetargetOpPropertyOverride& PropertyOverride)
	{
		if (PropertyOverride.GetBoundVariableName() == InVariableName)
		{
			PropertyOverride.SetBoundVariableName(NAME_None);
		}
	});

	Asset->OverridesVersion++;
	VariablesModified.Broadcast();
	return Result == EPropertyBagAlterationResult::Success;
}

bool UIKRetargeterController::RenameVariable(const FName InVariableName, const FName InNewName) const
{
	if (!GetVariableExists(InVariableName))
	{
		return false;
	}
	
	FScopedTransaction Transaction(LOCTEXT("RenameVar", "Rename Variable"));
	Asset->Modify();
	
	EPropertyBagAlterationResult Result = Asset->GetVariables().Bag.RenameProperty(InVariableName, InNewName);

	// update any bindings using the old name
	IterateAllOverrides([InVariableName, InNewName](FRetargetOpPropertyOverride& PropertyOverride)
	{
		if (PropertyOverride.GetBoundVariableName() == InVariableName)
		{
			PropertyOverride.SetBoundVariableName(InNewName);
		}
	});

	Asset->OverridesVersion++;
	VariablesModified.Broadcast();
	return Result == EPropertyBagAlterationResult::Success;
}

bool UIKRetargeterController::SetVariableType(const FName InVariableName, const FEdGraphPinType& InNewPinType) const
{
	if (!GetVariableExists(InVariableName))
	{
		return false;
	}
	
	const FProperty* VariableProperty = GetVariableProperty(InVariableName);
	if (!ensure(VariableProperty))
	{
		return false; // should not get this far with an unknown variable
	}

	FScopedTransaction Transaction(LOCTEXT("ChangeType", "Change Variable Type"));
	Asset->Modify();

	// modify the type for this variable
	FRetargetVariableContainer& Container = Asset->GetVariables();
	const FPropertyBagPropertyDesc* PropertyDesc = Container.Bag.FindPropertyDescByName(InVariableName);
	FPropertyBagPropertyDesc NewDesc = *PropertyDesc;
	UE::StructUtils::SetPropertyDescFromPin(NewDesc, InNewPinType);
	Container.Bag.AddProperties({ NewDesc });
	
	// changing the variable type potentially makes it's bindings incompatible
	const FProperty* NewVariableProperty = GetVariableProperty(InVariableName);
	for (TTuple<FName, FRetargetOverrideSet>& OverrideSetPair : Asset->OverrideSets)
	{
		for (FRetargetOpOverrides& OpOverrides : OverrideSetPair.Value.OpOverrides)
		{
			for (FRetargetOpPropertyOverride& PropertyOverride : OpOverrides.PropertyOverrides)
			{
				if (PropertyOverride.GetBoundVariableName() != InVariableName)
				{
					continue;
				}
				
				FProperty* OverrideProperty = GetPropertyForOverride(OpOverrides.OpName, PropertyOverride.GetPropertyPath());
				if (!IsPropertyCompatibleForBinding(NewVariableProperty, OverrideProperty))
				{
					// variable type no longer compatible with override property type
					PropertyOverride.SetBoundVariableName(NAME_None);
				}
			}
		}
	}

	Asset->OverridesVersion++;
	VariablesModified.Broadcast();
	return true;
}

bool UIKRetargeterController::GetVariableExists(const FName InVariableName) const
{
	const FRetargetVariableContainer& Variables = Asset->GetVariables();
	const FPropertyBagPropertyDesc* PropertyDesc = Variables.Bag.FindPropertyDescByName(InVariableName);
	return PropertyDesc != nullptr;
}

void UIKRetargeterController::IterateAllOverrides(const TFunctionRef<void(FRetargetOpPropertyOverride&)>& InVisitor) const
{
	for (TTuple<FName, FRetargetOverrideSet>& OverrideSetPair : Asset->OverrideSets)
	{
		for (FRetargetOpOverrides& OpOverrides : OverrideSetPair.Value.OpOverrides)
		{
			for (FRetargetOpPropertyOverride& PropertyOverride : OpOverrides.PropertyOverrides)
			{
				InVisitor(PropertyOverride);
			}
		}
	}
}

void UIKRetargeterController::PostEditOverrideSets() const
{
	// Ensure all sets have unique DisplayOrders. Sets from assets created before this fix
	// will all have DisplayOrder=0 (the default), which causes an unstable sort and
	// effectively alphabetical ordering. Normalize them now, grouped by parent.
	{
		TMap<FName, TArray<FName>> SiblingGroups;
		for (const TTuple<FName, FRetargetOverrideSet>& Entry : Asset->GetOverrideSets())
		{
			SiblingGroups.FindOrAdd(Entry.Value.ParentName).Add(Entry.Key);
		}

		for (TTuple<FName, TArray<FName>>& Group : SiblingGroups)
		{
			TSet<int32> SeenOrders;
			bool bHasDuplicates = false;
			for (const FName& SetName : Group.Value)
			{
				if (const FRetargetOverrideSet* Set = FindOverrideSet(SetName))
				{
					if (SeenOrders.Contains(Set->DisplayOrder))
					{
						bHasDuplicates = true;
						break;
					}
					SeenOrders.Add(Set->DisplayOrder);
				}
			}

			if (bHasDuplicates)
			{
				// Sort by DisplayOrder first, then name as a stable tiebreaker,
				// then reassign sequential values.
				Group.Value.Sort([this](const FName& A, const FName& B)
				{
					const FRetargetOverrideSet* SetA = FindOverrideSet(A);
					const FRetargetOverrideSet* SetB = FindOverrideSet(B);
					if (SetA && SetB && SetA->DisplayOrder != SetB->DisplayOrder)
					{
						return SetA->DisplayOrder < SetB->DisplayOrder;
					}
					return A.LexicalLess(B);
				});

				for (int32 Index = 0; Index < Group.Value.Num(); ++Index)
				{
					if (FRetargetOverrideSet* Set = const_cast<FRetargetOverrideSet*>(FindOverrideSet(Group.Value[Index])))
					{
						Set->DisplayOrder = Index;
					}
				}
			}
		}
	}

	// informs any attached runtime processors to rebuild their override cache
	Asset->OverridesVersion++;
	// informs editor to refresh the override UI
	OverrideSetsModified.Broadcast();
}

//
// BEGIN DEPRECATED API
//
PRAGMA_DISABLE_DEPRECATION_WARNINGS

FTargetRootSettings UIKRetargeterController::GetRootSettings() const
{
	FScopeLock Lock(&ControllerLock);
	return GetAsset()->GetRootSettingsUObject()->Settings;
}

void UIKRetargeterController::SetRootSettings(const FTargetRootSettings& RootSettings) const
{
	FScopeLock Lock(&ControllerLock);
	FScopedTransaction Transaction(LOCTEXT("SetRootSettings_Transaction", "Set Root Settings"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	GetAsset()->Modify();
	GetAsset()->GetRootSettingsUObject()->Settings = RootSettings;
}

FRetargetGlobalSettings UIKRetargeterController::GetGlobalSettings() const
{
	FScopeLock Lock(&ControllerLock);
	return GetAsset()->GetGlobalSettingsUObject()->Settings;
}

void UIKRetargeterController::SetGlobalSettings(const FRetargetGlobalSettings& GlobalSettings) const
{
	FScopeLock Lock(&ControllerLock);
	FScopedTransaction Transaction(LOCTEXT("SetGlobalSettings_Transaction", "Set Global Settings"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	GetAsset()->Modify();
	GetAsset()->GetGlobalSettingsUObject()->Settings = GlobalSettings;
}

FTargetChainSettings UIKRetargeterController::GetRetargetChainSettings(const FName& TargetChainName) const
{
	FScopeLock Lock(&ControllerLock);
	
	const URetargetChainSettings* ChainSettings = GetChainSettings(TargetChainName);
	if (!ChainSettings)
	{
		return FTargetChainSettings();
	}

	return ChainSettings->Settings;
}

bool UIKRetargeterController::SetRetargetChainSettings(const FName& TargetChainName, const FTargetChainSettings& Settings) const
{
	FScopeLock Lock(&ControllerLock);

	FScopedTransaction Transaction(LOCTEXT("SetChainSettings_Transaction", "Set Chain Settings"));
	FScopedReinitializeIKRetargeter Reinitialize(this);
	
	if (URetargetChainSettings* ChainSettings = GetChainSettings(TargetChainName))
	{
		ChainSettings->Modify();
		ChainSettings->Settings = Settings;
		return true;
	}

	return false;
}

const TArray<URetargetChainSettings*>& UIKRetargeterController::GetAllChainSettings() const
{
	return Asset->ChainSettings_DEPRECATED;
}

URetargetChainSettings* UIKRetargeterController::GetChainSettings(const FName& TargetChainName) const
{
	for (URetargetChainSettings* ChainMap : Asset->ChainSettings_DEPRECATED)
	{
		if (ChainMap->TargetChain == TargetChainName)
		{
			return ChainMap;
		}
	}

	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
//
// END DEPRECATED API
//

#undef LOCTEXT_NAMESPACE

