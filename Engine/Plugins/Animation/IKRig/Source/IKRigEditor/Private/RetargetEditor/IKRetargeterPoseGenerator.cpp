// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargeterPoseGenerator.h"

#include "AnimationRuntime.h"
#include "Engine/SkeletalMesh.h"
#include "ScopedTransaction.h"
#include "MeshDescription.h"
#include "ReferenceSkeleton.h"
#include "SkeletalMeshAttributes.h"
#include "Algo/LevenshteinDistance.h"
#include "RetargetEditor/IKRetargetEditorController.h"
#include "Retargeter/IKRetargeter.h"
#include "Async/ParallelFor.h"

#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargeterPoseGenerator)

#define LOCTEXT_NAMESPACE "RetargetPoseGenerator"

FRetargetAutoPoseGenerator::FRetargetAutoPoseGenerator(const TWeakObjectPtr<UIKRetargeterController> InController)
{
	Controller = InController;
	// we maintain our own processor separate from the editor, this allows the auto-pose generator
	// to run "headless" and be accessed by BP/Python API in the asset controller
	Processor = MakeShared<FIKRetargetProcessor>();
}

void FRetargetAutoPoseGenerator::AlignBones(
	const TArray<FName>& BonesToAlign,
	const ERetargetAutoAlignMethod Method, 
	const ERetargetSourceOrTarget SourceOrTarget,
	const bool bSuppressWarnings) const
{
	// cannot align bones until we have an initialized processor
	if (!CheckReadyToAlignBones())
	{
		return;
	}
	
	// make copy to sort hierarchically (we want to align from root to leaf order)
	TArray<FName> SortedBones = BonesToAlign;

	// if user passed in no bones, then align ALL bones
	if (SortedBones.IsEmpty())
	{
		SortedBones = Processor->GetSkeleton(SourceOrTarget).BoneNames;
	}
	
	const FRetargetSkeleton& Skeleton = Processor->GetSkeleton(SourceOrTarget);
	SortedBones.Sort([Skeleton](FName A, FName B) -> bool
	{
		return Skeleton.FindBoneIndexByName(A) < Skeleton.FindBoneIndexByName(B);
	});

	// actually align the bones (generating new entries in the retarget pose)
	for (const FName BoneToAlign : SortedBones)
	{
		AlignBone(BoneToAlign, Method, SourceOrTarget);
	}

	// warn if any bones would not be aligned so user isn't left wondering
	if (!bSuppressWarnings)
	{
		// warn user if they tried to align the pelvis (doesn't work)
		if (BonesToAlign.Contains(Processor->GetPelvisBone(SourceOrTarget, ERetargetOpsToSearch::AssetOps)))
		{
			Processor->Log.LogWarning(LOCTEXT("AutoAlignPelvisNotSupported", "Skipped aligning the pelvis. This is not supported."));
		}

		// warn user if they tried to align a non-retargeted bone (doesn't work)
		for (const FName Bone : BonesToAlign)
		{
			if (!Processor->IsBoneMapped(Bone, SourceOrTarget))
			{
				Processor->Log.LogWarning(FText::Format(LOCTEXT("SkipAutoAlignNonRetargeted", "Skipped aligning {0}. The bone is not retargeted and therefore has no 'matching' equivalent on other skeleton."), FText::FromName(Bone)));
			}
		}
	}
}

void FRetargetAutoPoseGenerator::AlignBone(
	const FName BoneToAlign,
	const ERetargetAutoAlignMethod Method,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	// cannot align bones until we have an initialized processor
	if (!CheckReadyToAlignBones())
	{
		return;
	}
	
	// can't auto-align if the bone is not retargeted  (warning is elsewhere)
	if (!Processor->IsBoneMapped(BoneToAlign, SourceOrTarget))
	{
		return;
	}

	// force regenerate the retarget pose to guarantee Skeleton.RetargetGlobalPose reflects any modifications made in the viewport
	const FName CurrentPoseName = Controller->GetCurrentRetargetPoseName(SourceOrTarget);
	Processor->UpdateRetargetPoseAtRuntime(CurrentPoseName, SourceOrTarget);

	// use a custom procedure to auto-align pelvis
	if (BoneToAlign == Processor->GetPelvisBone(SourceOrTarget, ERetargetOpsToSearch::AssetOps))
	{
		AlignPelvisBone(SourceOrTarget);
		return;
	}

	// determine which side we're aligning and which side we're matching
	const ERetargetSourceOrTarget SideToAlign = SourceOrTarget;
	const ERetargetSourceOrTarget SideToMatch = SideToAlign == ERetargetSourceOrTarget::Source ? ERetargetSourceOrTarget::Target : ERetargetSourceOrTarget::Source;
	
	// find chains on both sides
	FName ChainToAlign;
	FName ChainToMatch;
	if (!Processor->GetFirstChainMappingForBone(BoneToAlign, SideToAlign, ChainToAlign, ChainToMatch))
	{
		ensureMsgf(false, TEXT("Bone should never pass IsBoneMapped but not have both chains."));
		return;
	}
	
	// get the normalized chain parameter to match
	const float Param = Processor->GetParamOfBoneInChain(BoneToAlign, SideToAlign, ChainToAlign);

	// find tangents to align between source/target
	FVector DirectionToAlign;
	FVector DirectionToMatch;
	bool bFoundAlignVector = false;
	bool bFoundMatchVector = false;
	switch (Method)
	{
	case ERetargetAutoAlignMethod::ChainToChain:
		{
			bFoundAlignVector = GetChainTangent(ChainToAlign, Param, SideToAlign, DirectionToAlign);
			bFoundMatchVector = GetChainTangent(ChainToMatch, Param, SideToMatch, DirectionToMatch);
			break;
		}

	case ERetargetAutoAlignMethod::MeshToMesh:
		{
			bFoundAlignVector = GetChainTangentFromMesh(ChainToAlign, Param, SideToAlign, DirectionToAlign);
			bFoundMatchVector = GetChainTangentFromMesh(ChainToMatch, Param, SideToMatch, DirectionToMatch);
			break;
		}

	case ERetargetAutoAlignMethod::LocalRotationAxes:
		{
			// get local rotation of equivalent bone to match
			FName BoneToMatch = Processor->GetClosestBoneToParam(ChainToMatch, Param, SideToMatch);
			if (BoneToMatch == NAME_None)
			{
				return;
			}
			FTransform LocalRtgPoseBoneToMatch = Processor->GetRetargetPoseBoneTransform(BoneToMatch, SideToMatch, ERetargetBoneSpace::Local);

			// get the local ref pose of the bone to align
			const FRetargetSkeleton& SkeletonToAlign = Processor->GetSkeleton(SideToAlign);
			const FReferenceSkeleton& RefSkeletonToAlign = SkeletonToAlign.SkeletalMesh->GetRefSkeleton();
			const int32 BoneToAlignIndex = SkeletonToAlign.FindBoneIndexByName(BoneToAlign);
			FTransform LocalRefBoneToAlign = RefSkeletonToAlign.GetRefBonePose()[BoneToAlignIndex];

			// calc delta to rotate from the ref pose to match the LRA of the other bone's retarget pose
			const FQuat LocalRotationOffset = LocalRefBoneToAlign.GetRotation() * LocalRtgPoseBoneToMatch.GetRotation().Inverse();
			Controller->SetRotationOffsetForRetargetPoseBone(BoneToAlign, LocalRotationOffset, SideToAlign);
			Processor->UpdateRetargetPoseAtRuntime(CurrentPoseName, SideToAlign);
			return;
		}
		
	case ERetargetAutoAlignMethod::GlobalRotationAxes:
		{
			// get current retarget pose global transform with no delta of the bone to align
			const FRetargetSkeleton& SkeletonToAlign = Processor->GetSkeleton(SideToAlign);
			const FReferenceSkeleton& RefSkeletonToAlign = SkeletonToAlign.SkeletalMesh->GetRefSkeleton();
			const int32 BoneToAlignIndex = SkeletonToAlign.FindBoneIndexByName(BoneToAlign);
			FTransform ParentOfToAlignGlobal = FTransform::Identity;
			const int32 ParentOfToAlignIndex = RefSkeletonToAlign.GetParentIndex(BoneToAlignIndex);
			if (ParentOfToAlignIndex != INDEX_NONE)
			{
				const FName ParentOfToAlignName = RefSkeletonToAlign.GetBoneName(ParentOfToAlignIndex);
				ParentOfToAlignGlobal = Processor->GetRetargetPoseBoneTransform(ParentOfToAlignName, SideToAlign, ERetargetBoneSpace::Global);
			}
			FTransform BoneToAlignLocalRefPose = RefSkeletonToAlign.GetRefBonePose()[BoneToAlignIndex];
			FTransform BoneToAlignGlobalCurrentNoDelta = BoneToAlignLocalRefPose * ParentOfToAlignGlobal;

			// get current retarget pose global transform of bone to match
			FName BoneToMatch = Processor->GetClosestBoneToParam(ChainToMatch, Param, SideToMatch);
			if (BoneToMatch == NAME_None)
			{
				return;
			}
			FTransform BoneToMatchGlobalCurrentRtgPose = Processor->GetRetargetPoseBoneTransform(BoneToMatch, SideToMatch, ERetargetBoneSpace::Global);

			// transform BoneToAlign and BoneToMatch into the parent's local space
			FTransform ParentOfToAlignInv = ParentOfToAlignGlobal.Inverse();
			FTransform LocalToAlign = BoneToAlignGlobalCurrentNoDelta * ParentOfToAlignInv;
			FTransform LocalToMatch = BoneToMatchGlobalCurrentRtgPose * ParentOfToAlignInv;

			// compute the delta transform in local space
			FTransform LocalDeltaTransform = LocalToMatch * LocalToAlign.Inverse();

			Controller->SetRotationOffsetForRetargetPoseBone(BoneToAlign, LocalDeltaTransform.GetRotation().GetNormalized(), SideToAlign);
			Processor->UpdateRetargetPoseAtRuntime(CurrentPoseName, SideToAlign);
			
			return;
		}
		
	default:
		{
			checkNoEntry();
			break;
		}
	}
	
	// warn if unable to generate a direction vector from the mesh
	if (!(bFoundAlignVector && bFoundMatchVector))
	{
		// user asked to align the mesh, but direction vectors for the target could not be generated
		if (!bFoundAlignVector && Method == ERetargetAutoAlignMethod::MeshToMesh)
		{
			Processor->Log.LogWarning(
			FText::Format(LOCTEXT("AutoAlignMeshNoTarget", "Could not determine the direction of bone to align {0} from the mesh. Likely because no vertices are skinned to the target bone."),
				FText::FromName(BoneToAlign)));
		}
		
		// user asked to align the mesh, but direction vectors for the target could not be generated
		if (!bFoundMatchVector && Method == ERetargetAutoAlignMethod::MeshToMesh)
		{
			Processor->Log.LogWarning(
			FText::Format(LOCTEXT("AutoAlignMeshNoSource", "Could not determine the direction of bone to match from it's mesh for {0}. Likely because no vertices are skinned to the source bone."),
				FText::FromName(BoneToAlign)));
		}

		// user asked to align the mesh, but direction vectors for the target could not be generated
		if (!bFoundAlignVector || !bFoundMatchVector)
		{
			Processor->Log.LogWarning(
			FText::Format(LOCTEXT("AutoAlignFailed", "Could not determine the direction align the bone: "),
				FText::FromName(BoneToAlign)));
		}
		
		return;
	}

	// find quaternion to rotate from current to target
	const FQuat WorldRotationOffset = FQuat::FindBetweenNormals(DirectionToAlign, DirectionToMatch);

	// convert to local space offset and store in retarget pose
	ApplyWorldRotationToRetargetPose(WorldRotationOffset, BoneToAlign, SideToAlign);
}

void FRetargetAutoPoseGenerator::AlignPelvisBone(const ERetargetSourceOrTarget SourceOrTarget) const
{
	// when aligning the pelvis bone, we have to use the Spine to determine it's direction
	// any other assumptions are bound to fail in different ways, so finding a Spine is the only reliable way (that we've found so far)
	FName SourceSpineChainName;
	FName TargetSpineChainName;
	const bool bFoundSourceSpine = GetSpineChain(ERetargetSourceOrTarget::Source, SourceSpineChainName);
	const bool bFoundTargetSpine = GetSpineChain(ERetargetSourceOrTarget::Target, TargetSpineChainName);
	
	// user asked to align the pelvis but could not find the spine to use for its direction vector
	if (!(bFoundSourceSpine && bFoundTargetSpine))
	{
		Processor->Log.LogWarning(
		LOCTEXT("MissingSpine", "Could not auto align the pelvis because a Spine chain was not found. Expects a retarget chain named 'Spine' or similar."));
		return;
	}

	// get the direction of the source and target spine chains
	const FVector SourceSpineDirection = GetChainDirection(SourceSpineChainName, ERetargetSourceOrTarget::Source);
	const FVector TargetSpineDirection = GetChainDirection(TargetSpineChainName, ERetargetSourceOrTarget::Target);

	// determine which direction to align
	const bool bAlignSource = SourceOrTarget == ERetargetSourceOrTarget::Source;
	const FVector DirectionToAlign = bAlignSource ? SourceSpineDirection : TargetSpineDirection;
	const FVector DirectionToMatch = bAlignSource ? TargetSpineDirection : SourceSpineDirection;

	// find quaternion to rotate from current to target
	const FQuat WorldRotationOffset = FQuat::FindBetweenNormals(DirectionToAlign, DirectionToMatch);

	// convert to local space offset and store in retarget pose
	const FName BoneToAlign = Processor->GetPelvisBone(SourceOrTarget, ERetargetOpsToSearch::AssetOps);
	ApplyWorldRotationToRetargetPose(WorldRotationOffset, BoneToAlign, SourceOrTarget);
}

void FRetargetAutoPoseGenerator::SnapToGround(const FName BoneToPutOnGround, const ERetargetSourceOrTarget SourceOrTarget) const
{
	// cannot align bones until we have an initialized processor
	if (!CheckReadyToAlignBones())
	{
		return;
	}

	const FRetargetSkeleton& Skeleton = Processor->GetSkeleton(SourceOrTarget);
	
	// force regenerate the retarget pose to guarantee Skeleton.RetargetGlobalPose reflects any modifications made thus far
	const FName CurrentPoseName = Controller->GetCurrentRetargetPoseName(SourceOrTarget);
	Processor->UpdateRetargetPoseAtRuntime(CurrentPoseName, SourceOrTarget);

	// if no bone specified, we search the current retarget pose to find the lowest one and use that
	float HeightOfBoneInRetargetPose = std::numeric_limits<float>::max();
	int32 BoneToSnapIndex = INDEX_NONE;
	if (BoneToPutOnGround == NAME_None)
	{
		// find the lowest bone in the current retarget pose
		for (int32 BoneIndex=0; BoneIndex<Skeleton.BoneNames.Num(); ++BoneIndex)
		{
			if (!Processor->IsBoneMapped(Skeleton.BoneNames[BoneIndex], SourceOrTarget))
			{
				continue;
			}
			const FTransform& BoneTransform = Skeleton.RetargetPoses.GetGlobalRetargetPose()[BoneIndex];
			const float BoneHeight = static_cast<float>(BoneTransform.GetLocation().Z);
			if (BoneHeight < HeightOfBoneInRetargetPose)
			{
				HeightOfBoneInRetargetPose = BoneHeight;
				BoneToSnapIndex = BoneIndex;
			}
		}
	}
	else
	{
		// use the user specified bone
		BoneToSnapIndex = Skeleton.FindBoneIndexByName(BoneToPutOnGround);
		HeightOfBoneInRetargetPose = static_cast<float>(Skeleton.RetargetPoses.GetGlobalRetargetPose()[BoneToSnapIndex].GetLocation().Z);
	}

	if (BoneToSnapIndex == INDEX_NONE)
	{
		// shouldn't happen because of check at start of this function
		return;
	}
	
	// find the height of this bone in the ref pose
	const FReferenceSkeleton& RefSkeleton = Skeleton.SkeletalMesh->GetRefSkeleton();
	const FTransform& RefPoseOfBone = FAnimationRuntime::GetComponentSpaceTransformRefPose(RefSkeleton, BoneToSnapIndex);
	const double HeightInRefPose = RefPoseOfBone.GetLocation().Z;

	// snap it back to original height off the ground
	const double DeltaHeight = HeightInRefPose - HeightOfBoneInRetargetPose;
	Controller->SetRootOffsetInRetargetPose(FVector(0,0,DeltaHeight), SourceOrTarget);
}

bool FRetargetAutoPoseGenerator::GetSpineChain(const ERetargetSourceOrTarget SourceOrTarget, FName& OutChainName) const
{
	OutChainName = NAME_None;
	const FString ChainNameToFind = "spine";
	float HighestScore = -1.f;
	const UIKRigDefinition* IKRig = Processor->GetRetargetAsset()->GetIKRig(SourceOrTarget);
	const TArray<FResolvedBoneChain>* BoneChains = Processor->GetBoneChains().GetResolvedBoneChains(SourceOrTarget, IKRig);
	if (!BoneChains)
	{
		return false;
	}
	
	for (const FResolvedBoneChain& BoneChain : *BoneChains)
	{
		FName ChainNameToTest = BoneChain.ChainName;
		FString ChainNameStr = ChainNameToTest.ToString().ToLower();
		float WorstCase = static_cast<float>(ChainNameToFind.Len() + ChainNameStr.Len());
		WorstCase = WorstCase < 1.0f ? 1.0f : WorstCase;
		const float Score = 1.0f - (static_cast<float>(Algo::LevenshteinDistance(ChainNameToFind, ChainNameStr)) / WorstCase);
		if (Score > HighestScore && Score > 0.5f)
		{
			HighestScore = Score;
			OutChainName = ChainNameToTest;
		}
	}

	return OutChainName != NAME_None;
}

FVector FRetargetAutoPoseGenerator::GetChainDirection(const FName ChainName, const ERetargetSourceOrTarget SourceOrTarget) const
{
	const FTransform ChainStart = Processor->GetGlobalRetargetPoseAtParam(ChainName, 0.f, SourceOrTarget);
	const FTransform ChainEnd = Processor->GetGlobalRetargetPoseAtParam(ChainName, 1.f, SourceOrTarget);
	return (ChainEnd.GetLocation() - ChainStart.GetLocation()).GetSafeNormal();
}

bool FRetargetAutoPoseGenerator::GetChainTangent(
	const FName ChainName,
	const float Param,
	const ERetargetSourceOrTarget SourceOrTarget,
	FVector& OutDirection) const
{
	check(Param >= 0.f);
	
	// cannot use chain tangent at the end of the chain
	if (Param + ParamStepSize >= 1.0f)
	{
		return false;
	}
	
	// use finite differencing to generate a tangent vector at the given param
	const float ParamA = Param;
	const float ParamB = Param + ParamStepSize;
	const FTransform SampleA = Processor->GetGlobalRetargetPoseAtParam(ChainName, ParamA, SourceOrTarget);
	const FTransform SampleB = Processor->GetGlobalRetargetPoseAtParam(ChainName, ParamB, SourceOrTarget);
	OutDirection = (SampleB.GetLocation() - SampleA.GetLocation()).GetSafeNormal();
	return true;
}

bool FRetargetAutoPoseGenerator::GetChainTangentFromMesh(
	const FName ChainName,
	const float Param,
	const ERetargetSourceOrTarget SourceOrTarget,
	FVector& OutDirection) const
{
	// get the bone at this location in the chain
	const FName BoneName = Processor->GetClosestBoneToParam(ChainName, Param, SourceOrTarget);
	
	// get all children of bone
	TArray<FName> BonesToConsider = {BoneName};
	GetAllChildrenOfBone(BoneName, SourceOrTarget, BonesToConsider);
	
	// get all vertices primarily weighted to [children+bone]
	TArray<FVector> Points;
	GetPointsWeightedToBones(BonesToConsider, SourceOrTarget, Points);
	
	// cannot get bone direction from mesh since it's not connected to the mesh
	if (Points.IsEmpty())
	{
		OutDirection = FVector::ZeroVector;
		return false;
	}

	// get centroid of point cloud
	FVector Centroid(0, 0, 0);
	for (const FVector& Point : Points)
	{
		Centroid += Point;
	}
	Centroid /= static_cast<float>(Points.Num());

	// return vector from bone to centroid of mesh
	const FRetargetSkeleton& Skeleton = Processor->GetSkeleton(SourceOrTarget);
	const int32 BoneIndex = Skeleton.FindBoneIndexByName(BoneName);
	const FVector BonePosition = Skeleton.RetargetPoses.GetGlobalRetargetPose()[BoneIndex].GetLocation();
	OutDirection = (Centroid - BonePosition).GetSafeNormal();
	return true;
}

void FRetargetAutoPoseGenerator::GetPointsWeightedToBones(
	const TArray<FName>& BonesToConsider,
	const ERetargetSourceOrTarget SourceOrTarget,
	TArray<FVector>& OutPoints) const
{
	// get all the skin weights and ref pose vertex positions of the mesh
	TArray<TMap<FName, float>> SkinWeights;
	TArray<FVector> VertexRefPose;
	GetRefPoseVerticesAndWeights(SourceOrTarget, SkinWeights, VertexRefPose);
	
	// find vertices weighted to the bones we are considering
	TArray<int32> VerticesToConsider;
	for (int32 VertexIndex = 0; VertexIndex < SkinWeights.Num(); ++VertexIndex)
	{
		const TMap<FName, float>& VertexWeights = SkinWeights[VertexIndex];

		// find the highest weight bone for this vertex
		FName MaxBone;
		float MaxWeight = -1.0f;
		for (const TTuple<FName, float>& Pair : VertexWeights)
		{
			if (Pair.Value > MaxWeight)
			{
				MaxBone = Pair.Key;
				MaxWeight = Pair.Value;
			}
		}

		// check if the highest weight bone is in BonesToConsider
		if (BonesToConsider.Contains(MaxBone))
		{
			VerticesToConsider.Add(VertexIndex);
		}
	}
	
	// get the component space bone transform for the ref pose
	const FRetargetSkeleton& Skeleton = Processor->GetSkeleton(SourceOrTarget);
	const TArray<FTransform>& RefPoseLocal = Skeleton.SkeletalMesh->GetRefSkeleton().GetRefBonePose();
	TArray<FTransform> ComponentSpaceRefPoseBoneTransforms;
	FAnimationRuntime::FillUpComponentSpaceTransforms(
		Skeleton.SkeletalMesh->GetRefSkeleton(),
		RefPoseLocal,
		ComponentSpaceRefPoseBoneTransforms);

	// calculate the deformed vertex positions in the current retarget pose
	GetDeformedVertexPositions(
		VerticesToConsider,
		VertexRefPose,
		ComponentSpaceRefPoseBoneTransforms,
		Skeleton.RetargetPoses.GetGlobalRetargetPose(),
		SkinWeights,
		SourceOrTarget,
		OutPoints);
}

void FRetargetAutoPoseGenerator::GetAllChildrenOfBone(
	const FName BoneName,
	const ERetargetSourceOrTarget SourceOrTarget,
	TArray<FName>& OutAllChildren) const
{
	const FRetargetSkeleton& Skeleton = Processor->GetSkeleton(SourceOrTarget);
	const int32 BoneIndex = Skeleton.FindBoneIndexByName(BoneName);
	TArray<int32> ChildIndices;
	Skeleton.GetChildrenIndicesRecursive(BoneIndex, ChildIndices);
	for (const int32 ChildBoneIndex : ChildIndices)
	{
		OutAllChildren.Add(Skeleton.BoneNames[ChildBoneIndex]);	
	}
}

void FRetargetAutoPoseGenerator::GetRefPoseVerticesAndWeights(
	const ERetargetSourceOrTarget SourceOrTarget,
	TArray<TMap<FName, float>>& OutWeights,
	TArray<FVector>& OutPositions) const
{
	OutWeights.Reset();
	OutPositions.Reset();
	
	const FRetargetSkeleton& Skeleton = Processor->GetSkeleton(SourceOrTarget);
	const USkeletalMesh* Mesh = Skeleton.SkeletalMesh;
	constexpr int32 LODIndex = 0;
	const FMeshDescription *MeshDescription = Mesh->GetMeshDescription(LODIndex);
	if (!MeshDescription || MeshDescription->IsEmpty())
	{
		return;
	}
	
	const FSkeletalMeshConstAttributes MeshAttribs(*MeshDescription);
	const FSkinWeightsVertexAttributesConstRef VertexSkinWeights = MeshAttribs.GetVertexSkinWeights();
	const int32 NumVertices = MeshDescription->Vertices().Num();
	OutWeights.SetNum(NumVertices);
	OutPositions.SetNum(NumVertices);
	const FReferenceSkeleton& RefSkeleton = Mesh->GetRefSkeleton();
	ParallelFor(NumVertices,[RefSkeleton, &VertexSkinWeights, &OutWeights, &MeshDescription, &OutPositions](int32 VertexIndex)
	{
		const FVertexID VertexID(VertexIndex);

		// get weights of vertex
		int32 InfluenceIndex = 0;
		for (UE::AnimationCore::FBoneWeight BoneWeight: VertexSkinWeights.Get(VertexID))
		{
			const int32 BoneIndex = BoneWeight.GetBoneIndex();
			const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
			const float Weight = BoneWeight.GetWeight();
			OutWeights[VertexIndex].Add(BoneName, Weight);
			++InfluenceIndex;
		}

		// get position of vertex
		const FVector Position = static_cast<FVector>(MeshDescription->GetVertexPosition(VertexID));
		OutPositions[VertexIndex] = Position;
	});
}

void FRetargetAutoPoseGenerator::GetDeformedVertexPositions(
	const TArray<int32>& VerticesToDeform,
	const TArray<FVector>& VertexRefPose,
	const TArray<FTransform>& RefPoseBoneTransforms,
	const TArray<FTransform>& CurrentBoneTransforms,
	const TArray<TMap<FName, float>>& SkinWeights,
	const ERetargetSourceOrTarget SourceOrTarget,
	TArray<FVector>& OutDeformedPositions) const
{
	OutDeformedPositions.Reset(VerticesToDeform.Num());
	
	// generate map of bone name to bone index for fast look-up
	TMap<FName, int32> BoneNameToIndex;
	const FRetargetSkeleton& Skeleton = Processor->GetSkeleton(SourceOrTarget);
	for (int32 BoneIndex=0; BoneIndex<Skeleton.BoneNames.Num(); ++BoneIndex)
	{
		BoneNameToIndex.Add(Skeleton.BoneNames[BoneIndex], BoneIndex);
	}

	// generate deformed vertex position for each vertex we care about
	for (const int32 VertexIndex : VerticesToDeform)
	{
		const FVector& RefVertexPosition = VertexRefPose[VertexIndex];
		FVector DeformedPosition = FVector::ZeroVector;

		const TMap<FName, float>& VertexWeights = SkinWeights[VertexIndex];
		for (const TTuple<FName, float>& Pair : VertexWeights)
		{
			const int32 BoneIndex = BoneNameToIndex[Pair.Key];
			const float Weight = Pair.Value;
			FVector VertexInBoneSpace = RefPoseBoneTransforms[BoneIndex].Inverse().TransformPosition(RefVertexPosition);
			FVector BoneDeformedVertex = CurrentBoneTransforms[BoneIndex].TransformPosition(VertexInBoneSpace);
			DeformedPosition += BoneDeformedVertex * Weight;
		}

		OutDeformedPositions.Add(DeformedPosition);
	}
}

bool FRetargetAutoPoseGenerator::IsARetargetLeaf(
	const FName ChainName, 
	const float Param,
	ERetargetSourceOrTarget SourceOrTarget) const
{
	const FName BoneName = Processor->GetClosestBoneToParam(ChainName, Param, SourceOrTarget);
	TArray<FName> AllChildren;
	GetAllChildrenOfBone(BoneName, SourceOrTarget, AllChildren);
	for (const FName Child : AllChildren)
	{
		if (Processor->IsBoneMapped(Child, SourceOrTarget))
		{
			return false;
		}
	}
	return true;
}

bool FRetargetAutoPoseGenerator::CheckReadyToAlignBones() const
{
	// this should never happen
	if (!Controller.IsValid())
	{
		return false;
	}

	if (!Processor)
	{
		return false;
	}

	// initialize retarget processor with source and target skeletal meshes
	// NOTE: we have to initialize a processor to "resolve" the retarget asset setup onto actual skeletons to do the alignment
	FRetargetProfile RetargetProfile;
	RetargetProfile.FillProfileWithAssetSettings(Controller->GetAsset());
	FRetargetInitParameters Params;
	Params.SourceSkeletalMesh = Controller->GetPreviewMesh(ERetargetSourceOrTarget::Source);
	Params.TargetSkeletalMesh = Controller->GetPreviewMesh(ERetargetSourceOrTarget::Target);
	Params.RetargeterAsset = Controller->GetAsset();
	Params.CustomProfile = &RetargetProfile;
	Params.bSuppressWarnings = true;
	Processor->Initialize(Params);
	
	// can't auto align until processor is initialized because we need a fully resolved chain mapping on source/target
	// this could happen if the user attempts to edit the retarget pose on an incompatible IK Rig
	if (!Processor->IsInitialized())
	{
		return false;
	}

	return true;
}

void FRetargetAutoPoseGenerator::ApplyWorldRotationToRetargetPose(
	const FQuat& GlobalDeltaRotation,
	const FName BoneToAffect,
	const ERetargetSourceOrTarget SourceOrTarget) const
{
	const FRetargetSkeleton& Skeleton =  Processor->GetSkeleton(SourceOrTarget);
	const FReferenceSkeleton& RefSkeleton = Skeleton.SkeletalMesh->GetRefSkeleton();
	const int32 BoneIndex = Skeleton.FindBoneIndexByName(BoneToAffect);

	// get the curren parent global transform
	FTransform ParentGlobalTransform = FTransform::Identity;
	const int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
	if (ParentIndex != INDEX_NONE)
	{
		ParentGlobalTransform = Skeleton.RetargetPoses.GetGlobalRetargetPose()[ParentIndex];
	}

	// get the current global transform with no delta's at all
	FTransform LocalRefTransform = RefSkeleton.GetRefBonePose()[BoneIndex];
	FTransform CurrentGlobalTransformNoDelta = LocalRefTransform * ParentGlobalTransform;
				
	// get the stored local rotation offset from the retarget pose
	const FQuat LocalRotationDeltaFromPose = Controller->GetRotationOffsetForRetargetPoseBone(BoneToAffect, SourceOrTarget);
	// apply the retarget pose to get the global rotation w/ retarget pose
	// TODO get this from retarget pose??
	FQuat GlobalRetargetPoseRotation = CurrentGlobalTransformNoDelta.GetRotation() * LocalRotationDeltaFromPose;
	// get global delta from the retarget pose
	FQuat GlobalDeltaRotationFromRetargetPose = GlobalRetargetPoseRotation * CurrentGlobalTransformNoDelta.GetRotation().Inverse();
	// combine it with the new delta from the alignment
	FQuat NewGlobalDeltaRotation = GlobalDeltaRotation * GlobalDeltaRotationFromRetargetPose;

	// convert world space delta quaternion to bone-space
	const FVector RotationAxis = NewGlobalDeltaRotation.GetRotationAxis();
	const FVector UnRotatedAxis = CurrentGlobalTransformNoDelta.InverseTransformVector(RotationAxis);
	FQuat FinalLocalDeltaRotation = FQuat(UnRotatedAxis, NewGlobalDeltaRotation.GetAngle());

	// store the new rotation in the retarget pose
	Controller->SetRotationOffsetForRetargetPoseBone(BoneToAffect, FinalLocalDeltaRotation, SourceOrTarget);

	// regenerate the retarget pose in the currently running processor so it reflects the latest change
	const FName CurrentPoseName = Controller->GetCurrentRetargetPoseName(SourceOrTarget);
	Processor->UpdateRetargetPoseAtRuntime(CurrentPoseName, SourceOrTarget);
}

#undef LOCTEXT_NAMESPACE
