// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/IKRetargeterThumbnailRenderer.h"

#include "SceneView.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/SkeletalMeshActor.h"
#include "Engine/SkeletalMesh.h"
#include "RetargetEditor/IKRetargeterController.h"
#include "Retargeter/IKRetargeter.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRetargeterThumbnailRenderer)

FIKRetargeterThumbnailScene::FIKRetargeterThumbnailScene()
{
	bForceAllUsedMipsResident = false;

	FActorSpawnParameters SpawnInfo;
	SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnInfo.bNoFail = true;
	SpawnInfo.ObjectFlags = RF_Transient;
	
	SourceActor = GetWorld()->SpawnActor<ASkeletalMeshActor>(SpawnInfo);
	SourceActor->SetActorEnableCollision(false);
	
	TargetActor = GetWorld()->SpawnActor<ASkeletalMeshActor>(SpawnInfo);
	TargetActor->SetActorEnableCollision(false);
}

void FIKRetargeterThumbnailScene::SetSkeletalMeshes(USkeletalMesh* SourceMesh, USkeletalMesh* TargetMesh) const
{
	auto AssignMeshToActor = [this](USkeletalMesh* InMesh, ERetargetSourceOrTarget SourceOrTarget, bool bApplyOffset)
	{
		const bool bIsSourceMesh = SourceOrTarget == ERetargetSourceOrTarget::Source;
		ASkeletalMeshActor* PreviewActor = bIsSourceMesh ? SourceActor : TargetActor;
		PreviewActor->GetSkeletalMeshComponent()->OverrideMaterials.Empty();
		PreviewActor->GetSkeletalMeshComponent()->SetSkeletalMesh(InMesh, false);

		if (InMesh)
		{
			// center the mesh at the world origin
			PreviewActor->SetActorLocation(FVector(0,0,0), false);
			PreviewActor->GetSkeletalMeshComponent()->UpdateBounds();

			// X offset it sideways to put source/target side-by-side
			FVector Offset = FVector::ZeroVector;
			if (bApplyOffset)
			{
				const FBoxSphereBounds Bounds = InMesh->GetImportedBounds();
				const double HalfWidth = (Bounds.BoxExtent.X - Bounds.Origin.X) * 0.5;
				Offset.X = bIsSourceMesh ? -HalfWidth : HalfWidth;
			}
		
			// Z offset to put it on top of the plane
			Offset.Z = GetBoundsZOffset(PreviewActor->GetSkeletalMeshComponent()->Bounds);
			const FVector MoveToOrigin = -PreviewActor->GetSkeletalMeshComponent()->Bounds.Origin;
			PreviewActor->SetActorLocation(MoveToOrigin + Offset, false);
			PreviewActor->GetSkeletalMeshComponent()->UpdateBounds();
			PreviewActor->GetSkeletalMeshComponent()->RecreateRenderState_Concurrent();
		}
		else
		{
			PreviewActor->GetSkeletalMeshComponent()->ClearAnimScriptInstance();
		}
	};

	const bool bApplyOffset = SourceMesh && TargetMesh;
	AssignMeshToActor(SourceMesh, ERetargetSourceOrTarget::Source, bApplyOffset);
	AssignMeshToActor(TargetMesh, ERetargetSourceOrTarget::Target, bApplyOffset);
}

void FIKRetargeterThumbnailScene::GetViewMatrixParameters(
	const float InFOVDegrees,
	FVector& OutOrigin,
	float& OutOrbitPitch,
	float& OutOrbitYaw, 
	float& OutOrbitZoom) const
{
	const USkeletalMeshComponent* SourceComponent = SourceActor->GetSkeletalMeshComponent();
	const USkeletalMeshComponent* TargetComponent = TargetActor->GetSkeletalMeshComponent();
	check(SourceComponent);
	check(TargetComponent);

	// get bounds of both meshes combined
	const FBoxSphereBounds SourceBounds = SourceComponent->Bounds;
	const FBoxSphereBounds TargetBounds = TargetComponent->Bounds;
	const FBoxSphereBounds TotalBounds = TargetBounds + SourceBounds;

	// calc target distance
	const float HalfBoundSize = static_cast<float>(TotalBounds.GetBox().GetSize().Size()) * 0.5f;
	const float HalfFOVRadians = FMath::DegreesToRadians<float>(InFOVDegrees) * 0.5f;
	const float TargetDistance = HalfBoundSize / FMath::Tan(HalfFOVRadians);

	// get thumbnail info from mesh (preferring target if available)
	const USceneThumbnailInfo* ThumbnailInfo = nullptr;
	if(const USkeletalMesh* TargetMesh = TargetComponent->GetSkeletalMeshAsset())
	{
		ThumbnailInfo = Cast<USceneThumbnailInfo>(TargetMesh->GetThumbnailInfo());
	}
	else if (const USkeletalMesh* SourceMesh = SourceComponent->GetSkeletalMeshAsset())
	{
		ThumbnailInfo = Cast<USceneThumbnailInfo>(SourceMesh->GetThumbnailInfo());
	}
	else
	{
		ThumbnailInfo = USceneThumbnailInfo::StaticClass()->GetDefaultObject<USceneThumbnailInfo>();
	}

	if (ThumbnailInfo)
	{
		OutOrigin = -TotalBounds.GetBox().GetCenter();
		OutOrbitPitch = ThumbnailInfo->OrbitPitch;
		OutOrbitYaw = ThumbnailInfo->OrbitYaw;
		OutOrbitZoom = TargetDistance + ThumbnailInfo->OrbitZoom;
	}
}

void UIKRetargeterThumbnailRenderer::BeginDestroy()
{
	ThumbnailSceneCache.Clear();

	Super::BeginDestroy();
}

bool UIKRetargeterThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return HasSourceOrTargetMesh(Object);
}

void UIKRetargeterThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	TSharedRef<FIKRetargeterThumbnailScene> ThumbnailScene = ThumbnailSceneCache.EnsureThumbnailScene(Object);

	USkeletalMesh* SourceMesh = GetPreviewMeshFromAsset(Object, ERetargetSourceOrTarget::Source);
	USkeletalMesh* TargetMesh = GetPreviewMeshFromAsset(Object, ERetargetSourceOrTarget::Target);

	ThumbnailScene->SetSkeletalMeshes(SourceMesh, TargetMesh);

	FSceneViewFamilyContext ViewFamily( FSceneViewFamily::ConstructionValues( RenderTarget, ThumbnailScene->GetScene(), FEngineShowFlags(ESFIM_Game) )
	.SetTime(UThumbnailRenderer::GetTime())
	.SetAdditionalViewFamily(bAdditionalViewFamily));

	ViewFamily.EngineShowFlags.DisableAdvancedFeatures();
	ViewFamily.EngineShowFlags.MotionBlur = 0;
	ViewFamily.EngineShowFlags.LOD = 0;

	RenderViewFamily(Canvas, &ViewFamily, ThumbnailScene->CreateView(&ViewFamily, X, Y, Width, Height));
	
	ThumbnailScene->SetSkeletalMeshes(nullptr, nullptr);
}

EThumbnailRenderFrequency UIKRetargeterThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	const bool bHasSourceOrTarget = HasSourceOrTargetMesh(Object);
	return bHasSourceOrTarget ? EThumbnailRenderFrequency::Realtime : EThumbnailRenderFrequency::OnPropertyChange;
}

USkeletalMesh* UIKRetargeterThumbnailRenderer::GetPreviewMeshFromAsset(UObject* Object, ERetargetSourceOrTarget SourceOrTarget) const
{
	const UIKRetargeter* InRetargeter = Cast<UIKRetargeter>(Object);
	if (!InRetargeter)
	{
		return nullptr;
	}

	const UIKRetargeterController* Controller = UIKRetargeterController::GetController(InRetargeter);
	return Controller->GetPreviewMesh(SourceOrTarget);
}

bool UIKRetargeterThumbnailRenderer::HasSourceOrTargetMesh(UObject* Object) const
{
	const USkeletalMesh* SourceMesh = GetPreviewMeshFromAsset(Object, ERetargetSourceOrTarget::Source);
	const USkeletalMesh* TargetMesh = GetPreviewMeshFromAsset(Object, ERetargetSourceOrTarget::Target);
	return SourceMesh || TargetMesh;
}
