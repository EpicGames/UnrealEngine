// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/SIKRetargetAssetBrowser.h"

#include "SPositiveActionButton.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/PoseAsset.h"

#include "RetargetEditor/IKRetargetBatchOperation.h"
#include "RetargetEditor/IKRetargetEditorController.h"
#include "RetargetEditor/SRetargetAnimAssetsWindow.h"
#include "Retargeter/IKRetargeter.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Animation/Skeleton.h"
#include "Toolkits/GlobalEditorCommonCommands.h"

#define LOCTEXT_NAMESPACE "IKRetargeterAssetBrowser"

void SIKRetargetAssetBrowser::Construct(
	const FArguments& InArgs,
	TSharedRef<FIKRetargetEditorController> InEditorController)
{
	EditorController = InEditorController;
	EditorController.Pin()->SetAssetBrowserView(SharedThis(this));
	CommandList = MakeShared<FUICommandList>();

	const FGlobalEditorCommonCommands& GlobalCommands = FGlobalEditorCommonCommands::Get();
	CommandList->MapAction(
		GlobalCommands.FindInContentBrowser,
		FExecuteAction::CreateSP(this, &SIKRetargetAssetBrowser::ExecuteFindInContentBrowserAction),
		FCanExecuteAction::CreateSP(this, &SIKRetargetAssetBrowser::CanExecuteFindInContentBrowserAction)
	);
	
	ChildSlot
    [
        SNew(SVerticalBox)
        +SVerticalBox::Slot()
        .AutoHeight()
        [
			SNew(SPositiveActionButton)
			.IsEnabled(this, &SIKRetargetAssetBrowser::IsExportButtonEnabled)
			.Icon(FAppStyle::Get().GetBrush("Icons.Save"))
			.Text(LOCTEXT("ExportButtonLabel", "Export Selected Animations"))
			.ToolTipText(LOCTEXT("ExportButtonToolTip", "Generate new retargeted sequence assets on target skeletal mesh (uses current retargeting configuration)."))
			.OnClicked(this, &SIKRetargetAssetBrowser::OnExportButtonClicked)
        ]        

		+SVerticalBox::Slot()
		[
			SAssignNew(AssetBrowserBox, SBox)
		]
		
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SPositiveActionButton)
			.IsEnabled(this, &SIKRetargetAssetBrowser::IsPlayRefPoseEnabled)
			.Icon(FAppStyle::Get().GetBrush("GenericStop"))
			.Text(LOCTEXT("PlayRefPoseLabel", "Play Ref Pose"))
			.ToolTipText(LOCTEXT("PlayRefPoseTooltip", "Stop playback and set source to reference pose."))
			.OnClicked(this, &SIKRetargetAssetBrowser::OnPlayRefPoseClicked)
		]
    ];

	RefreshView();
}

void SIKRetargetAssetBrowser::RefreshView()
{
	// setup filtering
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimMontage::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UPoseAsset::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimComposite::StaticClass()->GetClassPathName());
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = true;
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::ItemDiskSize.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::VirtualizedData.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("RevisionControl"));
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SIKRetargetAssetBrowser::OnShouldFilterAsset);
	AssetPickerConfig.DefaultFilterMenuExpansion = EAssetTypeCategories::Animation;
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetSelected::CreateSP(this, &SIKRetargetAssetBrowser::OnAssetDoubleClicked);
	AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SIKRetargetAssetBrowser::OnGetAssetContextMenu);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = false;

	// hide all asset registry columns by default (we only really want the name and path)
	UObject* AnimSequenceDefaultObject = UAnimSequence::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(AnimSequenceDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	AnimSequenceDefaultObject->GetAssetRegistryTags(TagsContext);
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		AssetPickerConfig.HiddenColumnNames.Add(TagPair.Key.ToString());
	}

	// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Has Virtualized Data"));

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	AssetBrowserBox->SetContent(ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig));
}

FReply SIKRetargetAssetBrowser::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SBox::OnKeyDown(InGeometry, InKeyEvent);
}

void SIKRetargetAssetBrowser::ExecuteFindInContentBrowserAction()
{
	if (!GetCurrentSelectionDelegate.IsBound())
	{
		return;
	}

	const TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets(SelectedAssets);
}

bool SIKRetargetAssetBrowser::CanExecuteFindInContentBrowserAction()
{
	return GetCurrentSelectionDelegate.IsBound() && !GetCurrentSelectionDelegate.Execute().IsEmpty();
}

TSharedPtr<SWidget> SIKRetargetAssetBrowser::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets) const
{
	if (SelectedAssets.IsEmpty())
	{
		return nullptr;
	}

	FMenuBuilder MenuBuilder(true, CommandList);
	MenuBuilder.BeginSection(TEXT("Asset"), LOCTEXT("AssetSectionLabel", "Asset"));
	{
		const TSharedPtr<FUICommandInfo> FindInContentBrowserCommand = FGlobalEditorCommonCommands::Get().FindInContentBrowser;
		checkf(FindInContentBrowserCommand.IsValid(), TEXT("FGlobalEditorCommonCommands::FindInContentBrowser command not found!"));
		MenuBuilder.AddMenuEntry(FindInContentBrowserCommand);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FReply SIKRetargetAssetBrowser::OnExportButtonClicked()
{
	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		checkNoEntry();
		return FReply::Handled();
	}
	
	// assemble the context for the assets we want to batch duplicate/retarget
	FIKRetargetBatchOperationContext BatchContext;
	BatchContext.NameRule.FolderPath = PrevBatchOutputPath;
	BatchContext.SourceMesh = Controller->GetSkeletalMesh(ERetargetSourceOrTarget::Source);
	BatchContext.TargetMesh = Controller->GetSkeletalMesh(ERetargetSourceOrTarget::Target);
	BatchContext.IKRetargetAsset = Controller->AssetController->GetAsset();
	
	// get the export path from user
	const TSharedRef<SBatchExportPathDialog> PathDialog = SNew(SBatchExportPathDialog).BatchContext(&BatchContext).ExportRetargetAssets(false);
	if(PathDialog->ShowModal() == EAppReturnType::Cancel)
	{
		return FReply::Handled();
	}

	// store path for next time
	PrevBatchOutputPath = BatchContext.NameRule.FolderPath;

	// get the export options from user
	const TSharedRef<SBatchExportOptionsDialog> OptionsDialog = SNew(SBatchExportOptionsDialog).BatchContext(&BatchContext);
	if(OptionsDialog->ShowModal() == EAppReturnType::Cancel)
	{
		return FReply::Handled();
	}

	// add selected assets to dup/retarget
	TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
	for (const FAssetData& Asset : SelectedAssets)
	{
		if (UObject* Object = Asset.GetAsset())
		{
			BatchContext.AssetsToRetarget.Add(Object);
		}
	}

	// apply all active-by-default override sets from the retarget asset
	if (IsValid(BatchContext.IKRetargetAsset))
	{
		BatchContext.OverrideSetNames = BatchContext.IKRetargetAsset->GetOverrideSetsToApply();
	}

	// run the batch retarget
	const TStrongObjectPtr<UIKRetargetBatchOperation> BatchOperation(NewObject<UIKRetargetBatchOperation>());
	BatchOperation->RunRetarget(BatchContext);
	
	return FReply::Handled();
}

bool SIKRetargetAssetBrowser::IsExportButtonEnabled() const
{
	if (!EditorController.Pin().IsValid())
	{
		return false; // editor in bad state
	}

	const FIKRetargetProcessor* Processor = EditorController.Pin()->GetRetargetProcessor();
	if (!Processor)
	{
		return false; // no retargeter running
	}

	if (!Processor->IsInitialized())
	{
		return false; // retargeter not loaded and valid
	}

	TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
	if (SelectedAssets.IsEmpty())
	{
		return false; // nothing selected
	}

	return true;
}

FReply SIKRetargetAssetBrowser::OnPlayRefPoseClicked()
{
	FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		checkNoEntry();
		return FReply::Handled();
	}

	Controller->SetRetargeterMode(ERetargeterOutputMode::RunRetarget);
	Controller->PlaybackManager->StopPlayback();
	return FReply::Handled();
}

bool SIKRetargetAssetBrowser::IsPlayRefPoseEnabled() const
{
	FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		checkNoEntry();
		return false;
	}
	
	return Controller->GetSkeletalMesh(ERetargetSourceOrTarget::Source) != nullptr;
}

void SIKRetargetAssetBrowser::OnAssetDoubleClicked(const FAssetData& AssetData)
{
	if (!AssetData.GetAsset())
	{
		return;
	}
	
	UAnimationAsset* NewAnimationAsset = Cast<UAnimationAsset>(AssetData.GetAsset());
	if (NewAnimationAsset && EditorController.Pin().IsValid())
	{
		EditorController.Pin()->PlaybackManager->PlayAnimationAsset(NewAnimationAsset);
	}
}

bool SIKRetargetAssetBrowser::OnShouldFilterAsset(const struct FAssetData& AssetData)
{
	// is this an animation asset?
	if (!AssetData.IsInstanceOf(UAnimationAsset::StaticClass()))
	{
		return true;
	}
	
	// controller setup
	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return true;
	}

	// get source skeleton
	const USkeleton* DesiredSkeleton = Controller->GetSkeleton(ERetargetSourceOrTarget::Source);
	if (!DesiredSkeleton)
	{
		return true;
	}

	return !DesiredSkeleton->IsCompatibleForEditor(AssetData);
}

#undef LOCTEXT_NAMESPACE
