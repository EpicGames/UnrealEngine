// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/SIKRetargetChainMapList.h"

#include "ScopedTransaction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Retargeter/IKRetargeter.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Widgets/Input/SSearchBox.h"
#include "Retargeter/IKRetargetSettings.h"
#include "RigEditor/IKRigController.h"
#include "RigEditor/IKRigStructViewer.h"

#define LOCTEXT_NAMESPACE "SIKRigRetargetChains"

static const FName ColumnId_TargetChainLabel( "Target Bone Chain" );
static const FName ColumnId_SourceChainLabel( "Source Bone Chain" );
static const FName ColumnId_IKGoalNameLabel( "Target IK Goal" );
static const FName ColumnId_ResetLabel( "Reset" );

TSharedRef<ITableRow> FRetargetChainMapElement::MakeListRowWidget(
	const TSharedRef<STableViewBase>& InOwnerTable,
	TSharedRef<FRetargetChainMapElement> InChainElement,
	TSharedPtr<SIKRetargetChainMapList> InChainList)
{
	return SNew(SIKRetargetChainMapRow, InOwnerTable, InChainElement, InChainList);
}

void SIKRetargetChainMapRow::Construct(
	const FArguments& InArgs,
	const TSharedRef<STableViewBase>& InOwnerTableView,
	TSharedRef<FRetargetChainMapElement> InChainElement,
	TSharedPtr<SIKRetargetChainMapList> InChainList)
{
	ChainMapElement = InChainElement;
	ChainMapList = InChainList;

	// generate list of source chains
	// NOTE: cannot just use FName because "None" is considered a null entry and removed from ComboBox.
	SourceChainOptions.Reset();
	SourceChainOptions.Add(MakeShareable(new FString(TEXT("None"))));
	const UIKRigDefinition* SourceIKRig = ChainMapList.Pin()->Config.Controller->GetIKRig(ERetargetSourceOrTarget::Source);
	if (SourceIKRig)
	{
		const TArray<FBoneChain>& Chains = SourceIKRig->GetRetargetChains();
		for (const FBoneChain& BoneChain : Chains)
		{
			SourceChainOptions.Add(MakeShareable(new FString(BoneChain.ChainName.ToString())));
		}
	}

	SMultiColumnTableRow< FRetargetChainMapElementPtr >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef< SWidget > SIKRetargetChainMapRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == ColumnId_TargetChainLabel)
	{
		TSharedRef<SWidget> NewWidget =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 1.0f)
			[
				SNew(STextBlock)
				.Text(FText::FromName(ChainMapElement.Pin()->TargetChainName))
				.Font(FAppStyle::GetFontStyle(TEXT("BoldFont")))
			];
		return NewWidget;
	}

	if (ColumnName == ColumnId_SourceChainLabel)
	{
		TSharedRef<SWidget> NewWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.Padding(3.0f, 1.0f)
		[
			SNew(SSearchableComboBox)
			.OptionsSource(&SourceChainOptions)
			.IsEnabled_Lambda([this]()
			{
				return ChainMapList.Pin()->Config.bEnableChainMapping;
			})
			.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
			{
				return SNew(STextBlock).Text(FText::FromString(*InItem.Get()));
			})
			.OnSelectionChanged(this, &SIKRetargetChainMapRow::OnSourceChainComboSelectionChanged)
			[
				SNew(STextBlock)
				.Text(this, &SIKRetargetChainMapRow::GetSourceChainName)
			]
		];
		return NewWidget;
	}

	if (ColumnName == ColumnId_IKGoalNameLabel)
	{
		TSharedRef<SWidget> NewWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(3.0f, 1.0f)
		[
			SNew(STextBlock)
			.Text(GetTargetIKGoalName())
			.Font(FAppStyle::GetFontStyle(TEXT("BoldFont")))
		];
		return NewWidget;
	}

	if (ColumnName == ColumnId_ResetLabel)
	{
		TSharedRef<SWidget> NewWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(3.0f, 1.0f)
		[
			SNew(SButton)
			.OnClicked(this, &SIKRetargetChainMapRow::OnResetToDefaultClicked) 
			.Visibility(this, &SIKRetargetChainMapRow::GetResetToDefaultVisibility) 
			.ToolTipText(LOCTEXT("ResetChainToDefaultToolTip", "Reset Chain Settings to Default"))
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
			]
		];
		return NewWidget;
	}
	
	checkNoEntry();
	return SNullWidget::NullWidget;
}

FReply SIKRetargetChainMapRow::OnResetToDefaultClicked()
{
	ChainMapList.Pin()->ResetChainSettings(ChainMapElement.Pin()->TargetChainName);
	return FReply::Handled();
}

EVisibility SIKRetargetChainMapRow::GetResetToDefaultVisibility() const
{
	UIKRetargeterController* RetargeterController = ChainMapList.Pin()->Config.Controller.Get();
	if (!RetargeterController)
	{
		return EVisibility::Hidden; 
	}

	const FName TargetChainName = ChainMapElement.Pin()->TargetChainName;
	const FName OpName = ChainMapList.Pin()->Config.OpWithChainSettings;
	bool bAreSettingsAtDefault = RetargeterController->AreChainSettingsAtDefault(TargetChainName, OpName);
	return bAreSettingsAtDefault ? EVisibility::Hidden : EVisibility::Visible;
}

void SIKRetargetChainMapRow::OnSourceChainComboSelectionChanged(TSharedPtr<FString> InName, ESelectInfo::Type SelectInfo)
{
	UIKRetargeterController* RetargeterController = ChainMapList.Pin()->Config.Controller.Get();
	if (!RetargeterController)
	{
		return; 
	}

	const FName SourceChainName = FName(*InName.Get());
	const FName TargetChainName = ChainMapElement.Pin()->TargetChainName;
	const FName OpWithMappingName = ChainMapList.Pin()->Config.OpWithChainMapping;
	RetargeterController->SetSourceChain(SourceChainName, TargetChainName, OpWithMappingName);
}

FText SIKRetargetChainMapRow::GetSourceChainName() const
{
	UIKRetargeterController* RetargeterController = ChainMapList.Pin()->Config.Controller.Get();
	if (!RetargeterController)
	{
		return FText::FromName(NAME_None); 
	}

	const FName TargetChainName = ChainMapElement.Pin()->TargetChainName;
	const FName OpName = ChainMapList.Pin()->Config.OpWithChainMapping;
	const FName SourceChainName = RetargeterController->GetSourceChain(TargetChainName, OpName);
	return FText::FromName(SourceChainName);
}

FText SIKRetargetChainMapRow::GetTargetIKGoalName() const
{
	UIKRetargeterController* RetargeterController = ChainMapList.Pin()->Config.Controller.Get();
	if (!RetargeterController)
	{
		return FText(); 
	}

	const FName OpName = ChainMapList.Pin()->Config.OpWithChainMapping;
	const UIKRigDefinition* IKRig = RetargeterController->GetTargetIKRigForOp(OpName);
	if (!IKRig)
	{
		return FText(); 
	}

	const UIKRigController* RigController = UIKRigController::GetController(IKRig);
	const FBoneChain* Chain = RigController->GetRetargetChainByName(ChainMapElement.Pin()->TargetChainName);
	if (!Chain)
	{
		return FText(); 
	}

	if (Chain->IKGoalName == NAME_None)
	{
		return FText::FromString("");
	}
	
	return FText::FromName(Chain->IKGoalName);
}

void SIKRetargetChainMapList::Construct(const FArguments& InArgs)
{
	Config = InArgs._InChainMapListConfig;
	if (!ensure(Config.IsValid()))
	{
		return; // chain map editing must be associated with a particular op
	}
	
	// register callback to refresh when the retargeter is initialized (happens whenever IK Rigs are swapped or edited)
	TWeakPtr<SIKRetargetChainMapList> WeakSelf = StaticCastWeakPtr<SIKRetargetChainMapList>(AsWeak());
	Config.Controller->OnRetargeterNeedsInitialized().AddLambda([WeakSelf]()
	{
		if (WeakSelf.IsValid())
		{
			WeakSelf.Pin()->RefreshView();
		}
	});
	
	TextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));
	
	// create property view for the chain properties
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropertyModule.CreateDetailView(DetailsViewArgs);

	auto GenerateHeaderRow = [this]() -> TSharedPtr<SHeaderRow>
	{
		TSharedRef<SHeaderRow> HeaderRowWidget = SNew(SHeaderRow);
		
		HeaderRowWidget->AddColumn(SHeaderRow::Column(ColumnId_TargetChainLabel)
			.HAlignHeader(HAlign_Center)
			.DefaultLabel(LOCTEXT("TargetColumnLabel", "Target Chain"))
			.DefaultTooltip(LOCTEXT("TargetChainToolTip", "The chain on the target skeleton to copy animation TO.")));

		HeaderRowWidget->AddColumn(SHeaderRow::Column(ColumnId_IKGoalNameLabel)
			.HAlignHeader(HAlign_Center)
			.Visibility_Lambda([this]()
			{
				return Config.bEnableGoalColumn ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.DefaultLabel(LOCTEXT("IKColumnLabel", "Target IK Goal"))
			.DefaultTooltip(LOCTEXT("IKGoalToolTip",
    								"The IK Goal assigned to the target chain (if any). Note, this goal should be on the LAST bone in the chain.")));

		HeaderRowWidget->AddColumn(SHeaderRow::Column(ColumnId_SourceChainLabel)
			.HAlignHeader(HAlign_Center)
			.DefaultLabel(LOCTEXT("SourceColumnLabel", "Source Chain"))
			.DefaultTooltip(LOCTEXT("SourceChainToolTip", "The chain on the source skeleton to copy animation FROM.")));

		if (Config.ChainSettingsGetterFunc.IsSet())
		{
			HeaderRowWidget->AddColumn(SHeaderRow::Column(ColumnId_ResetLabel)
			.HAlignHeader(HAlign_Center)
			.FixedWidth(50.0f)
			.DefaultLabel(LOCTEXT("ResetColumnLabel", "Reset"))
			.DefaultTooltip(LOCTEXT("ResetChainToolTip", "Reset the settings for this chain on this op.")));
		}
			
		return HeaderRowWidget;
	};
	
	ChildSlot
    [
	    SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			// filter list text field
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSearchBox)
				.IsEnabled(this, &SIKRetargetChainMapList::IsListEnabled)
				.SelectAllTextWhenFocused(true)
				.OnTextChanged( this, &SIKRetargetChainMapList::OnFilterTextChanged )
				.HintText( LOCTEXT( "SearchBoxHint", "Filter Chain List...") )
			]

			// filter list options
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.f, 0.0))
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.ForegroundColor(FSlateColor::UseStyle())
				.ContentPadding(2.0f)
				.IsEnabled(this, &SIKRetargetChainMapList::IsListEnabled)
				.OnGetMenuContent( this, &SIKRetargetChainMapList::CreateFilterMenuWidget )
				.ToolTipText( LOCTEXT("ChainMapFilterToolTip", "Filter list of chain mappings."))
				.HasDownArrow(true)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]

			// chain mapping menu
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.f, 0.0))
			[
				SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.ForegroundColor(FSlateColor::UseStyle())
				.ContentPadding(2.0f)
				.Visibility_Lambda([this]()
				{
					const bool bIsVisible = IsChainMappingEnabled();
					return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.ToolTipText(LOCTEXT("AutoMapButtonToolTip", "Automatically assign source chains based on matching rule."))
				.OnGetMenuContent( this, &SIKRetargetChainMapList::CreateChainMapMenuWidget )
				.HasDownArrow(true)
				.ButtonContent()
				[
					SNew(STextBlock).Text(LOCTEXT("AutoMapButtonLabel", "Auto-Map Chains"))
				]
			]
		]

	    // chain list view
        +SVerticalBox::Slot()
        [
        	SNew(SBox)
			.MaxDesiredHeight(300.0f)
			[
				SAssignNew(ListView, SRetargetChainMapListViewType )
        		.ScrollbarVisibility(EVisibility::Visible)
				.SelectionMode(ESelectionMode::Multi)
				.IsEnabled(this, &SIKRetargetChainMapList::IsListEnabled)
				.ListItemsSource( &ListViewItems )
				.OnGenerateRow( this, &SIKRetargetChainMapList::MakeListRowWidget )
				.OnMouseButtonClick_Lambda([this](TSharedPtr<FRetargetChainMapElement> Item)
				{
					OnItemClicked(Item);
				})
				.HeaderRow
				(
					GenerateHeaderRow()
				)
			]
        ]
    	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			DetailsView.ToSharedRef()
		]
    ];

	RefreshView();
}

void SIKRetargetChainMapList::ClearSelection() const
{
	ListView->ClearSelection();
}

void SIKRetargetChainMapList::ResetChainSettings(const FName InTargetChainName) const
{
	Config.Controller->ResetChainSettingsToDefault(InTargetChainName, Config.OpWithChainSettings);

	// update wrappers to show new values
	for (TObjectPtr<UObject> Wrapper : AllStructWrappers)
	{
		if (UIKRigStructWrapperBase* WrapperBase = Cast<UIKRigStructWrapperBase>(Wrapper))
		{
			WrapperBase->UpdateWrapperStructWithLatestValues();
		}
	}
	
	DetailsView->ForceRefresh();
}

void SIKRetargetChainMapList::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(AllStructWrappers);
}

bool SIKRetargetChainMapList::IsListEnabled() const
{
	return Config.IsValid();
}

bool SIKRetargetChainMapList::IsChainMappingEnabled() const
{
	if (!Config.IsValid())
	{
		return false; 
	}

	if (!Config.bEnableChainMapping)
	{
		return false;
	}
	
	const FRetargetChainMapping* ChainMapping = Config.Controller->GetChainMapping(Config.OpWithChainMapping);
	if (!ChainMapping)
	{
		return false;
	}

	return ChainMapping->IsReady();
}

void SIKRetargetChainMapList::RefreshView()
{
	if (!Config.Controller.IsValid())
	{
		return; 
	}

	auto FilterString = [this](const FString& StringToTest) ->bool
	{
		return TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(StringToTest));
	};

	auto DoesChainHaveIK = [this](const FName TargetChainName) ->bool
	{
		const UIKRigDefinition* IKRig = Config.Controller->GetTargetIKRigForOp(Config.OpWithChainMapping);
		if (!IKRig)
		{
			return false;
		}

		const UIKRigController* RigController = UIKRigController::GetController(IKRig);
		const FBoneChain* Chain = RigController->GetRetargetChainByName(TargetChainName);
		if (!Chain)
		{
			return false;
		}
		
		return Chain->IKGoalName != NAME_None;
	};

	// record selected item to restore
	FName SelectedChainName = NAME_None;
	TArray<TSharedPtr<FRetargetChainMapElement>> SelectedItems = ListView->GetSelectedItems();
	if (!SelectedItems.IsEmpty())
	{
		SelectedChainName = SelectedItems[0].Get()->TargetChainName;	
	}

	// refresh items
	ListViewItems.Reset();

	// get the chain mapping for the op
	const FRetargetChainMapping* ChainMapping = Config.Controller->GetChainMapping(Config.OpWithChainMapping);
	if (!ChainMapping)
	{
		return; // op deleted, being cleaned up
	}
	
	if (!ChainMapping->HasAnyChains())
	{
		// must have at least a target IK rig assigned with some chains loaded
		return;
	}

	// add list item for each target chain
	static FName LiteralNone = FName("None");
	const TArray<FRetargetChainPair>& ChainMap = ChainMapping->GetChainPairs();
	for (const FRetargetChainPair& ChainPair : ChainMap)
	{
		const FName TargetChainName = ChainPair.TargetChainName;
		const FName SourceChainName = ChainPair.SourceChainName;

		// apply text filter to items
		if (!(TextFilter->GetFilterText().IsEmpty() ||
			FilterString(SourceChainName.ToString()) ||
			FilterString(TargetChainName.ToString())))
		{
			continue;
		}
		
		// apply "only IK" filter
		const bool bFilterNonIKChains = Config.Filter.bHideChainsWithoutIK || Config.Filter.bNeverShowChainsWithoutIK;
		if (bFilterNonIKChains && !DoesChainHaveIK(TargetChainName))
		{
			continue;
		}

		// apply "hide mapped chains" filter
		if (Config.Filter.bHideMappedChains && SourceChainName != LiteralNone)
		{
			continue;
		}
		
		// apply "hide un-mapped chains" filter
		if (Config.Filter.bHideUnmappedChains && SourceChainName == LiteralNone)
		{
			continue;
		}
		
		// create an item for this chain
		TSharedPtr<FRetargetChainMapElement> ChainItem = FRetargetChainMapElement::Make(TargetChainName);
		ListViewItems.Add(ChainItem);

		// restore selection
		if (ChainItem.Get()->TargetChainName == SelectedChainName)
		{
			ListView->SetSelection(ChainItem);
		}
	}

	ListView->RequestListRefresh();
}

TSharedRef<SWidget> SIKRetargetChainMapList::CreateFilterMenuWidget()
{
	const FUIAction FilterHideMappedAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			Config.Filter.bHideMappedChains = !Config.Filter.bHideMappedChains;
			RefreshView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return Config.Filter.bHideMappedChains;
		}));

	const FUIAction FilterOnlyUnMappedAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			Config.Filter.bHideUnmappedChains = !Config.Filter.bHideUnmappedChains;
			RefreshView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return Config.Filter.bHideUnmappedChains;
		}));

	const FUIAction FilterIKChainAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			Config.Filter.bHideChainsWithoutIK = !Config.Filter.bHideChainsWithoutIK;
			RefreshView();
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			return !Config.Filter.bNeverShowChainsWithoutIK;
		}),
		FIsActionChecked::CreateLambda([this]()
		{
			return Config.Filter.bHideChainsWithoutIK;
		}));
	
	static constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, MenuCommandList);

	MenuBuilder.BeginSection("Chain Map Filters", LOCTEXT("ChainMapFiltersSection", "Filter Chain Mappings"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HideMappedLabel", "Hide Mapped Chains"),
		LOCTEXT("HideMappedTooltip", "Hide chains mapped to a source chain."),
		FSlateIcon(),
		FilterHideMappedAction,
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HideUnMappedLabel", "Hide Unmapped Chains"),
		LOCTEXT("HideUnMappedTooltip", "Hide chains not mapped to a source chain."),
		FSlateIcon(),
		FilterOnlyUnMappedAction,
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HideNonIKLabel", "Hide Chains Without IK"),
		LOCTEXT("HideNonIKTooltip", "Hide chains not using IK."),
		FSlateIcon(),
		FilterIKChainAction,
		NAME_None,
		EUserInterfaceActionType::Check);
	
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Clear", LOCTEXT("ClearMapFiltersSection", "Clear"));
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearMapFilterLabel", "Clear Filters"),
		LOCTEXT("ClearMapFilterTooltip", "Clear all filters to show all chain mappings."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]
		{
			Config.Filter.Reset();
			RefreshView();
		})));

	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

void SIKRetargetChainMapList::OnFilterTextChanged(const FText& SearchText)
{
	TextFilter->SetFilterText(SearchText);
	RefreshView();
}

TSharedRef<ITableRow> SIKRetargetChainMapList::MakeListRowWidget(
	TSharedPtr<FRetargetChainMapElement> InElement,
    const TSharedRef<STableViewBase>& OwnerTable)
{
	return InElement->MakeListRowWidget(
		OwnerTable,
		InElement.ToSharedRef(),
		SharedThis(this));
}

void SIKRetargetChainMapList::OnItemClicked(TSharedPtr<FRetargetChainMapElement> InItem) 
{
	// get list of selected chains
	TArray<FName> SelectedChains;
	TArray<TSharedPtr<FRetargetChainMapElement>> SelectedItems = ListView.Get()->GetSelectedItems();
	for (const TSharedPtr<FRetargetChainMapElement>& Item : SelectedItems)
	{
		SelectedChains.Add(Item.Get()->TargetChainName);
	}

	// get list of settings to edit for the selected chains
	AllStructWrappers.Reset();
	if (Config.ChainSettingsGetterFunc.IsSet())
	{
		for (const FName SelectedChainName : SelectedChains)
		{
			UObject* StructWrapper = Config.ChainSettingsGetterFunc(SelectedChainName);
			if (IsValid(StructWrapper))
			{
				AllStructWrappers.Add(StructWrapper);
			}
		}
	}

	DetailsView->SetObjects(AllStructWrappers);
}

TSharedRef<SWidget> SIKRetargetChainMapList::CreateChainMapMenuWidget()
{
	const FUIAction MapAllByFuzzyNameAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			constexpr bool bForceRemap = true;
			AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);
		}),
		FCanExecuteAction(), FIsActionChecked());
	
	const FUIAction MapAllByExactNameAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			constexpr bool bForceRemap = true;
			AutoMapChains(EAutoMapChainType::Exact, bForceRemap);
		}),
		FCanExecuteAction(), FIsActionChecked());

	const FUIAction MapUnmappedByExactNameAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			constexpr bool bForceRemap = false;
			AutoMapChains(EAutoMapChainType::Exact, bForceRemap);
		}),
		FCanExecuteAction(), FIsActionChecked());

	const FUIAction MapUnmappedByFuzzyNameAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			constexpr bool bForceRemap = false;
			AutoMapChains(EAutoMapChainType::Fuzzy, bForceRemap);
		}),
		FCanExecuteAction(), FIsActionChecked());

	const FUIAction ClearAllMappingsAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			constexpr bool bForceRemap = true;
			AutoMapChains(EAutoMapChainType::Clear, bForceRemap);
		}),
		FCanExecuteAction(), FIsActionChecked());
	
	static constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, MenuCommandList);

	MenuBuilder.BeginSection("Auto-Map Chains Fuzzy", LOCTEXT("FuzzyNameSection", "Fuzzy Name Matching"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MapAllByNameFuzzyLabel", "Map All (Fuzzy)"),
		LOCTEXT("MapAllByNameFuzzyTooltip", "Map all chains to the source chain with the closest name (not necessarily exact)."),
		FSlateIcon(),
		MapAllByFuzzyNameAction,
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MapMissingByNameFuzzyLabel", "Map Only Empty (Fuzzy)"),
		LOCTEXT("MapMissingByNameFuzzyTooltip", "Map all unmapped chains to the source chain with the closest name (not necessarily exact)."),
		FSlateIcon(),
		MapUnmappedByFuzzyNameAction,
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.EndSection();
	MenuBuilder.BeginSection("Auto-Map Chains Exact", LOCTEXT("ExactNameSection", "Exact Name Matching"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MapAllByNameExactLabel", "Map All (Exact)"),
		LOCTEXT("MapAllByNameExactTooltip", "Map all chains with identical name. If no match found, does not change mapping."),
		FSlateIcon(),
		MapAllByExactNameAction,
		NAME_None,
		EUserInterfaceActionType::Button);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("MapMissingByNameExactLabel", "Map Only Empty (Exact)"),
		LOCTEXT("MapMissingByNameExactTooltip", "Map unmapped chains using identical name. If no match found, does not change mapping."),
		FSlateIcon(),
		MapUnmappedByExactNameAction,
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.EndSection();
	MenuBuilder.BeginSection("Clear", LOCTEXT("ClearMapSection", "Clear All"));
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearMapLabel", "Clear All Mappings"),
		LOCTEXT("ClearMapTooltip", "Map all chains to None."),
		FSlateIcon(),
		ClearAllMappingsAction,
		NAME_None,
		EUserInterfaceActionType::Button);

	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

void SIKRetargetChainMapList::AutoMapChains(const EAutoMapChainType AutoMapType, const bool bForceRemap)
{
	// TODO EditorController->ClearOutputLog();
	Config.Controller->AutoMapChains(AutoMapType, bForceRemap);
}

#undef LOCTEXT_NAMESPACE

