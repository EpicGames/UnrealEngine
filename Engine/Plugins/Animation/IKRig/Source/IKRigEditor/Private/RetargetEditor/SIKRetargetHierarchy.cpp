// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/SIKRetargetHierarchy.h"

#include "Preferences/PersonaOptions.h"
#include "RetargetEditor/SIKRetargetPoseEditor.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Engine/SkeletalMesh.h"
#include "SPositiveActionButton.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Widgets/Input/SSearchBox.h"

#define LOCTEXT_NAMESPACE "SIKRetargetHierarchy"

static const FName BoneColumnName(TEXT("BoneName"));
static const FName ChainColumnName(TEXT("RetargetChainName"));

FIKRetargetHierarchyElement::FIKRetargetHierarchyElement(
	const FName& InName,
	const TSharedRef<FIKRetargetEditorController>& InEditorController)
	: Key(FText::FromName(InName)),
	Name(InName),
	EditorController(InEditorController)
{}

FName FIKRetargetHierarchyElement::GetChainName() const
{
	TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller)
	{
		return NAME_None;
	}

	return Controller->GetChainDisplayNameForBone(Name, Controller->GetSourceOrTarget());
}

TSharedRef<SWidget> SIKRetargetHierarchyRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	const FName BoneName = WeakTreeElement.Pin()->Name;
	ERetargetSourceOrTarget CurrentSkeleton = EditorController.Pin()->GetSourceOrTarget();
	const bool bIsBoneRetargeted = EditorController.Pin()->IsBoneRetargeted(BoneName, CurrentSkeleton);
	
	// determine icon based on if bone is retargeted
	const FSlateBrush* Brush;
	if (bIsBoneRetargeted)
	{
		Brush = FAppStyle::Get().GetBrush("SkeletonTree.Bone");
	}
	else
	{
		Brush = FAppStyle::Get().GetBrush("SkeletonTree.BoneNonWeighted");
	}

	// determine text based on if bone is retargeted
	FTextBlockStyle NormalText = FIKRigEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("IKRig.Tree.NormalText");
	FTextBlockStyle ItalicText = FIKRigEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("IKRig.Tree.ItalicText");
	FSlateFontInfo TextFont;
	FSlateColor TextColor;
	if (bIsBoneRetargeted)
	{
		// elements connected to the selected solver are green
		TextFont = ItalicText.Font;
		TextColor = NormalText.ColorAndOpacity;
	}
	else
	{
		TextFont = NormalText.Font;
		TextColor = FLinearColor(0.2f, 0.2f, 0.2f, 0.5f);
	}
	
	if (ColumnName == BoneColumnName)
	{
		TSharedPtr< SHorizontalBox > RowBox;
		SAssignNew(RowBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SExpanderArrow, SharedThis(this) )
			.ShouldDrawWires(true)
		];

		RowBox->AddSlot()
		.MaxWidth(18)
		.FillWidth(1.0)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(Brush)
		];

		RowBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(FText::FromName(BoneName))
			.Font(TextFont)
			.ColorAndOpacity(TextColor)
		];

		return RowBox.ToSharedRef();
	}

	if (ColumnName == ChainColumnName)
	{
		TSharedPtr< SHorizontalBox > RowBox;
		SAssignNew(RowBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text_Lambda([this](){ return FText::FromName(WeakTreeElement.Pin()->GetChainName()); })
			.Font(TextFont)
			.ColorAndOpacity(TextColor)
		];

		return RowBox.ToSharedRef();
	}
	
	return SNullWidget::NullWidget;
}

void SIKRetargetHierarchy::Construct(
	const FArguments& InArgs,
	TSharedRef<FIKRetargetEditorController> InEditorController)
{
	EditorController = InEditorController;
	EditorController.Pin()->SetHierarchyView(SharedThis(this));
	CommandList = MakeShared<FUICommandList>();

	TextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SBox)
			.Padding(2.0f)
			.HAlign(HAlign_Center)
			[
				SNew(SSegmentedControl<ERetargetSourceOrTarget>)
				.Value_Lambda([this]()
				{
					return EditorController.Pin()->GetSourceOrTarget();
				})
				.OnValueChanged_Lambda([this](ERetargetSourceOrTarget Mode)
				{
					EditorController.Pin()->SetSourceOrTargetMode(Mode);
				})
				+SSegmentedControl<ERetargetSourceOrTarget>::Slot(ERetargetSourceOrTarget::Source)
				.Text(LOCTEXT("SourceSkeleton", "Source"))
				+ SSegmentedControl<ERetargetSourceOrTarget>::Slot(ERetargetSourceOrTarget::Target)
				.Text(LOCTEXT("TargetSkeleton", "Target"))
			]
		]

		+SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SSeparator)
		]
		
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SAssignNew(EditPoseView, SIKRetargetPoseEditor, InEditorController)
		]

		+SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SSeparator)
		]

		+SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSearchBox)
				.SelectAllTextWhenFocused(true)
				.OnTextChanged( this, &SIKRetargetHierarchy::OnFilterTextChanged )
				.HintText( LOCTEXT( "SearchBoxHint", "Filter Hierarchy Tree...") )
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.f, 0.0))
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.ForegroundColor(FSlateColor::UseStyle())
				.ContentPadding(2.0f)
				.OnGetMenuContent( this, &SIKRetargetHierarchy::CreateFilterMenuWidget )
				.ToolTipText( LOCTEXT( "HierarchyFilterToolTip", "Options to filter the hierarchy tree.") )
				.HasDownArrow(true)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]

		+SVerticalBox::Slot()
		.Padding(2.0f)
		[
			SNew(SBorder)
			.Padding(2.0f)
			.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
			[
				SAssignNew(TreeView, SIKRetargetHierarchyTreeView)
				.TreeItemsSource(&RootElements)
				.SelectionMode(ESelectionMode::Multi)
				.OnGenerateRow_Lambda( [this](
					TSharedPtr<FIKRetargetHierarchyElement> InItem,
					const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
				{
					return SNew(SIKRetargetHierarchyRow, OwnerTable)
					.EditorController(EditorController.Pin())
					.TreeElement(InItem);
				})
				.OnGetChildren(this, &SIKRetargetHierarchy::HandleGetChildrenForTree)
				.OnSelectionChanged(this, &SIKRetargetHierarchy::OnSelectionChanged)
				.OnMouseButtonDoubleClick(this, &SIKRetargetHierarchy::OnItemDoubleClicked)
				.OnSetExpansionRecursive(this, &SIKRetargetHierarchy::OnSetExpansionRecursive)
				.HighlightParentNodesForSelection(false)
				.HeaderRow
				(
					SNew(SHeaderRow)
					
					+ SHeaderRow::Column(BoneColumnName)
					.DefaultLabel(LOCTEXT("RetargetBoneNameLabel", "Bone Name"))
					.FillWidth(0.7f)
						
					+ SHeaderRow::Column(ChainColumnName)
					.DefaultLabel(LOCTEXT("RetargetChainNameLabel", "Retarget Chain"))
					.FillWidth(0.3f)
				)
			]
		]
	];

	constexpr bool IsInitialSetup = true;
	RefreshTreeView(IsInitialSetup);
}

void SIKRetargetHierarchy::ShowItemAfterSelection(FName ItemName)
{
	TSharedPtr<FIKRetargetHierarchyElement> ItemToShow;
	for (const TSharedPtr<FIKRetargetHierarchyElement>& Element : AllElements)
	{
		if (Element.Get()->Name == ItemName)
		{
			ItemToShow = Element;
		}
	}

	if (!ItemToShow.IsValid())
	{
		return;
	}
	
	if(GetDefault<UPersonaOptions>()->bExpandTreeOnSelection)
	{
		TWeakPtr<FIKRetargetHierarchyElement> ItemToExpand = ItemToShow->Parent;
		while(ItemToExpand.IsValid())
		{
			TreeView->SetItemExpansion(ItemToExpand.Pin(), true);
			ItemToExpand = ItemToExpand.Pin()->Parent;
		}
	}
    
	TreeView->RequestScrollIntoView(ItemToShow);
}

TSharedRef<SWidget> SIKRetargetHierarchy::CreateFilterMenuWidget()
{
	const FUIAction FilterHideNotInChainAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			FilterOptions.bHideBonesNotInChain = !FilterOptions.bHideBonesNotInChain;
			RefreshTreeView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return FilterOptions.bHideBonesNotInChain;
		}));

	const FUIAction FilterHideNotRetargetedAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			FilterOptions.bHideNotRetargetedBones = !FilterOptions.bHideNotRetargetedBones;
			RefreshTreeView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return FilterOptions.bHideNotRetargetedBones;
		}));

	const FUIAction FilterHideRetargetedAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			FilterOptions.bHideRetargetedBones = !FilterOptions.bHideRetargetedBones;
			RefreshTreeView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return FilterOptions.bHideRetargetedBones;
		}));
	
	static constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);

	MenuBuilder.BeginSection("Hierarchy Filters", LOCTEXT("HierarchyFiltersSection", "Hierarchy Filters"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HideNotInChainLabel", "Hide Bones Not in a Chain"),
		LOCTEXT("HideNotInChainTooltip", "Hide bones that are not part of a retarget chain."),
		FSlateIcon(),
		FilterHideNotInChainAction,
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HideUnMappedLabel", "Hide Bones Not Retargeted"),
		LOCTEXT("HideUnMappedTooltip", "Hide bones that are not being affected by the retargeting; either by not being in a retarget chain or being in a chain that is not mapped to anything on the source."),
		FSlateIcon(),
		FilterHideNotRetargetedAction,
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HideNonIKLabel", "Hide Retargeted Bones"),
		LOCTEXT("HideNonIKTooltip", "Hide all bones that are part of a mapped retarget chain."),
		FSlateIcon(),
		FilterHideRetargetedAction,
		NAME_None,
		EUserInterfaceActionType::Check);
	
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Clear", LOCTEXT("ClearMapFiltersSection", "Clear"));
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearIKRigHierarchyFilterLabel", "Clear Filters"),
		LOCTEXT("ClearIKRigHierarchyFilterTooltip", "Clear all filters to show all bones."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]
		{
			FilterOptions = FRetargetHierarchyFilterOptions();
			RefreshTreeView();
		})));

	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

void SIKRetargetHierarchy::OnFilterTextChanged(const FText& SearchText)
{
	TextFilter->SetFilterText(SearchText);
	RefreshTreeView(false);
}

void SIKRetargetHierarchy::RefreshTreeView(bool IsInitialSetup)
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

	const TSharedRef<FIKRetargetEditorController> ControllerRef = Controller.ToSharedRef();

	// synchronize selection with editor controller
	const TArray<FName> SelectedBones = Controller->GetSelectedBones();
	for (auto Element : AllElements)
	{
		const bool bIsSelected = SelectedBones.Contains(Element.Get()->Name);
		TreeView->SetItemSelection(Element, bIsSelected, ESelectInfo::Direct);
	}
	
	// save expansion and selection state
	TreeView->SaveAndClearState();

	// reset all tree items
	RootElements.Reset();
	AllElements.Reset();

	// validate we have a skeleton to load
	const ERetargetSourceOrTarget SourceOrTarget = Controller->GetSourceOrTarget();
	const USkeletalMesh* Mesh = Controller->GetSkeletalMesh(SourceOrTarget);
	if (!Mesh)
	{
		TreeView->RequestTreeRefresh();
		return;
	}

	// get the skeleton that is currently being viewed in the editor
	const FReferenceSkeleton& Skeleton = Mesh->GetRefSkeleton();
	TArray<FName> BoneNames;
	TArray<int32> ParentIndices;
	BoneNames.AddUninitialized(Skeleton.GetNum());
	ParentIndices.AddUninitialized(Skeleton.GetNum());
	for (int32 BoneIndex=0; BoneIndex<Skeleton.GetNum(); ++BoneIndex)
	{
		BoneNames[BoneIndex] = Skeleton.GetBoneName(BoneIndex);
		ParentIndices[BoneIndex] = Skeleton.GetParentIndex(BoneIndex);
	}

	auto FilterString = [this](const FString& StringToTest) ->bool
	{
		return TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(StringToTest));
	};
	
	// record bone element indices
	TMap<FName, int32> BoneTreeElementIndices;

	// create all bone elements
	for (const FName BoneName : BoneNames)
	{
		// create "Bone" tree element for this bone
		TSharedPtr<FIKRetargetHierarchyElement> BoneElement = MakeShared<FIKRetargetHierarchyElement>(BoneName, ControllerRef);
		const int32 BoneElementIndex = AllElements.Add(BoneElement);
		BoneTreeElementIndices.Add(BoneName, BoneElementIndex);

		// get the chain this bone belongs to
		FName ChainName = Controller->GetChainDisplayNameForBone(BoneName, Controller->GetSourceOrTarget());
		
		// apply text filter to items
		if (!(TextFilter->GetFilterText().IsEmpty() ||
			FilterString(BoneName.ToString()) ||
			FilterString(ChainName.ToString()))
			)
		{
			BoneElement->bIsHidden = true;
		}

		// apply filter "hide bones not in chain"
		if (FilterOptions.bHideBonesNotInChain && ChainName == NAME_None)
		{
			BoneElement->bIsHidden = true;
		}

		// apply filter "hide bones not retargeted"
		if (FilterOptions.bHideNotRetargetedBones || FilterOptions.bHideRetargetedBones)
		{
			const bool bIsRetargeted = Controller->IsBoneRetargeted(BoneName, Controller->GetSourceOrTarget());

			if (FilterOptions.bHideRetargetedBones && bIsRetargeted)
			{
				BoneElement->bIsHidden = true;
			}
			
			if (FilterOptions.bHideNotRetargetedBones && !bIsRetargeted)
			{
				BoneElement->bIsHidden = true;
			}
		}
	}

	// store children/parent pointers on all bone elements
	for (int32 BoneIndex=0; BoneIndex<BoneNames.Num(); ++BoneIndex)
	{
		const FName BoneName = BoneNames[BoneIndex];
		TWeakPtr<FIKRetargetHierarchyElement> BoneTreeElement = AllElements[BoneTreeElementIndices[BoneName]];

		if (BoneTreeElement.IsValid() && BoneTreeElement.Pin()->bIsHidden)
		{
			continue;
		}
		
		// find first parent that is not filtered
		TWeakPtr<FIKRetargetHierarchyElement> FirstNonHiddenParentElement;
		int32 ParentBoneIndex = ParentIndices[BoneIndex];
		while (true)
		{
			if (ParentBoneIndex < 0)
			{
				break;
			}

			const FName ParentBoneName = BoneNames[ParentBoneIndex];
			TSharedPtr<FIKRetargetHierarchyElement>& ParentBoneTreeElement = AllElements[BoneTreeElementIndices[ParentBoneName]];
			if (!ParentBoneTreeElement->bIsHidden)
			{
				FirstNonHiddenParentElement = ParentBoneTreeElement;
				break;
			}
			
			ParentBoneIndex = ParentIndices[ParentBoneIndex];
		}

		if (FirstNonHiddenParentElement.IsValid())
		{
			// store pointer to child on parent
			FirstNonHiddenParentElement.Pin()->Children.Add(BoneTreeElement);
			// store pointer to parent on child
			BoneTreeElement.Pin()->Parent = FirstNonHiddenParentElement;
		}
		else
		{
			// has no parent, store a root element
			RootElements.Add(BoneTreeElement.Pin());
		}
	}

	// expand all elements upon the initial construction of the tree
	if (IsInitialSetup)
	{
		for (TSharedPtr<FIKRetargetHierarchyElement> RootElement : RootElements)
		{
			SetExpansionRecursive(RootElement, false, true);
		}
	}
	else
	{
		// restore expansion and selection state
		for (const TSharedPtr<FIKRetargetHierarchyElement>& Element : AllElements)
		{
			TreeView->RestoreState(Element);
		}
	}
	
	TreeView->RequestTreeRefresh();
}

void SIKRetargetHierarchy::HandleGetChildrenForTree(
	TSharedPtr<FIKRetargetHierarchyElement> InItem,
	TArray<TSharedPtr<FIKRetargetHierarchyElement>>& OutChildren)
{
	for (TWeakPtr<FIKRetargetHierarchyElement>& Child : InItem->Children)
	{
		OutChildren.Add(Child.Pin());
	}
}

void SIKRetargetHierarchy::OnSelectionChanged(
	TSharedPtr<FIKRetargetHierarchyElement> Selection,
	ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::Direct)
	{
		return;
	}
	
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

	TArray<FName> SelectedBoneNames;
	TArray<TSharedPtr<FIKRetargetHierarchyElement>> SelectedItems = TreeView->GetSelectedItems();
	for (auto SelectedItem : SelectedItems)
	{
		SelectedBoneNames.Add(SelectedItem.Get()->Name);
	}

	constexpr bool bFromHierarchy = true;
	Controller->EditBoneSelection(SelectedBoneNames, ESelectionEdit::Replace, bFromHierarchy);
}

void SIKRetargetHierarchy::OnItemDoubleClicked(TSharedPtr<FIKRetargetHierarchyElement> InItem)
{
	if (TreeView->IsItemExpanded(InItem))
	{
		SetExpansionRecursive(InItem, false, false);
	}
	else
	{
		SetExpansionRecursive(InItem, false, true);
	}
}

void SIKRetargetHierarchy::OnSetExpansionRecursive(TSharedPtr<FIKRetargetHierarchyElement> InItem, bool bShouldBeExpanded)
{
	SetExpansionRecursive(InItem, false, bShouldBeExpanded);
}

void SIKRetargetHierarchy::SetExpansionRecursive(
	TSharedPtr<FIKRetargetHierarchyElement> InElement,
	bool bTowardsParent,
	bool bShouldBeExpanded)
{
	TreeView->SetItemExpansion(InElement, bShouldBeExpanded);
    
	if (bTowardsParent)
	{
		if (InElement->Parent.IsValid())
		{
			SetExpansionRecursive(InElement->Parent.Pin(), bTowardsParent, bShouldBeExpanded);
		}
	}
	else
	{
		for (int32 ChildIndex = 0; ChildIndex < InElement->Children.Num(); ++ChildIndex)
		{
			SetExpansionRecursive(InElement->Children[ChildIndex].Pin(), bTowardsParent, bShouldBeExpanded);
		}
	}
}

#undef LOCTEXT_NAMESPACE
