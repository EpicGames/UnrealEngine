// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/SIKRetargetOverrideManager.h"

#include "RetargetEditor/IKRetargetEditorController.h"
#include "Preferences/PersonaOptions.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Retargeter/IKRetargetProcessor.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "SPositiveActionButton.h"
#include "GeometryCollection/Facades/CollectionConstraintOverrideFacade.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Net/ReplayPlaylistTracker.h"
#include "RetargetEditor/IKRetargetEditorStyle.h"
#include "RigEditor/IKRigStructViewer.h"
#include "Framework/Commands/GenericCommands.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SIKRetargetOverrideManager"

static const FName OverrideSetActiveColumn(TEXT("OverrideSetActive"));
static const FName OverrideSetNameColumn(TEXT("OverrideSetName"));
static const FName OverrideSetOpsColumn(TEXT("NumOps"));
static const FName OverrideSetOverridesColumn(TEXT("NumOverrides"));

FIKRetargetOverrideSetElement::FIKRetargetOverrideSetElement(
	const FName& InName,
	const TSharedRef<FIKRetargetEditorController>& InEditorController)
	: Key(FText::FromName(InName)), Name(InName), EditorController(InEditorController)
{}

FText FIKRetargetOverrideSetElement::GetNumOpsLabel() const
{
	TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller)
	{
	   return FText::AsNumber(0);
	}

	FRetargetOverrideSet* OverrideSet = Controller->AssetController->FindOverrideSet(Name);
	if (!ensure(OverrideSet))
	{
	   return FText::AsNumber(0);
	}

	return FText::AsNumber(OverrideSet->OpOverrides.Num());
}

FText FIKRetargetOverrideSetElement::GetNumOverridesLabel() const
{
	TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller)
	{
	   return FText::AsNumber(0);
	}

	FRetargetOverrideSet* OverrideSet = Controller->AssetController->FindOverrideSet(Name);
	if (!ensure(OverrideSet))
	{
	   return FText::AsNumber(0);
	}

	int32 NumOverrides = 0;
	for (const FRetargetOpOverrides& OpOverrides : OverrideSet->OpOverrides)
	{
	   NumOverrides += OpOverrides.GetNumPropertyOverrides();
	}

	return FText::AsNumber(NumOverrides);
}

void SIKRetargetOverrideSetRow::Construct(
	const FArguments& InArgs,
	const TSharedRef<STableViewBase>& InOwnerTable)
{
	OwnerTable = InOwnerTable;
	WeakTreeElement = InArgs._TreeElement;
	EditorController = InArgs._EditorController;
	OverrideManager = InArgs._OverrideManager;
	
	SMultiColumnTableRow<TSharedPtr<FIKRetargetOverrideSetElement>>::Construct(
	   FSuperRowType::FArguments()
	   .OnDragDetected(OverrideManager.Pin().Get(), &SIKRetargetOverrideManager::OnDragDetected)
	   .OnCanAcceptDrop(OverrideManager.Pin().Get(), &SIKRetargetOverrideManager::OnCanAcceptDrop)
	   .OnAcceptDrop(OverrideManager.Pin().Get(), &SIKRetargetOverrideManager::OnAcceptDrop), InOwnerTable);
}

TSharedRef<SWidget> SIKRetargetOverrideSetRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	const FSlateBrush* Brush = FIKRetargetEditorStyle::Get().GetBrush("IKRetarget.PostSettingsSmall");

	FTextBlockStyle NormalText = FIKRigEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("IKRig.Tree.NormalText");
	FSlateFontInfo TextFont = NormalText.Font;
	FSlateColor TextColor = NormalText.ColorAndOpacity;

	if (ColumnName == OverrideSetActiveColumn)
	{
		return SNew(SBox)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked(this, &SIKRetargetOverrideSetRow::IsOverrideSetActive)
			.OnCheckStateChanged(this, &SIKRetargetOverrideSetRow::OnOverrideSetActiveChanged)
			.ToolTipText(LOCTEXT("OverrideSetActiveTooltip", "Toggle if this override set is active in the current stack."))
		];
	}

	if (ColumnName == OverrideSetNameColumn)
	{
		TSharedPtr< SHorizontalBox > RowBox;
		SAssignNew(RowBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SExpanderArrow, SharedThis(this) )
			.ShouldDrawWires(true)
		];

		RowBox->AddSlot()
		.MaxWidth(18)
		.FillWidth(1.0)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(Brush)
		];

		RowBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SAssignNew(InlineTextBlock, SInlineEditableTextBlock)
			.Text(this, &SIKRetargetOverrideSetRow::GetItemText)
			.OnVerifyTextChanged(this, &SIKRetargetOverrideSetRow::OnVerifyTextChanged)
			.OnTextCommitted(this, &SIKRetargetOverrideSetRow::OnTextCommitted)
		];

		return RowBox.ToSharedRef();
	}

	if (ColumnName == OverrideSetOpsColumn)
	{
		TWeakPtr<FIKRetargetOverrideSetElement> TreeElementPtr = WeakTreeElement;
		
		TSharedPtr< SHorizontalBox > RowBox;
		SAssignNew(RowBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text_Lambda([TreeElementPtr]()
				{
					if (TreeElementPtr.IsValid())
					{
						return TreeElementPtr.Pin()->GetNumOpsLabel();
					}
					return FText::GetEmpty();
				})
			.Font(TextFont)
			.ColorAndOpacity(TextColor)
		];

		return RowBox.ToSharedRef();
	}

	if (ColumnName == OverrideSetOverridesColumn)
	{
		TWeakPtr<FIKRetargetOverrideSetElement> TreeElementPtr = WeakTreeElement;
		
		TSharedPtr< SHorizontalBox > RowBox;
		SAssignNew(RowBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text_Lambda([TreeElementPtr]()
				{
					if (TreeElementPtr.IsValid())
					{
						return TreeElementPtr.Pin()->GetNumOverridesLabel();	
					}
					return FText::GetEmpty();
				})
			.Font(TextFont)
			.ColorAndOpacity(TextColor)
		];

		return RowBox.ToSharedRef();
	}

	return SNullWidget::NullWidget;
}

void SIKRetargetOverrideSetRow::OnRequestRename()
{
	if (InlineTextBlock.IsValid())
	{
		InlineTextBlock.Get()->EnterEditingMode();	
	}
}

FText SIKRetargetOverrideSetRow::GetItemText() const
{
	if (WeakTreeElement.IsValid())
	{
		return FText::FromName(WeakTreeElement.Pin()->Name);	
	}
	
	return FText::GetEmpty();
}

bool SIKRetargetOverrideSetRow::OnVerifyTextChanged(const FText& InNewText, FText& OutErrorMessage)
{
	if (InNewText.IsEmpty())
	{
	   OutErrorMessage = LOCTEXT("OverrideSetNameEmpty", "New override set name cannot be empty.");
	   return false;
	}

	FName NewName = FName(*InNewText.ToString());
	FRetargetOverrideSet* ExistingSet = EditorController.Pin()->AssetController->FindOverrideSet(NewName);
	if (ExistingSet)
	{
	   OutErrorMessage = LOCTEXT("OverrideSetNameNotUnique", "New override set name already in use.");
	   return false;
	}

	return true;
}

void SIKRetargetOverrideSetRow::OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo)
{
	if (WeakTreeElement.IsValid())
	{
		const FName OldName = WeakTreeElement.Pin()->Name;
		const FName NewName = FName(*InText.ToString());
		EditorController.Pin()->AssetController->RenameRetargetOverrideSet(OldName, NewName);
	}
}

ECheckBoxState SIKRetargetOverrideSetRow::IsOverrideSetActive() const
{
	if (TSharedPtr<FIKRetargetOverrideSetElement> Item = WeakTreeElement.Pin())
	{
		TObjectPtr<UIKRetargeterController> Controller = EditorController.Pin()->AssetController;
		const bool bIsActiveByDefault = Controller->GetOverrideSetActiveByDefault(Item->Name);
		return bIsActiveByDefault ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void SIKRetargetOverrideSetRow::OnOverrideSetActiveChanged(ECheckBoxState NewState) const
{
	if (TSharedPtr<FIKRetargetOverrideSetElement> Item = WeakTreeElement.Pin())
	{
		TObjectPtr<UIKRetargeterController> Controller = EditorController.Pin()->AssetController;
		const bool bNewState = NewState == ECheckBoxState::Checked;
		Controller->SetOverrideSetActiveByDefault(Item->Name, bNewState);
	}
}

TSharedRef<FRetargetOverrideSetDragDropOp> FRetargetOverrideSetDragDropOp::New(TWeakPtr<FIKRetargetOverrideSetElement> InElement)
{
	TSharedRef<FRetargetOverrideSetDragDropOp> Operation = MakeShared<FRetargetOverrideSetDragDropOp>();
	Operation->Element = InElement;
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FRetargetOverrideSetDragDropOp::GetDefaultDecorator() const
{
	return FDecoratedDragDropOp::GetDefaultDecorator();
}

void SIKRetargetOverrideManager::Construct(
	const FArguments& InArgs,
	TSharedRef<FIKRetargetEditorController> InEditorController)
{
	EditorController = InEditorController;
	EditorController.Pin()->SetOverrideSetsView(SharedThis(this));
	
	TextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));

	BindCommands();

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	DetailsView = PropertyModule.CreateDetailView(DetailsViewArgs);
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		.HAlign(HAlign_Fill)
		[
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			[
				SNew(SPositiveActionButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				.Text(LOCTEXT("AddRetargetOverrideSetLabel", "Add Override Set"))
				.ToolTipText(LOCTEXT("AddOverrideSetToolTip", "Create a new retarget override set."))
				.OnClicked_Lambda([this]()
				{
					const FName NewOverrideSet = FName("NewRetargetOverrideSet");
					EditorController.Pin()->AssetController->AddNewRetargetOverrideSet(NewOverrideSet);
					return FReply::Handled();
				})
			]
			
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSearchBox)
				.SelectAllTextWhenFocused(true)
				.OnTextChanged( this, &SIKRetargetOverrideManager::OnFilterTextChanged )
				.HintText( LOCTEXT( "SearchBoxHint", "Filter Hierarchy Tree...") )
			]
		]

		+SVerticalBox::Slot()
		.Padding(2.0f)
		[
			SNew(SSplitter)
			.Orientation(Orient_Vertical)
			
			+ SSplitter::Slot()
			.Value(0.3)
			[
				SNew(SBorder)
				.Padding(2.0f)
				.BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
				[
					SAssignNew(TreeView, SIKRetargetOverrideSetTreeView)
					.TreeItemsSource(&RootElements)
					.SelectionMode(ESelectionMode::Multi)
					.OnGenerateRow_Lambda( [this](
						TSharedPtr<FIKRetargetOverrideSetElement> InItem,
						const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
					{
						return SNew(SIKRetargetOverrideSetRow, OwnerTable)
						.EditorController(EditorController.Pin())
						.TreeElement(InItem)
						.OverrideManager(SharedThis(this));
					})
					.OnGetChildren(this, &SIKRetargetOverrideManager::HandleGetChildrenForTree)
					.OnSelectionChanged(this, &SIKRetargetOverrideManager::OnSelectionChanged)
					.OnMouseButtonDoubleClick(this, &SIKRetargetOverrideManager::OnItemDoubleClicked)
					.OnSetExpansionRecursive(this, &SIKRetargetOverrideManager::OnSetExpansionRecursive)
					.OnContextMenuOpening(this, &SIKRetargetOverrideManager::OnGenerateContextMenu)
					.HighlightParentNodesForSelection(false)
					.HeaderRow
					(
						SNew(SHeaderRow)

						+ SHeaderRow::Column(OverrideSetActiveColumn)
						.DefaultLabel(LOCTEXT("OverrideSetActiveLabel", "Active"))
						.FillWidth(0.1f)
						
						+ SHeaderRow::Column(OverrideSetNameColumn)
						.DefaultLabel(LOCTEXT("OverrideSetNameLabel", "Override Set Name"))
						.FillWidth(0.7f)

						+ SHeaderRow::Column(OverrideSetOpsColumn)
						.DefaultLabel(LOCTEXT("NumOpsLabel", "Ops"))
						.FillWidth(0.1f)
						
						+ SHeaderRow::Column(OverrideSetOverridesColumn)
						.DefaultLabel(LOCTEXT("NumOverridesLabel", "Properties"))
						.FillWidth(0.1f)
					)
				]
			]
			
			+ SSplitter::Slot()
			.Value(0.7f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					DetailsView.ToSharedRef()
				]
			]
		]
	];

	constexpr bool IsInitialSetup = true;
	RefreshTreeView(IsInitialSetup);
}

FName SIKRetargetOverrideManager::GetSelectedOverrideSetName() const
{
	TArray<TSharedPtr<FIKRetargetOverrideSetElement>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
	   return NAME_None;
	}

	return SelectedItems[0]->Name;
}

void SIKRetargetOverrideManager::OnFilterTextChanged(const FText& SearchText)
{
	TextFilter->SetFilterText(SearchText);
	RefreshTreeView(false);
}

void SIKRetargetOverrideManager::RefreshTreeView(bool IsInitialSetup)
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

	TreeView->SaveAndClearState();

	RootElements.Reset();
	AllElements.Reset();

	const TMap<FName, FRetargetOverrideSet>& AllOverrideSets = Controller->AssetController->GetAllOverrideSets();
	
	TMap<FName, TSharedPtr<FIKRetargetOverrideSetElement>> SetToElementMap;

	const bool bHasTextFilter = !TextFilter->GetFilterText().IsEmpty();
	auto PassesTextFilter = [&](const FName& NameToCheck) -> bool
	{
		return TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(NameToCheck.ToString()));
	};

	for (const TTuple<FName, FRetargetOverrideSet>& Pair : AllOverrideSets)
	{
		const FName& SetName = Pair.Key;
		const FRetargetOverrideSet& SetData = Pair.Value;

		TSharedPtr<FIKRetargetOverrideSetElement> NewElement = MakeShared<FIKRetargetOverrideSetElement>(SetName, Controller.ToSharedRef());
		
		AllElements.Add(NewElement);
		SetToElementMap.Add(SetName, NewElement);

		NewElement->bIsHidden= false;
		if (FilterOptions.bHideEmptySets && SetData.OpOverrides.IsEmpty())
		{
			NewElement->bIsHidden = true;
		}
		else if (bHasTextFilter && !PassesTextFilter(SetName))
		{
			NewElement->bIsHidden = true;
		}
	}

	for (TSharedPtr<FIKRetargetOverrideSetElement>& CurrentElement : AllElements)
	{
		if (CurrentElement->bIsHidden)
		{
			continue;
		}

		const FRetargetOverrideSet* SetData = AllOverrideSets.Find(CurrentElement->Name);
		if (!ensure(SetData))
		{
			continue;
		}

		TSharedPtr<FIKRetargetOverrideSetElement> ValidParent = nullptr;
		FName NextParentName = SetData->ParentName;

		while (NextParentName != NAME_None)
		{
			TSharedPtr<FIKRetargetOverrideSetElement>* ParentPtr = SetToElementMap.Find(NextParentName);
			if (!ParentPtr)
			{
			   break;
			}
			TSharedPtr<FIKRetargetOverrideSetElement> Parent = *ParentPtr;
			
			if (!Parent->bIsHidden)
			{
			   ValidParent = Parent;
			   break;
			}

			const FRetargetOverrideSet* ParentSet = AllOverrideSets.Find(NextParentName);
			NextParentName = ParentSet ? ParentSet->ParentName : NAME_None;
		}

		if (ValidParent.IsValid())
		{
			ValidParent->Children.Add(CurrentElement);
			CurrentElement->Parent = ValidParent;
		}
		else
		{
			RootElements.Add(CurrentElement);
		}
	}

	// sort root elements and children by DisplayOrder
	auto SortByDisplayOrder = [&AllOverrideSets](const TSharedPtr<FIKRetargetOverrideSetElement>& A, const TSharedPtr<FIKRetargetOverrideSetElement>& B)
	{
		if (!A || !B)
		{
			return false;
		}
		const FRetargetOverrideSet* SetA = AllOverrideSets.Find(A->Name);
		const FRetargetOverrideSet* SetB = AllOverrideSets.Find(B->Name);
		if (SetA && SetB)
		{
			return SetA->DisplayOrder < SetB->DisplayOrder;
		}
		return false;
	};
	RootElements.Sort(SortByDisplayOrder);
	for (const TSharedPtr<FIKRetargetOverrideSetElement>& Element : AllElements)
	{
		Element->Children.Sort([&SortByDisplayOrder](const TWeakPtr<FIKRetargetOverrideSetElement>& A, const TWeakPtr<FIKRetargetOverrideSetElement>& B)
		{
			return SortByDisplayOrder(A.Pin(), B.Pin());
		});
	}

	if (IsInitialSetup)
	{
		for (const TSharedPtr<FIKRetargetOverrideSetElement>& Root : RootElements)
		{
			SetExpansionRecursive(Root, false, true);
		}
	}
	else
	{
		for (const TSharedPtr<FIKRetargetOverrideSetElement>& Element : AllElements)
		{
			TreeView->RestoreState(Element);
		}
	}

	TreeView->RequestTreeRefresh();
}

void SIKRetargetOverrideManager::HandleGetChildrenForTree(
	TSharedPtr<FIKRetargetOverrideSetElement> InItem,
	TArray<TSharedPtr<FIKRetargetOverrideSetElement>>& OutChildren)
{
	if (InItem.IsValid())
	{
		for (TWeakPtr<FIKRetargetOverrideSetElement>& Child : InItem->Children)
		{
			if (Child.IsValid())
			{
				OutChildren.Add(Child.Pin());	
			}
		}	
	}
}

void SIKRetargetOverrideManager::OnSelectionChanged(
	TSharedPtr<FIKRetargetOverrideSetElement> Selection,
	ESelectInfo::Type SelectInfo)
{
	TArray<TSharedPtr<FIKRetargetOverrideSetElement>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
	   DetailsView->SetObject(nullptr, true /*bForceRefresh*/);
	   return;
	}

	const TObjectPtr<UIKRetargeterController> AssetController = EditorController.Pin()->AssetController;

	const FName SetToViewName = SelectedItems[0].Get()->Name;
	FRetargetOverrideSet* SetToView = AssetController->FindOverrideSet(SetToViewName);
	if (!SetToView)
	{
	   DetailsView->SetObject(nullptr, true /*bForceRefresh*/);
	   return;
	}

	UIKRetargeter* Asset = AssetController->GetAsset();
	auto MemoryProvider = [Asset, SetToViewName]() -> uint8*
	   {
		  UIKRetargeterController* Controller = UIKRetargeterController::GetController(Asset);
		  if (FRetargetOverrideSet* SetToEdit = Controller->FindOverrideSet(SetToViewName))
		  {
			 return reinterpret_cast<uint8*>(SetToEdit);
		  }
	   
		  return nullptr;
	   };
	
	FIKRigStructToView StructToView;
	StructToView.Type = FRetargetOverrideSet::StaticStruct();
	StructToView.MemoryProvider = MemoryProvider;
	StructToView.Owner = AssetController->GetAsset();
	StructToView.UniqueName = SetToViewName;
	
	UIKRigStructViewer* StructViewer = AssetController->GetStructViewer(ERetargetStructViewerMode::OverrideSets);
	StructViewer->SetStructToView(StructToView);
	
	constexpr bool bForceRefresh = true;
	DetailsView->SetObject(StructViewer, bForceRefresh);
}

void SIKRetargetOverrideManager::OnItemDoubleClicked(TSharedPtr<FIKRetargetOverrideSetElement> InItem)
{
	if (TreeView->IsItemExpanded(InItem))
	{
	   SetExpansionRecursive(InItem, false, false);
	}
	else
	{
	   SetExpansionRecursive(InItem, false, true);
	}
}

void SIKRetargetOverrideManager::OnSetExpansionRecursive(TSharedPtr<FIKRetargetOverrideSetElement> InItem, bool bShouldBeExpanded)
{
	SetExpansionRecursive(InItem, false, bShouldBeExpanded);
}

void SIKRetargetOverrideManager::SetExpansionRecursive(
	TSharedPtr<FIKRetargetOverrideSetElement> InElement,
	bool bTowardsParent,
	bool bShouldBeExpanded)
{
	TreeView->SetItemExpansion(InElement, bShouldBeExpanded);

	if (bTowardsParent)
	{
		if (InElement->Parent.IsValid())
		{
			SetExpansionRecursive(InElement->Parent.Pin(), bTowardsParent, bShouldBeExpanded);
		}
	}
	else
	{
		for (int32 ChildIndex = 0; ChildIndex < InElement->Children.Num(); ++ChildIndex)
		{
			SetExpansionRecursive(InElement->Children[ChildIndex].Pin(), bTowardsParent, bShouldBeExpanded);
		}
	}
}

TSharedPtr<SWidget> SIKRetargetOverrideManager::OnGenerateContextMenu()
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.BeginSection("ItemActions", INVTEXT("Override Set Actions"));
	{
		MenuBuilder.AddMenuEntry(
		LOCTEXT("RenameItem", "Rename"),
		LOCTEXT("RenameLabel", "Rename the selected override set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
		FUIAction(
		FExecuteAction::CreateLambda([this]()
		{
			TArray<TSharedPtr<FIKRetargetOverrideSetElement>> Selection = TreeView->GetSelectedItems();
			if (Selection.IsEmpty())
			{
				return;
			}

			TSharedPtr<ITableRow> RowWidget = TreeView->WidgetFromItem(Selection[0]);
			if (RowWidget.IsValid())
			{
				TSharedRef<SWidget> Widget = RowWidget->AsWidget();
				SIKRetargetOverrideSetRow* SetRow = static_cast<SIKRetargetOverrideSetRow*>(&Widget.Get());
				if (SetRow)
				{
					SetRow->OnRequestRename();
				}
			}
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			return TreeView.IsValid() && TreeView->GetNumItemsSelected() == 1;
		})));

		MenuBuilder.AddMenuEntry(
		LOCTEXT("DuplicateItem", "Duplicate"),
		LOCTEXT("DuplicateLabel", "Duplicate the selected override set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Duplicate"),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			TArray<TSharedPtr<FIKRetargetOverrideSetElement>> Selection = TreeView->GetSelectedItems();
			if (Selection.IsEmpty())
			{
				return;
			}

			EditorController.Pin()->AssetController->DuplicateRetargetOverrideSet(Selection[0]->Name);
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			return TreeView.IsValid() && TreeView->GetNumItemsSelected() == 1;
		})));

		MenuBuilder.AddMenuEntry(
		LOCTEXT("AddChildItem", "Add Child"),
		LOCTEXT("AddChildLabel", "Create a new override set parented under the selected set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			TArray<TSharedPtr<FIKRetargetOverrideSetElement>> Selection = TreeView->GetSelectedItems();
			if (Selection.IsEmpty())
			{
				return;
			}

			const FName ParentName = Selection[0]->Name;
			const FName ChildName = EditorController.Pin()->AssetController->AddNewRetargetOverrideSet(FName("NewChildOverrideSet"));
			if (ChildName != NAME_None)
			{
				EditorController.Pin()->AssetController->SetParentOverrideSet(ChildName, ParentName);
			}
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			return TreeView.IsValid() && TreeView->GetNumItemsSelected() == 1;
		})));

		MenuBuilder.AddMenuEntry(
		LOCTEXT("UnparentItem", "Unparent"),
		LOCTEXT("UnparentLabel", "Unparent the selected override set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Rename"),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			TArray<TSharedPtr<FIKRetargetOverrideSetElement>> Selection = TreeView->GetSelectedItems();
			if (Selection.IsEmpty())
			{
				return;
			}
			
			TSharedPtr<FIKRetargetOverrideSetElement>& SetElement = Selection[0];
			EditorController.Pin()->AssetController->SetParentOverrideSet(SetElement->Name, NAME_None);
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			return TreeView.IsValid() && TreeView->GetNumItemsSelected() == 1;
		})));

		MenuBuilder.AddMenuEntry(
		LOCTEXT("RemoveAllLabel", "Remove All Overrides"),
		LOCTEXT("RemoveAllTip", "Remove all overrides from the selected override set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.SelectAll"),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			TArray<TSharedPtr<FIKRetargetOverrideSetElement>> Selection = TreeView->GetSelectedItems();
			if (Selection.IsEmpty())
			{
				return;
			}

			TSharedPtr<FIKRetargetOverrideSetElement> SelectedSetElement = Selection[0];
			if (!SelectedSetElement.IsValid())
			{
				return;
			}
			
			const FName SetToClear = SelectedSetElement.Get()->Name;
			EditorController.Pin()->AssetController->RemoveAllPropertyOverridesFromSet(SetToClear);
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			return TreeView.IsValid() && TreeView->GetNumItemsSelected() > 0;
		})));

		MenuBuilder.AddMenuEntry(
		LOCTEXT("DeleteItem", "Delete"),
		LOCTEXT("DeleteLabel", "Delete the selected override set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Delete"),
		FUIAction(FExecuteAction::CreateLambda([this]()
		{
			TArray<TSharedPtr<FIKRetargetOverrideSetElement>> Selection = TreeView->GetSelectedItems();
			if (Selection.IsEmpty())
			{
				return;
			}

			TSharedPtr<FIKRetargetOverrideSetElement> SelectedSetElement = Selection[0];
			if (!SelectedSetElement.IsValid())
			{
				return;
			}
			
			const FName SetToDelete = SelectedSetElement.Get()->Name;
			EditorController.Pin()->AssetController->RemoveRetargetOverrideSet(SetToDelete);
		}),
		FCanExecuteAction::CreateLambda([this]()
		{
			return TreeView.IsValid() && TreeView->GetNumItemsSelected() > 0;
		})));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SIKRetargetOverrideManager::BindCommands()
{
	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(
		FGenericCommands::Get().Delete,
		FExecuteAction::CreateSP(this, &SIKRetargetOverrideManager::DeleteSelectedOverrideSet),
		FCanExecuteAction::CreateSP(this, &SIKRetargetOverrideManager::HasSelectedOverrideSet));

	CommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateSP(this, &SIKRetargetOverrideManager::RenameSelectedOverrideSet),
		FCanExecuteAction::CreateSP(this, &SIKRetargetOverrideManager::HasSelectedOverrideSet));
}

FReply SIKRetargetOverrideManager::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SCompoundWidget::OnKeyDown(InGeometry, InKeyEvent);
}

void SIKRetargetOverrideManager::DeleteSelectedOverrideSet()
{
	TArray<TSharedPtr<FIKRetargetOverrideSetElement>> Selection = TreeView->GetSelectedItems();
	if (Selection.IsEmpty())
	{
		return;
	}

	const FName SetToDelete = Selection[0]->Name;
	EditorController.Pin()->AssetController->RemoveRetargetOverrideSet(SetToDelete);
}

void SIKRetargetOverrideManager::RenameSelectedOverrideSet()
{
	TArray<TSharedPtr<FIKRetargetOverrideSetElement>> Selection = TreeView->GetSelectedItems();
	if (Selection.IsEmpty())
	{
		return;
	}

	TSharedPtr<ITableRow> RowWidget = TreeView->WidgetFromItem(Selection[0]);
	if (RowWidget.IsValid())
	{
		TSharedRef<SWidget> Widget = RowWidget->AsWidget();
		SIKRetargetOverrideSetRow* SetRow = static_cast<SIKRetargetOverrideSetRow*>(&Widget.Get());
		if (SetRow)
		{
			SetRow->OnRequestRename();
		}
	}
}

bool SIKRetargetOverrideManager::HasSelectedOverrideSet() const
{
	return TreeView.IsValid() && TreeView->GetNumItemsSelected() > 0;
}

FReply SIKRetargetOverrideManager::OnDragDetected(
	const FGeometry& MyGeometry,
	const FPointerEvent& MouseEvent)
{
	const TArray<TSharedPtr<FIKRetargetOverrideSetElement>> CurrentSelectedItems = TreeView->GetSelectedItems();
	if (CurrentSelectedItems.Num() != 1)
	{
		return FReply::Unhandled();
	}

	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TSharedPtr<FIKRetargetOverrideSetElement> DraggedElement = CurrentSelectedItems[0];
		const TSharedRef<FRetargetOverrideSetDragDropOp> DragDropOp = FRetargetOverrideSetDragDropOp::New(DraggedElement);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SIKRetargetOverrideManager::OnCanAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
	TSharedPtr<FIKRetargetOverrideSetElement> TargetElement)
{
	TOptional<EItemDropZone> ReturnedDropZone;
	const TSharedPtr<FRetargetOverrideSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FRetargetOverrideSetDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return ReturnedDropZone;
	}

	const TSharedPtr<FIKRetargetOverrideSetElement>& DraggedElement = DragDropOp.Get()->Element.Pin();
	if (!ensure(DraggedElement))
	{
		return ReturnedDropZone;
	}

	if (TargetElement == DraggedElement)
	{
		return ReturnedDropZone;
	}

	ReturnedDropZone = DropZone;
	return ReturnedDropZone;
}

FReply SIKRetargetOverrideManager::OnAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
	TSharedPtr<FIKRetargetOverrideSetElement> TargetElement)
{
	const TSharedPtr<FRetargetOverrideSetDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FRetargetOverrideSetDragDropOp>();
	if (!ensure(DragDropOp.IsValid()))
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<FIKRetargetOverrideSetElement>& DraggedElement = DragDropOp.Get()->Element.Pin();
	if (!ensure(DraggedElement))
	{
		return FReply::Unhandled();
	}

	if (DropZone == EItemDropZone::OntoItem)
	{
		// drop onto item = parent the dragged set to the target
		EditorController.Pin()->AssetController->SetParentOverrideSet(DraggedElement.Get()->Name, TargetElement.Get()->Name);
	}
	else
	{
		// drop above/below item = reorder as sibling (same parent as target)
		const bool bInsertAfter = (DropZone == EItemDropZone::BelowItem);
		EditorController.Pin()->AssetController->ReorderOverrideSet(DraggedElement.Get()->Name, TargetElement.Get()->Name, bInsertAfter);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE