// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/SIKRetargetPoseEditor.h"

#include "RetargetEditor/IKRetargetCommands.h"
#include "RetargetEditor/IKRetargetEditor.h"
#include "RetargetEditor/IKRetargetEditorController.h"

#include "ToolMenus.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SIKRetargetPoseEditor"

void SIKRetargetPoseEditor::Construct(
	const FArguments& InArgs,
	TSharedRef<FIKRetargetEditorController> InEditorController)
{
	EditorController = InEditorController;
	
	// the editor controller
	FIKRetargetEditorController* Controller = EditorController.Pin().Get();

	// the commands for the menus
	TSharedPtr<FUICommandList> Commands = Controller->Editor.Pin()->GetToolkitCommands();

	RefreshPoseNames();
	
	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		.HAlign(HAlign_Center)
		[
			SNew(SHorizontalBox)
			
			// pose selection label
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4,0)
			[
				SNew(STextBlock).Text(LOCTEXT("CurrentPose", "Current Retarget Pose:"))
			]
						
			// pose selection combobox
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SComboBox<TSharedPtr<FName>>)
				.OptionsSource(&PoseNames)
				.OnComboBoxOpening_Lambda([this]()
				{
					RefreshPoseNames();
				})
				.OnGenerateWidget_Lambda([](TSharedPtr<FName> InItem)
				{
					return SNew(STextBlock).Text(FText::FromName(*InItem.Get()));
				})
				.OnSelectionChanged(Controller, &FIKRetargetEditorController::OnPoseSelected)
				[
					SNew(STextBlock).Text(Controller, &FIKRetargetEditorController::GetCurrentPoseName)
				]
			]

			// pose blending slider
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpinBox<float>)
				.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.NormalFont"))
				.MinDesiredWidth(100)
				.MinValue(0.0f)
				.MaxValue(1.0f)
				.Value(Controller, &FIKRetargetEditorController::GetRetargetPoseAmount)
				.OnValueChanged(Controller, &FIKRetargetEditorController::SetRetargetPoseAmount)
			]
		]

		// pose editing toolbar
		+SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Center)
			[
				MakeToolbar(Commands)
			]
		]
	];
}

void SIKRetargetPoseEditor::RefreshPoseNames()
{
	// get the retarget poses from the editor controller
	const FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	const TMap<FName, FIKRetargetPose>& RetargetPoses = Controller->AssetController->GetRetargetPoses(Controller->GetSourceOrTarget());

	// fill list of pose names
	PoseNames.Reset();
	for (const TTuple<FName, FIKRetargetPose>& Pose : RetargetPoses)
	{
		PoseNames.Add(MakeShareable(new FName(Pose.Key)));
	}
}

TSharedRef<SWidget>  SIKRetargetPoseEditor::MakeToolbar(TSharedPtr<FUICommandList> Commands)
{
	FToolBarBuilder ToolbarBuilder(Commands, FMultiBoxCustomization::None);
	
	ToolbarBuilder.BeginSection("Edit Current Pose");

	ToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &SIKRetargetPoseEditor::GenerateResetMenuContent, Commands),
		LOCTEXT("ResetPose_Label", "Reset"),
		LOCTEXT("ResetPoseToolTip_Label", "Reset bones to reference pose."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Refresh"));

	ToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &SIKRetargetPoseEditor::GenerateEditMenuContent, Commands),
		LOCTEXT("AutoAlign_Label", "Auto Align"),
		LOCTEXT("AutoAlignTip_Label", "Automatically aligns bones on source skeleton to target (or vice versa)."),
		FSlateIcon(FIKRetargetEditorStyle::Get().GetStyleSetName(),"IKRetarget.AutoAlign"));

	ToolbarBuilder.EndSection();

	ToolbarBuilder.BeginSection("Create Poses");

	ToolbarBuilder.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateSP(this, &SIKRetargetPoseEditor::GenerateNewMenuContent, Commands),
		LOCTEXT("CreatePose_Label", "Create"),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"));

	ToolbarBuilder.AddToolBarButton(
		FIKRetargetCommands::Get().DeleteRetargetPose,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(),"Icons.Delete"));

	ToolbarBuilder.AddToolBarButton(
		FIKRetargetCommands::Get().RenameRetargetPose,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(),"Icons.Settings"));

	ToolbarBuilder.EndSection();

	return ToolbarBuilder.MakeWidget();
}

TSharedRef<SWidget> SIKRetargetPoseEditor::GenerateResetMenuContent(TSharedPtr<FUICommandList> Commands)
{
	FMenuBuilder MenuBuilder(true, Commands);
	MenuBuilder.AddMenuEntry(FIKRetargetCommands::Get().ResetSelectedBones);
	MenuBuilder.AddMenuEntry(FIKRetargetCommands::Get().ResetSelectedAndChildrenBones);
	MenuBuilder.AddMenuEntry(FIKRetargetCommands::Get().ResetAllBones);
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SIKRetargetPoseEditor::GenerateEditMenuContent(TSharedPtr<FUICommandList> Commands)
{
	FMenuBuilder MenuBuilder(true, Commands);

	static TArray<TSharedPtr<ERetargetAutoAlignMethod>> AlignModes =
	{
		MakeShared<ERetargetAutoAlignMethod>(ERetargetAutoAlignMethod::ChainToChain),
		MakeShared<ERetargetAutoAlignMethod>(ERetargetAutoAlignMethod::LocalRotationAxes),
		MakeShared<ERetargetAutoAlignMethod>(ERetargetAutoAlignMethod::GlobalRotationAxes),
		MakeShared<ERetargetAutoAlignMethod>(ERetargetAutoAlignMethod::MeshToMesh),
	};

	static auto ToDisplayText = [](ERetargetAutoAlignMethod Type)
	{
		switch (Type)
		{
		case ERetargetAutoAlignMethod::ChainToChain: return LOCTEXT("DirectionTypeLabel", "Direction");
		case ERetargetAutoAlignMethod::LocalRotationAxes: return LOCTEXT("LocalAxisTypeLabel", "Local Rotation Axes");
		case ERetargetAutoAlignMethod::GlobalRotationAxes: return LOCTEXT("GlobalAxisTypeLabel", "Global Rotation Axes");
		case ERetargetAutoAlignMethod::MeshToMesh: return LOCTEXT("MeshTypeLabel", "Mesh");
		default: return FText::GetEmpty();
		}
	};

	MenuBuilder.BeginSection("AlignBonesHeader", LOCTEXT("AlignBonesHeader", "Auto-Align Bones"));
	MenuBuilder.AddMenuEntry(FIKRetargetCommands::Get().AlignAllBones);
	MenuBuilder.AddMenuEntry(FIKRetargetCommands::Get().AlignSelected);
	MenuBuilder.AddMenuEntry(FIKRetargetCommands::Get().AlignSelectedAndChildren);
	MenuBuilder.AddWidget(
		SNew(SBox)
		[
			SNew(SComboBox<TSharedPtr<ERetargetAutoAlignMethod>>)
			.OptionsSource(&AlignModes)
			.ToolTipText(LOCTEXT("PoseAlignmentTooltipText",
				"Direction: aligns the direction of the bone to match that of the equivalent bone in the other skeleton. Uses the chain hierarchy to define a direction vector. \n"
				"Local Rotation Axes: aligns the local axes of the bone to match those of the equivalent bone in the other skeleton. May produce nonsensical results on skeletons with different rotation axes.\n"
				"Global Rotation Axes: aligns the global axes of the bone to match those of the equivalent bone in the other skeleton. May produce nonsensical results on skeletons with different rotation axes.\n"
				"Mesh: Generates a direction vector for the bone based on the principle axis of the vertices weighted to the bone."))
			.OnGenerateWidget_Lambda([](TSharedPtr<ERetargetAutoAlignMethod> InOption)
			{
				return SNew(STextBlock).Text(ToDisplayText(*InOption));
			})
			.OnSelectionChanged_Lambda([this](TSharedPtr<ERetargetAutoAlignMethod> NewSelection, ESelectInfo::Type SelectInfo)
			{
				EditorController.Pin()->CurrentPoseAlignmentMode = *NewSelection;
			})
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					return ToDisplayText(EditorController.Pin()->CurrentPoseAlignmentMode);
				})
			]
		]
		, LOCTEXT("AlignModes_TitleText", "Alignment Method"), true /*noindent*/);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("RootTranslationHeader", LOCTEXT("RootTranslationHeader", "Root Translation"));
	MenuBuilder.AddMenuEntry(FIKRetargetCommands::Get().SnapCharacterToGround);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SIKRetargetPoseEditor::GenerateNewMenuContent(TSharedPtr<FUICommandList> Commands)
{
	const FName ParentEditorName = EditorController.Pin()->Editor.Pin()->GetToolMenuName();
	const FName MenuName = FName(ParentEditorName.ToString() + TEXT(".CreateMenu"));
	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(MenuName);

	FToolMenuSection& CreateSection = ToolMenu->AddSection("Create", LOCTEXT("CreatePoseOperations", "Create New Retarget Pose"));
	CreateSection.AddMenuEntry(FIKRetargetCommands::Get().NewRetargetPose);
	CreateSection.AddMenuEntry(FIKRetargetCommands::Get().DuplicateRetargetPose);
	
	FToolMenuSection& ImportSection = ToolMenu->AddSection("Import",LOCTEXT("ImportPoseOperations", "Import Retarget Pose"));
	ImportSection.AddMenuEntry(FIKRetargetCommands::Get().ImportRetargetPose);
	ImportSection.AddMenuEntry(FIKRetargetCommands::Get().ImportRetargetPoseFromAnim);

	FToolMenuSection& ExportSection = ToolMenu->AddSection("Export",LOCTEXT("ExportPoseOperations", "Export Retarget Pose"));
	ExportSection.AddMenuEntry(FIKRetargetCommands::Get().ExportRetargetPose);

	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(Commands));
}

#undef LOCTEXT_NAMESPACE
