// Copyright Epic Games, Inc. All Rights Reserved.
#include "RetargetEditor/SIKRetargetVariablesEditor.h"

#include "RetargetEditor/IKRetargetEditorController.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ISinglePropertyView.h"
#include "PropertyBagDetails.h"
#include "SPinTypeSelector.h"
#include "RigEditor/IKRigStructViewer.h"
#include "SPositiveActionButton.h"

#define LOCTEXT_NAMESPACE "RetargetVariablesEditor"


void SIKRetargetVariablesEditor::Construct(const FArguments& InArgs, TSharedRef<FIKRetargetEditorController> InEditorController)
{
    EditorController = InEditorController;
	EditorController.Pin()->SetVariablesView(SharedThis(this));
	
    TObjectPtr<UIKRetargeterController> LocalController = EditorController.Pin()->AssetController;

    ChildSlot
    [
        SNew(SVerticalBox)
        + SVerticalBox::Slot()
        .AutoHeight()
        .Padding(5.0f)
        [
            SNew(SPositiveActionButton)
            .Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
            .Text(LOCTEXT("AddVariableLabel", "Add Variable"))
            .OnClicked_Lambda([LocalController]()
            {
            	if (LocalController)
            	{
            		LocalController->AddNewVariable(NAME_None);
            	}
            	
                return FReply::Handled();
            })
        ]

        + SVerticalBox::Slot()
        .FillHeight(1.0f)
        [
            SAssignNew(VariablesListView, SListView<TSharedPtr<FVariableRow>>)
            .ListItemsSource(&VariableRows)
            .OnGenerateRow(this, &SIKRetargetVariablesEditor::OnGenerateVariableRow)
        ]
    ];

    RefreshList();
}

void SIKRetargetVariablesEditor::NotifyPreChange(FProperty* PropertyAboutToChange)
{
	if (UIKRetargeter* Asset = GetAsset())
	{
		// variables are free-floating properties so we have to inject the asset into the transaction
		Asset->Modify();
	}
}

void SIKRetargetVariablesEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		// only refresh when the value is finally set (not during interactive changes)
		RefreshList();
	}
}

TSharedRef<ITableRow> SIKRetargetVariablesEditor::OnGenerateVariableRow(
	TSharedPtr<FVariableRow> InItem,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<TSharedPtr<FVariableRow>>, OwnerTable)
	[
		SNew(SHorizontalBox)
        
		// rename
		+ SHorizontalBox::Slot()
		.FillWidth(0.3f)
		.VAlign(VAlign_Center)
		[
			SNew(SEditableText)
			.Text(FText::FromName(InItem->PropertyName))
			.OnTextCommitted_Lambda([this, InItem](const FText& NewText, ETextCommit::Type CommitType)
			{
				if (CommitType == ETextCommit::OnEnter)
				{
					if (EditorController.IsValid())
					{
						EditorController.Pin()->AssetController->RenameVariable(InItem->PropertyName, FName(*NewText.ToString()));
					}
				}
			})
		]

		// type selector
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			MakeTypeSelectorForVariable(InItem)
		]

		// value widget (uses single property editor for different types)
		+ SHorizontalBox::Slot()
		.FillWidth(0.4f)
		[
			MakeValueWidgetForVariable(InItem)
		]

		// delete
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.OnClicked_Lambda([this, InItem]()
			{
				if (EditorController.IsValid())
				{
					EditorController.Pin()->AssetController->DeleteVariable(InItem->PropertyName);
				}
				
				return FReply::Handled();
			})
			[
				SNew(SImage).Image(FAppStyle::Get().GetBrush("Icons.Delete"))
			]
		]
	];
}

UIKRetargeter* SIKRetargetVariablesEditor::GetAsset() const
{
	if (EditorController.IsValid())
	{
		return EditorController.Pin()->AssetController->GetAsset();	
	}
	
	return nullptr;
}

void SIKRetargetVariablesEditor::RefreshList()
{
    VariableRows.Empty();
    UIKRetargeter* Asset = GetAsset();
    if (!Asset)
    {
	    return;
    }

    const FRetargetVariableContainer& Container = Asset->GetVariables();
    if (const UPropertyBag* BagStruct = Container.Bag.GetPropertyBagStruct())
    {
        for (const FPropertyBagPropertyDesc& Desc : BagStruct->GetPropertyDescs())
        {
            TSharedPtr<FVariableRow> NewRow = MakeShared<FVariableRow>();
            NewRow->PropertyName = Desc.Name;
            VariableRows.Add(NewRow);
        }
    }

    if (VariablesListView.IsValid())
    {
        VariablesListView->RequestListRefresh();
    }
}

TSharedRef<SWidget> SIKRetargetVariablesEditor::MakeTypeSelectorForVariable(TSharedPtr<FVariableRow> InRow) const
{
	return SNew(SPinTypeSelector, FGetPinTypeTree::CreateStatic([](TArray<TSharedPtr<UEdGraphSchema_K2::FPinTypeTreeInfo>>& TypeTree, ETypeTreeFilter Filter)
		{
			if (const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>())
			{
				// this fills the dropdown with the standard Blueprint types
				K2Schema->GetVariableTypeTree(TypeTree, Filter);
			}
		}))
		.TargetPinType_Lambda([this, InRow]()
		{
			UIKRetargeter* Asset = GetAsset();
			if (!Asset)
			{
				return FEdGraphPinType();
			}
			const FPropertyBagPropertyDesc* Desc = Asset->GetVariables().Bag.FindPropertyDescByName(InRow->PropertyName);
			return Desc ? UE::StructUtils::GetPropertyDescAsPin(*Desc) : FEdGraphPinType();
		})
		.OnPinTypeChanged_Lambda([this, InRow](const FEdGraphPinType& NewPinType)
		{
			if (EditorController.IsValid())
			{
				EditorController.Pin()->AssetController->SetVariableType(InRow->PropertyName, NewPinType);
			}
		})
		.Schema(GetDefault<UEdGraphSchema_K2>())
		.bAllowArrays(false) // for some reason I haven't figured out, this doesn't prevent array types :(
		.SelectorType(SPinTypeSelector::ESelectorType::Partial);
}

TSharedRef<SWidget> SIKRetargetVariablesEditor::MakeValueWidgetForVariable(TSharedPtr<FVariableRow> InRow)
{
	UIKRetargeter* Asset = GetAsset();
	if (!Asset)
	{
		return SNullWidget::NullWidget;
	}

	FRetargetVariableContainer& VarContainer = Asset->GetVariables();
	const UPropertyBag* BagStruct = VarContainer.Bag.GetPropertyBagStruct();
    
	if (!BagStruct)
	{
		return SNullWidget::NullWidget;
	}

	// NOTE: we're using the property editor module to generate a widget for the type stored in the property bag
	// the property bag allows us to store generic variable types, this allows us to edit them

	// the TStructureDataProvider is so the widget knows where to read/write memory
	TSharedPtr<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(BagStruct,VarContainer.Bag.GetMutableValue().GetMemory());
	TSharedPtr<IStructureDataProvider> StructProvider = MakeShared<FStructOnScopeStructureDataProvider>(StructOnScope);

	// create the widget targeting the variable name directly in that struct
	FSinglePropertyParams Params;
	Params.NotifyHook = this; // routes property edit callbacks to this::NotifyPreChange() / NotifyPostChange()
	Params.NamePlacement = EPropertyNamePlacement::Hidden;
	FPropertyEditorModule& EditModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedPtr<ISinglePropertyView> SinglePropView = EditModule.CreateSingleProperty(
		StructProvider, 
		InRow->PropertyName, 
		Params
	);

	return SinglePropView.IsValid() ? SinglePropView.ToSharedRef() : SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE