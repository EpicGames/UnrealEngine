// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/SRetargetAnimAssetsWindow.h"

#include "Animation/AnimSequence.h"
#include "Animation/AnimMontage.h"
#include "AnimPreviewInstance.h"
#include "AssetToolsModule.h"
#include "AssetViewerSettings.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "SSkeletonWidget.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Layout/SSeparator.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Misc/ScopedSlowTask.h"
#include "RetargetEditor/IKRetargeterController.h"
#include "Retargeter/IKRetargeter.h"
#include "Viewports.h"
#include "Animation/AimOffsetBlendSpace.h"
#include "Animation/AimOffsetBlendSpace1D.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Components/SkyLightComponent.h"
#include "Editor/EditorPerProjectUserSettings.h"
#include "RetargetEditor/IKRetargetAnimInstance.h"
#include "RetargetEditor/IKRetargetEditorController.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"
#include "Retargeter/RetargetOps/PinBoneOp.h"
#include "Retargeter/RetargetOps/RootMotionGeneratorOp.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "RigEditor/IKRigAutoCharacterizer.h"
#include "RigEditor/IKRigController.h"
#include "Settings/SkeletalMeshEditorSettings.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/SavePackage.h"
#include "Widgets/Input/SEditableTextBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SRetargetAnimAssetsWindow)

#define LOCTEXT_NAMESPACE "RetargetAnimAssetWindow"

void SBatchExportPathDialog::Construct(const FArguments& InArgs)
{
	BatchContext = InArgs._BatchContext;
	check(BatchContext);
	if(BatchContext->NameRule.FolderPath.IsEmpty())
	{
		BatchContext->NameRule.FolderPath = FString("/Game");
	}

	// path picker
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	FPathPickerConfig PathPickerConfig;
	PathPickerConfig.DefaultPath = BatchContext->NameRule.FolderPath;
	PathPickerConfig.OnPathSelected = FOnPathSelected::CreateLambda([this](const FString& NewPath){BatchContext->NameRule.FolderPath = NewPath;});
	PathPickerConfig.bAddDefaultPath = true;

	// adjust UI based on if we're exporting retarget assets or animation assets
	bExportingRetargetAssets = InArgs._ExportRetargetAssets;
	const FText TitleText = bExportingRetargetAssets ? LOCTEXT("TitleA", "Export Retarget Assets") : LOCTEXT("TitleB", "Export Animations");
	const int32 WindowHeight = bExportingRetargetAssets ? 300 : 650;

	SWindow::Construct(SWindow::FArguments()
	.Title(TitleText)
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	.IsTopmostWindow(true)
	.ClientSize(FVector2D(300, WindowHeight))
	[
		SNew(SVerticalBox)
		
		+ SVerticalBox::Slot()
		.Padding(2)
		[
			SNew(SVerticalBox)
			.IsEnabled_Lambda([this](){return !BatchContext->bUseSourcePath;})

			+SVerticalBox::Slot()
			.FillHeight(1)
			.Padding(3)
			[
				ContentBrowserModule.Get().CreatePathPicker(PathPickerConfig)
			]
			
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2, 3)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DuplicateAndRetarget_Folder", "Export Path: "))
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]

				+SHorizontalBox::Slot()
				.FillWidth(1)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock).Text_Lambda([this]()
					{
						return FText::FromString(BatchContext->NameRule.FolderPath);
					})
				]
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(3)
		[
			SNew(SHorizontalBox)
			.Visibility_Lambda([this]{ return bExportingRetargetAssets ? EVisibility::Collapsed : EVisibility::Visible;})

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.Padding(0.0f, 0.0f, 12.f, 0.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UseSourcePath_Label", "Use Source Path: "))
				.ToolTipText(LOCTEXT("UseSourcePathTooltip", "Places new animation assets in the same location as the source asset."))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					return BatchContext->bUseSourcePath ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					BatchContext->bUseSourcePath = InCheckBoxState == ECheckBoxState::Checked;
				})
			]
		]
		
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Visibility_Lambda([this]{ return bExportingRetargetAssets ? EVisibility::Collapsed : EVisibility::Visible;})
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DuplicateAndRetarget_RenameLabel", "Rename New Assets"))
					.Font(FAppStyle::Get().GetFontStyle("NormalFontBold"))
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 1)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(2, 1)
					[
						SNew(STextBlock).Text(LOCTEXT("DuplicateAndRetarget_Prefix", "Prefix"))
					]

					+SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
							.Text_Lambda([this]()
							{
								return FText::FromString(BatchContext->NameRule.Prefix);
							})
							.OnTextChanged_Lambda([this](const FText& InText)
							{
								BatchContext->NameRule.Prefix = ConvertToCleanString(InText);
								UpdateExampleText();	
							})
							.MinDesiredWidth(100)
							.IsReadOnly(false)
							.RevertTextOnEscape(true)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 1)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(2, 1)
					[
						SNew(STextBlock).Text(LOCTEXT("DuplicateAndRetarget_Suffix", "Suffix"))
					]

					+SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
							.Text_Lambda([this]()
							{
								return FText::FromString(BatchContext->NameRule.Suffix);
							})
							.OnTextChanged_Lambda([this](const FText& InText)
							{
								BatchContext->NameRule.Suffix = ConvertToCleanString(InText);
								UpdateExampleText();	
							})
							.MinDesiredWidth(100)
							.IsReadOnly(false)
							.RevertTextOnEscape(true)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 1)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(2, 1)
					[
						SNew(STextBlock).Text(LOCTEXT("DuplicateAndRetarget_Search", "Search "))
					]

					+SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
							.Text_Lambda([this]()
							{
								return FText::FromString(BatchContext->NameRule.ReplaceFrom);
							})
							.OnTextChanged_Lambda([this](const FText& InText)
							{
								BatchContext->NameRule.ReplaceFrom = ConvertToCleanString(InText);
								UpdateExampleText();	
							})
							.MinDesiredWidth(100)
							.IsReadOnly(false)
							.RevertTextOnEscape(true)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 1)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(2, 1)
					[
						SNew(STextBlock).Text(LOCTEXT("DuplicateAndRetarget_Replace", "Replace "))
					]

					+SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
							.Text_Lambda([this]()
							{
								return FText::FromString(BatchContext->NameRule.ReplaceTo);
							})
							.OnTextChanged_Lambda([this](const FText& InText)
							{
								BatchContext->NameRule.ReplaceTo = ConvertToCleanString(InText);
								UpdateExampleText();	
							})
							.MinDesiredWidth(100)
							.IsReadOnly(false)
							.RevertTextOnEscape(true)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 3)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(5, 5)
					[
						SNew(STextBlock)
						.Text_Lambda([this](){return ExampleText; })
						.Font(FAppStyle::GetFontStyle("Persona.RetargetManager.ItalicFont"))
					]
				]
			]
		]
		
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(5)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))

			+SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SBatchExportPathDialog::OnButtonClick, EAppReturnType::Cancel)
			]
			
			+SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Export", "Export"))
				.OnClicked(this, &SBatchExportPathDialog::OnButtonClick, EAppReturnType::Ok)
			]
		]
	]);

	UpdateExampleText();
}

FReply SBatchExportPathDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	UserResponse = ButtonID;
	RequestDestroyWindow();
	return FReply::Handled();
}

EAppReturnType::Type SBatchExportPathDialog::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

void SBatchExportPathDialog::UpdateExampleText()
{
	const FString ReplaceFrom = FString::Printf(TEXT("Old Name : ***%s***"), *BatchContext->NameRule.ReplaceFrom);
	const FString ReplaceTo = FString::Printf(TEXT("New Name : %s***%s***%s"), *BatchContext->NameRule.Prefix, *BatchContext->NameRule.ReplaceTo, *BatchContext->NameRule.Suffix);
	ExampleText = FText::FromString(FString::Printf(TEXT("%s\n%s"), *ReplaceFrom, *ReplaceTo));
}

FText SBatchExportPathDialog::GetFolderPath() const
{
	return FText::FromString(BatchContext->NameRule.FolderPath);
}

FString SBatchExportPathDialog::ConvertToCleanString(const FText& ToClean)
{
	static TSet<TCHAR> IllegalChars = {' ','$', '&', '^', '/', '\\', '#', '@', '!', '*', '_', '(', ')'};
	
	FString StrToClean = ToClean.ToString();
	for (TCHAR& Char : StrToClean)
	{
		if (IllegalChars.Contains(Char))
		{
			Char = TEXT('_'); // Replace illegal char with underscore
		}
	}

	return MoveTemp(StrToClean);
}

UBatchExportOptions* UBatchExportOptions::GetInstance()
{
	if (!SingletonInstance)
	{
		SingletonInstance = NewObject<UBatchExportOptions>(GetTransientPackage(), UBatchExportOptions::StaticClass());
		SingletonInstance->AddToRoot();
	}
	return SingletonInstance;
}

UBatchExportOptions* UBatchExportOptions::SingletonInstance = nullptr;

SBatchExportOptionsDialog::SBatchExportOptionsDialog()
{
	UserResponse = EAppReturnType::Cancel;
}

void SBatchExportOptionsDialog::Construct(const FArguments& InArgs)
{
	BatchContext = InArgs._BatchContext;
	check(BatchContext);

	FDetailsViewArgs GridDetailsViewArgs;
	GridDetailsViewArgs.bAllowSearch = false;
	GridDetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	GridDetailsViewArgs.bHideSelectionTip = true;
	GridDetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	GridDetailsViewArgs.bShowOptions = false;
	GridDetailsViewArgs.bAllowMultipleTopLevelObjects = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(GridDetailsViewArgs);
	DetailsView->SetObject(UBatchExportOptions::GetInstance());

	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("ExportOptionsWindowTitle", "Batch Export Options"))
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	.IsTopmostWindow(true)
	.ClientSize(FVector2D(450, 200))
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.Padding(2)
		[
			DetailsView
		]
		
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Bottom)
		.Padding(5)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))

			+SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Cancel", "Cancel"))
				.OnClicked(this, &SBatchExportOptionsDialog::OnButtonClick, EAppReturnType::Cancel)
			]
			
			+SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("Export", "Export"))
				.OnClicked(this, &SBatchExportOptionsDialog::OnButtonClick, EAppReturnType::Ok)
			]
		]
	]);
}

EAppReturnType::Type SBatchExportOptionsDialog::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	return UserResponse;
}

FReply SBatchExportOptionsDialog::OnButtonClick(EAppReturnType::Type ButtonID)
{
	// update batch context with the user specified options
	const UBatchExportOptions* ExportOptions = UBatchExportOptions::GetInstance();
	BatchContext->bIncludeReferencedAssets = ExportOptions->bIncludeReferencedAssets;
	BatchContext->bOverwriteExistingFiles = ExportOptions->bOverwriteExistingFiles;
	BatchContext->bRetainAdditiveFlags = ExportOptions->bRetainAdditiveFlags;
	//BatchContext->bExportOnlyAnimatedBones = ExportOptions->bExportOnlyAnimatedBones;
	
	UserResponse = ButtonID;
	RequestDestroyWindow();
	return FReply::Handled();
}

void SRetargetPoseViewport::Construct(const FArguments& InArgs)
{
	BatchContext = InArgs._BatchContext;
	check(BatchContext);
	
	SEditorViewport::Construct(SEditorViewport::FArguments());

	SourceComponent = NewObject<UDebugSkelMeshComponent>();
	TargetComponent = NewObject<UDebugSkelMeshComponent>();
	
	SourceComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;
	TargetComponent->VisibilityBasedAnimTickOption = EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones;

	SourceComponent->SetProcessRootMotionMode(EProcessRootMotionMode::Ignore);
	TargetComponent->SetProcessRootMotionMode(EProcessRootMotionMode::Ignore);

	// setup SOURCE anim instance running a preview node that suppresses root motion
	SourceAnimInstance = NewObject<UIKRetargetAnimInstance>(SourceComponent, TEXT("IKRetargetSourceAnimScriptInstance"));
	SourceAnimInstance->SetRetargetMode(ERetargeterOutputMode::RunRetarget);
	SourceComponent->PreviewInstance = SourceAnimInstance.Get();

	// setup TARGET anim instance running a retargeter that copies input pose from the source component
	TargetAnimInstance = NewObject<UIKRetargetAnimInstance>(TargetComponent, TEXT("IKRetargetTargetAnimScriptInstance"));
	TargetAnimInstance->SetRetargetMode(ERetargeterOutputMode::RunRetarget);
	TargetComponent->PreviewInstance = TargetAnimInstance.Get();
	
	PreviewScene.AddComponent(SourceComponent, FTransform::Identity);
	PreviewScene.AddComponent(TargetComponent, FTransform::Identity);

	FIKRetargetProcessor* Processor = TargetAnimInstance->GetRetargetProcessor();
	if (ensure(Processor))
	{
		Processor->Log.SetLogTarget(BatchContext->IKRetargetAsset);
	}

	SetSkeletalMesh(BatchContext->SourceMesh, ERetargetSourceOrTarget::Source);
	SetRetargetAsset(BatchContext->IKRetargetAsset);
}

void SRetargetPoseViewport::SetSkeletalMesh(USkeletalMesh* InSkeletalMesh, ERetargetSourceOrTarget SourceOrTarget)
{
	AnimThatWasPlaying = SourceAnimInstance->GetAnimationAsset();
	TimeWhenPaused = SourceAnimInstance->GetCurrentTime();
	
	const TObjectPtr<UDebugSkelMeshComponent> Component = SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceComponent : TargetComponent;
	Component->SetSkeletalMesh(InSkeletalMesh);
	Component->EnablePreview(true, nullptr);

	// translate target sufficiently far from source to avoid them touching on X axis (sideways)
	const double Offset = SourceComponent->Bounds.GetBox().Max.X + FMath::Abs(TargetComponent->Bounds.GetBox().Min.X);
	TargetComponent->SetWorldLocation(FVector(Offset, 0.f, 0.f));
	
	// update camera to show both meshes
	const FBoxSphereBounds Bounds = SourceComponent->Bounds + TargetComponent->Bounds;
	Client->FocusViewportOnBox(Bounds.GetBox(), true);
	Client->Invalidate();

	if (AnimThatWasPlaying)
	{
		SourceAnimInstance->SetAnimationAsset(AnimThatWasPlaying);
		SourceAnimInstance->SetPlaying(true);
		SourceAnimInstance->SetPosition(TimeWhenPaused);
	}
}

void SRetargetPoseViewport::SetRetargetAsset(UIKRetargeter* RetargetAsset)
{
	// apply the IK retargeter and give a reference to the source component
	SourceAnimInstance->ConfigureAnimInstance(ERetargetSourceOrTarget::Source, RetargetAsset, nullptr);
	TargetAnimInstance->ConfigureAnimInstance(ERetargetSourceOrTarget::Target, RetargetAsset, SourceComponent);
}

void SRetargetPoseViewport::SetOverrideSets(const TArray<FName>& InOverrideSets)
{
	if (TargetAnimInstance)
	{
		TargetAnimInstance->SetOverrideSets(InOverrideSets);
	}
}

void SRetargetPoseViewport::PlayAnimation(UAnimationAsset* AnimationAsset)
{
	SourceComponent->EnablePreview(true, AnimationAsset);
}

bool SRetargetPoseViewport::IsRetargeterValid()
{
	if (!TargetAnimInstance)
	{
		return false;
	}

	FIKRetargetProcessor* Processor = TargetAnimInstance->GetRetargetProcessor();
	if (!Processor)
	{
		return false;
	}

	return Processor->IsInitialized();
}

FRetargetPoseViewportClient::FRetargetPoseViewportClient(FPreviewScene& InPreviewScene, const TSharedRef<SRetargetPoseViewport>& InRetargetPoseViewport)
	: FEditorViewportClient(nullptr, &InPreviewScene, StaticCastSharedRef<SEditorViewport>(InRetargetPoseViewport))
{
	SetViewMode(VMI_Lit);

	// always composite editor objects after post processing in the editor
	EngineShowFlags.SetCompositeEditorPrimitives(true);
	EngineShowFlags.DisableAdvancedFeatures();

	// update lighting
	const USkeletalMeshEditorSettings* Options = GetDefault<USkeletalMeshEditorSettings>();
	PreviewScene->SetLightDirection(Options->AnimPreviewLightingDirection);
	PreviewScene->SetLightColor(Options->AnimPreviewDirectionalColor);
	PreviewScene->SetLightBrightness(Options->AnimPreviewLightBrightness);

	// add a skylight so that models are visible from all angles
	// TODO, why isn't this working?
	FPreviewSceneProfile& DefaultProfile = UAssetViewerSettings::Get()->Profiles[GetMutableDefault<UEditorPerProjectUserSettings>()->AssetViewerProfileIndex];
	DefaultProfile.LoadProfileObjects();
	UTextureCube* CubeMap = DefaultProfile.EnvironmentCubeMap.Get();
	PreviewScene->SkyLight->SetVisibility(true, false);
	PreviewScene->SetSkyCubemap(CubeMap);
	PreviewScene->SetSkyBrightness(1.f); // tried up to 250... nothing

	// setup defaults for the common draw helper.
	DrawHelper.bDrawPivot = false;
	DrawHelper.bDrawWorldBox = false;
	DrawHelper.bDrawKillZ = false;
	DrawHelper.bDrawGrid = true;
	DrawHelper.GridColorAxis = FColor(40, 40, 40);
	DrawHelper.GridColorMajor = FColor(20, 20, 20);
	DrawHelper.GridColorMinor =  FColor(10, 10, 10);
	DrawHelper.PerspectiveGridSize = UE_OLD_HALF_WORLD_MAX1;
}

void FRetargetPoseViewportClient::Tick(float DeltaTime)
{
	if (PreviewScene)
	{
		PreviewScene->GetWorld()->Tick(LEVELTICK_All, DeltaTime);
	}

	FEditorViewportClient::Tick(DeltaTime);
}

FProceduralRetargetAssets::FProceduralRetargetAssets()
{
	FName Name = FName("BatchRetargeter");
	Name = MakeUniqueObjectName(GetTransientPackage(), UIKRetargeter::StaticClass(), Name, EUniqueObjectNameOptions::GloballyUnique);
	Retargeter = NewObject<UIKRetargeter>( GetTransientPackage(), Name, RF_Public | RF_Standalone);
}

void FProceduralRetargetAssets::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SourceIKRig);
	Collector.AddReferencedObject(TargetIKRig);
	Collector.AddReferencedObject(Retargeter);
}

void FProceduralRetargetAssets::AutoGenerateIKRigAsset(USkeletalMesh* Mesh, ERetargetSourceOrTarget SourceOrTarget)
{
	TObjectPtr<UIKRigDefinition>* IKRig = SourceOrTarget == ERetargetSourceOrTarget::Source ? &SourceIKRig : &TargetIKRig;
	FName AssetName = SourceOrTarget == ERetargetSourceOrTarget::Source ? FName("BatchRetargetSourceIKRig") : FName("BatchRetargetTargetIKRig");
	AssetName = MakeUniqueObjectName(GetTransientPackage(), UIKRigDefinition::StaticClass(), AssetName, EUniqueObjectNameOptions::GloballyUnique);
	*IKRig = NewObject<UIKRigDefinition>( GetTransientPackage(), AssetName, RF_Public | RF_Standalone);

	if (!Mesh)
	{
		return;
	}

	// auto-setup retarget chains and IK for the given mesh
	const UIKRigController* Controller = UIKRigController::GetController(*IKRig);
	
	Controller->SetSkeletalMesh(Mesh);
	FAutoCharacterizeResults& CharacterizationResults = SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceCharacterizationResults : TargetCharacterizationResults;
	Controller->AutoGenerateRetargetDefinition(CharacterizationResults);
	Controller->SetRetargetDefinition(CharacterizationResults.AutoRetargetDefinition.RetargetDefinition);
	FAutoFBIKResults IKResults = SourceOrTarget == ERetargetSourceOrTarget::Source ? SourceIKResults : TargetIKResults;
	Controller->AutoGenerateFBIK(IKResults);

	// assign to the retargeter
	const UIKRetargeterController* RetargetController = UIKRetargeterController::GetController(Retargeter);
	FScopedReinitializeIKRetargeter ReinitializeRetargeter(RetargetController);
	RetargetController->SetIKRig(SourceOrTarget, *IKRig);
}

void FProceduralRetargetAssets::AutoGenerateIKRetargetAsset()
{
	if (!(SourceIKRig && SourceIKRig->GetPreviewMesh() && TargetIKRig && TargetIKRig->GetPreviewMesh()))
	{
		return;
	}

	const UIKRetargeterController* RetargetController = UIKRetargeterController::GetController(Retargeter);
	
	FScopedReinitializeIKRetargeter Reinitialize(RetargetController, ERetargetRefreshMode::ProcessorAndOpStack);

	// re-assign both IK Rigs
	RetargetController->SetIKRig(ERetargetSourceOrTarget::Source, SourceIKRig);
	RetargetController->SetIKRig(ERetargetSourceOrTarget::Target, TargetIKRig);

	// clear the op stack and add default ops
	// NOTE: this must be done BEFORE generating the retarget pose since it relies on the first op chain mapping
	RetargetController->RemoveAllOps();
	RetargetController->AddDefaultOps();

	// use only exact chain mapping in the auto-retarget (default is fuzzy which can map toes to fingers and other chaos)
	FIKRetargetFKChainsOp* FKChainOp = RetargetController->GetFirstRetargetOpOfType<FIKRetargetFKChainsOp>();
	constexpr bool bForceRemap = true;
	FKChainOp->Settings.ChainMapping.AutoMapChains(EAutoMapChainType::Exact, bForceRemap);

	// reset and regenerate the target retarget pose
	const FName TargetRetargetPoseName = RetargetController->GetCurrentRetargetPoseName(ERetargetSourceOrTarget::Target);
	RetargetController->ResetRetargetPose(TargetRetargetPoseName, TArray<FName>() /* all bones if empty */, ERetargetSourceOrTarget::Target);
	RetargetController->AutoAlignAllBones(ERetargetSourceOrTarget::Target);

	// templates records a list of bones to exclude from the auto pose, so we reset them now; immediately after auto-aligning the whole skeleton.
	// by default this is used on biped feet where we want the feet to remain at their flat / default retarget pose
	const TArray<FName>& BonesToExcludeFromAutoPose = TargetCharacterizationResults.AutoRetargetDefinition.BonesToExcludeFromAutoPose;
	if (!BonesToExcludeFromAutoPose.IsEmpty())
	{
		const FName CurrentPose = RetargetController->GetCurrentRetargetPoseName(ERetargetSourceOrTarget::Target);
		RetargetController->ResetRetargetPose(
			CurrentPose,
			TargetCharacterizationResults.AutoRetargetDefinition.BonesToExcludeFromAutoPose,
			ERetargetSourceOrTarget::Target);
	}

	// disable IK pass because auto-IK is not yet robust enough, still useful for manual editing though
	FIKRetargetRunIKRigOp* RunIKOp = RetargetController->GetFirstRetargetOpOfType<FIKRetargetRunIKRigOp>();
	RunIKOp->SetEnabled(false);
	
	// setup the root motion generator (but only if the skeleton has a root bone separate from the pelvis)
	FIKRetargetRootMotionOp* RootOp = RetargetController->GetFirstRetargetOpOfType<FIKRetargetRootMotionOp>();
	const FName TargetSkeletonRoot = TargetIKRig->GetSkeleton().BoneNames[0];
	const FName TargetPelvis = TargetCharacterizationResults.AutoRetargetDefinition.RetargetDefinition.PelvisBone;
	if (TargetSkeletonRoot != TargetPelvis)
	{
		// if the source does not have a dedicated root bone, configure the root motion to be generated from the pelvis motion on target
		const FName SourceSkeletonRoot = SourceIKRig->GetSkeleton().BoneNames[0];
		const FName SourcePelvis = SourceCharacterizationResults.AutoRetargetDefinition.RetargetDefinition.PelvisBone;
		if (SourceSkeletonRoot == SourcePelvis)
		{
			RootOp->Settings.RootMotionSource = ERootMotionSource::GenerateFromTargetPelvis;
			RootOp->Settings.RootHeightSource = ERootMotionHeightSource::SnapToGround;
			RootOp->Settings.bRotateWithPelvis = true;
			RootOp->Settings.bMaintainOffsetFromPelvis = true;
		}
	}
	else
	{
		// the pelvis IS the root, so we let the Pelvis op handle "root" motion
		RootOp->SetEnabled(false);
	}
	
	// setup a "Pin Bone Op" to pin IK bones to the new target locations
	RetargetController->AddRetargetOp(FIKRetargetPinBoneOp::StaticStruct());
	FIKRetargetPinBoneOp* PinBoneOp = RetargetController->GetFirstRetargetOpOfType<FIKRetargetPinBoneOp>();
	PinBoneOp->Settings.TranslationMode = EPinBoneTranslationMode::CopyGlobalPosition;
	PinBoneOp->Settings.RotationMode = EPinBoneRotationMode::CopyGlobalRotation;
	const TArray<FBoneToPin>& AllBonesToPin = TargetCharacterizationResults.AutoRetargetDefinition.BonesToPin.GetBonesToPin();
	for (const FBoneToPin& BonesToPin : AllBonesToPin)
	{
		FPinBoneData PinBoneData;
		PinBoneData.BoneToCopyFrom.BoneName = BonesToPin.BoneToPinTo;
		PinBoneData.BoneToCopyTo.BoneName = BonesToPin.BoneToPin;
		PinBoneOp->Settings.BonesToPin.Add(PinBoneData);
	}
}

TSharedRef<FEditorViewportClient> SRetargetPoseViewport::MakeEditorViewportClient()
{
	TSharedPtr<FEditorViewportClient> EditorViewportClient = MakeShareable(new FRetargetPoseViewportClient(PreviewScene, SharedThis(this)));

	EditorViewportClient->ViewportType = LVT_Perspective;
	EditorViewportClient->bSetListenerPosition = false;
	EditorViewportClient->SetViewLocation(EditorViewportDefs::DefaultPerspectiveViewLocation);
	EditorViewportClient->SetViewRotation(EditorViewportDefs::DefaultPerspectiveViewRotation);

	EditorViewportClient->SetRealtime(true);
	EditorViewportClient->SetViewMode(VMI_Lit);

	return EditorViewportClient.ToSharedRef();
}

UBatchRetargetSettings* UBatchRetargetSettings::GetInstance()
{
	if (!SingletonInstance)
	{
		SingletonInstance = NewObject<UBatchRetargetSettings>(GetTransientPackage(), UBatchRetargetSettings::StaticClass());
		SingletonInstance->AddToRoot();
	}
	return SingletonInstance;
}

UBatchRetargetSettings* UBatchRetargetSettings::SingletonInstance = nullptr;

void SRetargetExporterAssetBrowser::Construct(const FArguments& InArgs, const TSharedRef<SRetargetAnimAssetsWindow> InRetargetWindow)
{
	RetargetWindow = InRetargetWindow;
	
	ChildSlot
	[
		SAssignNew(AssetBrowserBox, SBox)
	];

	RefreshView();
}

void SRetargetExporterAssetBrowser::RefreshView()
{
	// setup filtering
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimBlueprint::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimMontage::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UPoseAsset::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UBlendSpace::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UBlendSpace1D::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UAimOffsetBlendSpace::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UAimOffsetBlendSpace1D::StaticClass()->GetClassPathName());
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.DefaultFilterMenuExpansion = EAssetTypeCategories::Animation;
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SRetargetExporterAssetBrowser::OnShouldFilterAsset);
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetSelected::CreateSP(this, &SRetargetExporterAssetBrowser::OnAssetDoubleClicked);
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = false;
	AssetPickerConfig.bShowPathInColumnView = false;
	AssetPickerConfig.bShowTypeInColumnView = false;
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::ItemDiskSize.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::VirtualizedData.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Path"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("RevisionControl"));

	// hide all asset registry columns by default (we only really want the name and path)
	UObject* AnimSequenceDefaultObject = UAnimSequence::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(AnimSequenceDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	AnimSequenceDefaultObject->GetAssetRegistryTags(TagsContext);
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		AssetPickerConfig.HiddenColumnNames.Add(TagPair.Key.ToString());
	}

	// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	AssetBrowserBox->SetContent(ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig));
}

void SRetargetExporterAssetBrowser::GetSelectedAssets(TArray<FAssetData>& OutSelectedAssets) const
{
	OutSelectedAssets = GetCurrentSelectionDelegate.Execute();
}

bool SRetargetExporterAssetBrowser::AreAnyAssetsSelected() const
{
	return !GetCurrentSelectionDelegate.Execute().IsEmpty();
}

void SRetargetExporterAssetBrowser::OnAssetDoubleClicked(const FAssetData& AssetData)
{
	if (!AssetData.GetAsset())
	{
		return;
	}

	UAnimationAsset* NewAnimationAsset = Cast<UAnimationAsset>(AssetData.GetAsset());
	if (!NewAnimationAsset)
	{
		return;
	}

	const TSharedPtr<SRetargetPoseViewport> Viewport = RetargetWindow.Get()->GetViewport();
	if (!ensure(Viewport))
	{
		return;
	}

	Viewport->PlayAnimation(NewAnimationAsset);
}

bool SRetargetExporterAssetBrowser::OnShouldFilterAsset(const FAssetData& AssetData)
{
	// is this an animation asset?
	const bool bIsAnimAsset = AssetData.IsInstanceOf(UAnimationAsset::StaticClass());
	const bool bIsAnimBlueprint = AssetData.IsInstanceOf(UAnimBlueprint::StaticClass());
	if (!(bIsAnimAsset || bIsAnimBlueprint))
	{
		return true;
	}
	
	const TObjectPtr<UBatchRetargetSettings> BatchRetargetSettings = UBatchRetargetSettings::GetInstance();
	if (!ensure(BatchRetargetSettings))
	{
		return true;
	}
	
	if (!BatchRetargetSettings->SourceSkeletalMesh)
	{
		return true;
	}

	const USkeleton* DesiredSkeleton = BatchRetargetSettings->SourceSkeletalMesh->GetSkeleton();
	if (!DesiredSkeleton)
	{
		return true;
	}

	if (bIsAnimBlueprint)
	{
		const FAssetDataTagMapSharedView::FFindTagResult Result = AssetData.TagsAndValues.FindTag("TargetSkeleton");
		if (!Result.IsSet())
		{
			return true;
		}

		return !DesiredSkeleton->IsCompatibleForEditor(Result.GetValue());
	}

	return !DesiredSkeleton->IsCompatibleForEditor(AssetData);
}

SRetargetAnimAssetsWindow::SRetargetAnimAssetsWindow()
{
	// get the global settings uobject
	Settings = UBatchRetargetSettings::GetInstance();

	// reset settings to defaults when opening a new dialog
	Settings->bAutoGenerateRetargeter = true;
	Settings->RetargetAsset = nullptr;
	Settings->OverrideSetToApply = NAME_None;

	// assign default retargeter
	BatchContext.IKRetargetAsset = ProceduralAssets.Retargeter;

	// register log name
	Log.SetLogTarget(ProceduralAssets.Retargeter);
}

TSharedPtr<SWindow> SRetargetAnimAssetsWindow::Window;

void SRetargetAnimAssetsWindow::Construct(const FArguments& InArgs)
{
	FDetailsViewArgs GridDetailsViewArgs;
	GridDetailsViewArgs.bAllowSearch = false;
	GridDetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	GridDetailsViewArgs.bHideSelectionTip = true;
	GridDetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	GridDetailsViewArgs.bShowOptions = false;
	GridDetailsViewArgs.bAllowMultipleTopLevelObjects = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(GridDetailsViewArgs);
	DetailsView->SetObject(Settings);
	DetailsView->OnFinishedChangingProperties().AddSP(this, &SRetargetAnimAssetsWindow::OnFinishedChangingSelectionProperties);
	
	this->ChildSlot
	[
		SNew (SHorizontalBox)
		
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillWidth(1.f)
		[

			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+ SSplitter::Slot()
			.Value(0.6f)
			[
				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+ SSplitter::Slot()
				.Value(0.8f)
				[
					SAssignNew(Viewport, SRetargetPoseViewport).BatchContext(&BatchContext)
				]
				+ SSplitter::Slot()
				.Value(0.2f)
				[
					SAssignNew(LogView, SIKRigOutputLog, Log.GetLogTarget())
				]
			]
			+ SSplitter::Slot()
			.Value(0.4f)
			[

				SNew(SSplitter)
				.Orientation(Orient_Vertical)
				+ SSplitter::Slot()
				.Value(0.4f)
				[
					DetailsView
				]
				+ SSplitter::Slot()
				.Value(0.6f)
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					[
						SAssignNew(AssetBrowser, SRetargetExporterAssetBrowser, SharedThis(this))
					]

					+SVerticalBox::Slot()
					.Padding(2)
					.AutoHeight()
					[
						SNew(SWarningOrErrorBox)
						.Visibility_Lambda([this]{return GetWarningVisibility(); })
						.MessageStyle(EMessageStyle::Warning)
						.Message_Lambda([this] { return GetWarningText(); })
					]
					
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(2)
					[
						SNew(SUniformGridPanel)
						.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
						.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
						.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
						+SUniformGridPanel::Slot(0, 0)
						[
							SNew(SButton).HAlign(HAlign_Center)
							.Text(LOCTEXT("ExportAssets", "Export Retarget Assets"))
							.IsEnabled(this, &SRetargetAnimAssetsWindow::CanExportRetargetAssets)
							.OnClicked(this, &SRetargetAnimAssetsWindow::OnExportRetargetAssets)
							.HAlign(HAlign_Center)
							.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						]
						+SUniformGridPanel::Slot(1, 0)
						[
							SNew(SButton).HAlign(HAlign_Center)
							.Text(LOCTEXT("ExportAnims", "Export Animations"))
							.IsEnabled(this, &SRetargetAnimAssetsWindow::CanExportAnimations)
							.OnClicked(this, &SRetargetAnimAssetsWindow::OnExportAnimations)
							.HAlign(HAlign_Center)
							.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						]	
					]
				]
			]
		]
	];

	LogView.Get()->ClearLog();
}

void SRetargetAnimAssetsWindow::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Settings);
}

void SRetargetAnimAssetsWindow::OnFinishedChangingSelectionProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (!PropertyChangedEvent.Property)
	{
		return;
	}

	const bool ChangedMesh = PropertyChangedEvent.Property->GetName() == "SourceSkeletalMesh" || PropertyChangedEvent.Property->GetName() == "TargetSkeletalMesh";
	if (ChangedMesh)
	{
		SetAssets(Settings->SourceSkeletalMesh, Settings->TargetSkeletalMesh, Settings->RetargetAsset);
	}

	const bool ChangedRetargeter = PropertyChangedEvent.Property->GetName() == "RetargetAsset" || PropertyChangedEvent.Property->GetName() == "bAutoGenerateRetargeter";
	if (ChangedRetargeter)
	{
		USkeletalMesh* TargetMesh = Settings->TargetSkeletalMesh;
		if (Settings->RetargetAsset && !Settings->TargetSkeletalMesh)
		{
			TargetMesh = Settings->RetargetAsset->GetPreviewMesh(ERetargetSourceOrTarget::Target);
		}

		// clear override set if it no longer exists on the new retargeter
		if (Settings->RetargetAsset)
		{
			const TMap<FName, FRetargetOverrideSet>& ValidSets = Settings->RetargetAsset->GetOverrideSets();
			if (!Settings->OverrideSetToApply.IsNone() && !ValidSets.Contains(Settings->OverrideSetToApply))
			{
				Settings->OverrideSetToApply = NAME_None;
			}
		}
		else
		{
			Settings->OverrideSetToApply = NAME_None;
		}

		SetAssets(Settings->SourceSkeletalMesh, TargetMesh, Settings->RetargetAsset);
	}

	const bool ChangedOverrideSet = PropertyChangedEvent.Property->GetName() == "OverrideSetToApply";
	if (ChangedOverrideSet)
	{
		ApplyOverrideSetToPreview();
	}
}

bool SRetargetAnimAssetsWindow::CanExportAnimations() const
{
	return GetCurrentState() == EBatchRetargetUIState::READY_TO_EXPORT;
}

FReply SRetargetAnimAssetsWindow::OnExportAnimations()
{
	// get the export path from user
	const TSharedRef<SBatchExportPathDialog> PathDialog = SNew(SBatchExportPathDialog).BatchContext(&BatchContext).ExportRetargetAssets(false);
	if(PathDialog->ShowModal() == EAppReturnType::Cancel)
	{
		return FReply::Handled();
	}

	// get the export options from user
	const TSharedRef<SBatchExportOptionsDialog> OptionsDialog = SNew(SBatchExportOptionsDialog).BatchContext(&BatchContext);
	if(OptionsDialog->ShowModal() == EAppReturnType::Cancel)
	{
		return FReply::Handled();
	}

	// get the assets to export
	TArray<FAssetData> SelectedAssets;
	AssetBrowser->GetSelectedAssets(SelectedAssets);
	BatchContext.AssetsToRetarget.Reset();
	for (const FAssetData& Asset : SelectedAssets)
	{
		if (UObject* AssetObject = Cast<UObject>(Asset.GetAsset()))
		{
			BatchContext.AssetsToRetarget.Add(AssetObject);	
		}
	}

	// apply override set to batch context
	BatchContext.OverrideSetNames.Reset();
	if (!Settings->OverrideSetToApply.IsNone())
	{
		BatchContext.OverrideSetNames.Add(Settings->OverrideSetToApply);
	}

	// run the batch retarget
	const TStrongObjectPtr<UIKRetargetBatchOperation> BatchOperation(NewObject<UIKRetargetBatchOperation>());
	BatchOperation->RunRetarget(BatchContext);
	return FReply::Handled();
}

bool SRetargetAnimAssetsWindow::CanExportRetargetAssets() const
{
	const EBatchRetargetUIState CurrentState = GetCurrentState();
	return Settings->bAutoGenerateRetargeter &&
		(CurrentState == EBatchRetargetUIState::READY_TO_EXPORT || CurrentState == EBatchRetargetUIState::NO_ANIMATIONS_SELECTED);
}

FReply SRetargetAnimAssetsWindow::OnExportRetargetAssets()
{
	TSharedRef<SBatchExportPathDialog> PathDialog = SNew(SBatchExportPathDialog).BatchContext(&BatchContext).ExportRetargetAssets(true);
	if(PathDialog->ShowModal() == EAppReturnType::Cancel)
	{
		return FReply::Handled();
	}

	auto SaveAssetToDisk = [](UObject* Asset, FString& AssetName, const FString& FolderPath)
	{
		// create a unique package name and path
		const FString BasePackageName = FolderPath / AssetName;
		const FString Suffix = "";
		FString UniquePackageName, UniqueAssetName;
		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
		AssetTools.CreateUniqueAssetName(BasePackageName, Suffix, UniquePackageName, UniqueAssetName);
		
		// duplicate the asset
		FObjectDuplicationParameters ObjParameters(Asset, GetTransientPackage());
		ObjParameters.DestName = MakeUniqueObjectName(GetTransientPackage(), Asset->GetClass(), *UniqueAssetName);
		ObjParameters.ApplyFlags = RF_Transactional;
		UObject* AssetToSave = StaticDuplicateObjectEx(ObjParameters);
			
		// create a new asset package
		UPackage* Package = CreatePackage(*UniquePackageName);
		AssetToSave->Rename(*UniqueAssetName, Package);
		FAssetRegistryModule::AssetCreated(AssetToSave);
		Package->MarkPackageDirty();

		return AssetToSave;
	};

	// export procedural assets to disk
	UIKRetargeterController* RetargetController = UIKRetargeterController::GetController(BatchContext.IKRetargetAsset);
	UIKRigDefinition* SourceIKRig = RetargetController->GetIKRigWriteable(ERetargetSourceOrTarget::Source);
	UIKRigDefinition* TargetIKRig = RetargetController->GetIKRigWriteable(ERetargetSourceOrTarget::Target);

	// save the retargeter
	FString RetargetAssetName = TEXT("RTG_AutoGenerated");
	UObject* ExportedRetargeterObj = SaveAssetToDisk(BatchContext.IKRetargetAsset, RetargetAssetName, BatchContext.NameRule.FolderPath);
	// save the SOURCE IK Rig
	FString SourceIKRigAssetName = TEXT("IK_AutoGeneratedSource");
	UObject* ExportedSourceIKRigObj = SaveAssetToDisk(SourceIKRig, SourceIKRigAssetName, BatchContext.NameRule.FolderPath);
	// save the TARGET IK Rig
	FString TargetIKRigAssetName = TEXT("IK_AutoGeneratedTarget");
	UObject* ExportedTargetIKRigObj = SaveAssetToDisk(TargetIKRig, TargetIKRigAssetName, BatchContext.NameRule.FolderPath);

	// modify exported retargeter to point to exported ik rigs
	UIKRetargeter* ExportedRetargeter = CastChecked<UIKRetargeter>(ExportedRetargeterObj);
	UIKRigDefinition* ExportedSourceIKRig = CastChecked<UIKRigDefinition>(ExportedSourceIKRigObj);
	UIKRigDefinition* ExportedTargetIKRig = CastChecked<UIKRigDefinition>(ExportedTargetIKRigObj);
	UIKRetargeterController* ExportedController = UIKRetargeterController::GetController(ExportedRetargeter);
	ExportedController->SetIKRig(ERetargetSourceOrTarget::Source, ExportedSourceIKRig);
	ExportedController->SetIKRig(ERetargetSourceOrTarget::Target, ExportedTargetIKRig);

	// select all new assets and show in the content browser
	TArray<FAssetData> NewAssets;
	NewAssets.Add(FAssetData(ExportedRetargeterObj));
	NewAssets.Add(FAssetData(ExportedSourceIKRigObj));
	NewAssets.Add(FAssetData(ExportedTargetIKRigObj));
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets(NewAssets);
	
	return FReply::Handled();
}

EBatchRetargetUIState SRetargetAnimAssetsWindow::GetCurrentState() const
{
	if (!(BatchContext.SourceMesh && BatchContext.TargetMesh))
	{
		return EBatchRetargetUIState::MISSING_MESH;
	}

	if (!Viewport->IsRetargeterValid())
	{
		return Settings->bAutoGenerateRetargeter ? EBatchRetargetUIState::AUTO_RETARGET_INVALID : EBatchRetargetUIState::MANUAL_RETARGET_INVALID;
	}

	if (!AssetBrowser->AreAnyAssetsSelected())
	{
		return EBatchRetargetUIState::NO_ANIMATIONS_SELECTED;
	}
	
	return EBatchRetargetUIState::READY_TO_EXPORT;
}

EVisibility SRetargetAnimAssetsWindow::GetWarningVisibility() const
{
	return GetCurrentState() == EBatchRetargetUIState::READY_TO_EXPORT ? EVisibility::Collapsed : EVisibility::Visible;
}

FText SRetargetAnimAssetsWindow::GetWarningText() const
{
	EBatchRetargetUIState CurrentState = GetCurrentState();
	switch (CurrentState)
	{
	case EBatchRetargetUIState::MISSING_MESH:
		return LOCTEXT("MissingMesh", "Assign a source and target mesh to transfer animation between.");
	case EBatchRetargetUIState::AUTO_RETARGET_INVALID:
		return LOCTEXT("AutoInvalid", "Auto-generated retargeter was invalid. See output for details.");
	case EBatchRetargetUIState::MANUAL_RETARGET_INVALID:
		return LOCTEXT("UserInvalid", "User supplied retargeter was invalid. See output for details.");
	case EBatchRetargetUIState::NO_ANIMATIONS_SELECTED:
		return LOCTEXT("NoAnimsSelcted", "Ready to export! Select animations to export.");
	case EBatchRetargetUIState::READY_TO_EXPORT:
		return FText::GetEmpty(); // message hidden when warnings are all dealt with
	default:
		checkNoEntry();
	};

	return FText::GetEmpty();
}

TArray<FString> UBatchRetargetSettings::GetOverrideSetOptions() const
{
	TArray<FString> Options;
	Options.Add(TEXT("None"));

	if (RetargetAsset)
	{
		const TMap<FName, FRetargetOverrideSet>& AllSets = RetargetAsset->GetOverrideSets();
		for (const TTuple<FName, FRetargetOverrideSet>& Entry : AllSets)
		{
			Options.Add(Entry.Key.ToString());
		}
	}

	return Options;
}

void SRetargetAnimAssetsWindow::ApplyOverrideSetToPreview()
{
	BatchContext.OverrideSetNames.Reset();
	if (!Settings->OverrideSetToApply.IsNone())
	{
		BatchContext.OverrideSetNames.Add(Settings->OverrideSetToApply);
	}

	if (Viewport.IsValid())
	{
		Viewport->SetOverrideSets(BatchContext.OverrideSetNames);
	}
}

void SRetargetAnimAssetsWindow::ShowWindow(TArray<UObject*> InSelectedAssets)
{	
	if(Window.IsValid())
	{
		FSlateApplication::Get().DestroyWindowImmediately(Window.ToSharedRef());
	}
	
	Window = SNew(SWindow)
		.Title(LOCTEXT("RetargetAnimWindowTitle", "Retarget Animations"))
		.SupportsMinimize(true)
		.SupportsMaximize(true)
		.HasCloseButton(true)
		.IsTopmostWindow(false)
		.ClientSize(FVector2D(1280, 720))
		.SizingRule(ESizingRule::UserSized);
	
	TSharedPtr<class SRetargetAnimAssetsWindow> DialogWidget;
	TSharedPtr<SBorder> DialogWrapper =
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(4.0f)
		[
			SAssignNew(DialogWidget, SRetargetAnimAssetsWindow)
		];
	Window->SetOnWindowClosed(FOnWindowClosed::CreateLambda([](const TSharedRef<SWindow>&){Window = nullptr;}));
	Window->SetContent(DialogWrapper.ToSharedRef());

	// load selected assets and source mesh into the UI
	{
		// filter out any selected asset that is not an animation asset
		TArray<TWeakObjectPtr<UObject>> AssetsToRetarget;
		for (UObject* SelectedAsset : InSelectedAssets)
		{
			AssetsToRetarget.Add(SelectedAsset);
		}
		// set default assets to retarget
		DialogWidget->BatchContext.AssetsToRetarget = AssetsToRetarget;

		// set default skeletal mesh
		if (!AssetsToRetarget.IsEmpty())
		{
			USkeletalMesh* Mesh = nullptr;
			USkeleton* Skeleton = nullptr;
			if (UAnimationAsset* AnimationAsset = Cast<UAnimationAsset>(AssetsToRetarget[0].Get()))
			{
				Mesh = AnimationAsset->GetPreviewMesh();
				Skeleton = AnimationAsset->GetSkeleton();
				
			}
			else if (UAnimBlueprint* ABP = Cast<UAnimBlueprint>(AssetsToRetarget[0].Get()))
			{
				Skeleton = ABP->TargetSkeleton;
			}

			if (!Mesh && Skeleton)
            {
            	Mesh = Skeleton->GetPreviewMesh();
            	if (!Mesh)
            	{
            		Mesh = Skeleton->FindCompatibleMesh();
            	}
            }
			
			DialogWidget->Settings->SourceSkeletalMesh = Mesh;
			DialogWidget->SetAssets(Mesh, nullptr, nullptr);
		}
	}
	
	FSlateApplication::Get().AddWindow(Window.ToSharedRef());
}

void SRetargetAnimAssetsWindow::SetAssets(
	USkeletalMesh* SourceMesh,
	USkeletalMesh* TargetMesh,
	UIKRetargeter* Retargeter)
{
	LogView->ClearLog();
	
	const bool ReplacedSourceMesh = BatchContext.SourceMesh != SourceMesh;

	Settings->SourceSkeletalMesh = SourceMesh;
	Settings->TargetSkeletalMesh = TargetMesh;
	Settings->RetargetAsset = Retargeter;
	
	BatchContext.SourceMesh = SourceMesh;
	BatchContext.TargetMesh = TargetMesh;
	BatchContext.IKRetargetAsset = Retargeter;

	// auto generate procedural assets
	if (Settings->bAutoGenerateRetargeter)
	{
		// update procedurally generated IK Rig and retargeter (regenerates retarget pose with new mesh)
		ProceduralAssets.AutoGenerateIKRigAsset(SourceMesh, ERetargetSourceOrTarget::Source);
		ProceduralAssets.AutoGenerateIKRigAsset(TargetMesh, ERetargetSourceOrTarget::Target);
		ProceduralAssets.AutoGenerateIKRetargetAsset();
		BatchContext.IKRetargetAsset = ProceduralAssets.Retargeter;
	}
	
	// update viewport world to show new meshes
	Viewport->SetSkeletalMesh(SourceMesh, ERetargetSourceOrTarget::Source);
	Viewport->SetSkeletalMesh(TargetMesh, ERetargetSourceOrTarget::Target);
	Viewport->SetRetargetAsset(BatchContext.IKRetargetAsset);

	// apply override set to preview
	ApplyOverrideSetToPreview();

	ShowAssetWarnings();
	
	if (ReplacedSourceMesh)
	{
		AssetBrowser.Get()->RefreshView();
	}
}

void SRetargetAnimAssetsWindow::ShowAssetWarnings()
{
	// missing source mesh
	if (!Settings->SourceSkeletalMesh)
	{
		Log.LogError(LOCTEXT( "MissingSourceMeshError", "No source mesh assigned."));
	}

	// missing target mesh
	if (!Settings->TargetSkeletalMesh)
	{
		Log.LogError(LOCTEXT( "MissinTargetMeshError", "No target mesh assigned."));
	}

	// auto retarget results
	if (Settings->bAutoGenerateRetargeter)
	{
		// source auto-characterize results
		if (Settings->SourceSkeletalMesh)
		{
			if (ProceduralAssets.SourceCharacterizationResults.bUsedTemplate)
			{
				Log.LogInfo(FText::Format(LOCTEXT("UsingSourceTemplate", "Using {0} template for source skeleton."), FText::FromName(ProceduralAssets.SourceCharacterizationResults.BestTemplateName)));
			}
			else
			{
				Log.LogError(LOCTEXT("NoSourceTemplate", "Trying to auto-generate a retargeter but no template was compatible with the source mesh."));
			}
		}
		
		// target auto-characterize results
		if (Settings->TargetSkeletalMesh)
		{
			if (ProceduralAssets.TargetCharacterizationResults.bUsedTemplate)
			{
				Log.LogInfo(FText::Format(LOCTEXT("UsingTargetTemplate", "Using {0} template for target skeleton."), FText::FromName(ProceduralAssets.TargetCharacterizationResults.BestTemplateName)));
			}
			else
			{
				Log.LogError(LOCTEXT("NoTargetTemplate", "Trying to auto-generate a retargeter but no template was compatible with the target mesh."));
			}
		}
	}
	else
	{
		if (!Settings->RetargetAsset && !Settings->bAutoGenerateRetargeter)
		{
			Log.LogError(LOCTEXT("NoRetargeter", "Not using auto-generated retargeter and no IK Retargeter asset was provided."));
		}
	}
}

#undef LOCTEXT_NAMESPACE
