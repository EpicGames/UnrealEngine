// Copyright Epic Games, Inc. All Rights Reserved.

#include "RetargetEditor/SRetargetOpStack.h"

#include "UObject/UObjectIterator.h"
#include "SPositiveActionButton.h"
#include "RetargetEditor/IKRetargetAnimInstance.h"
#include "RetargetEditor/IKRetargetEditorController.h"
#include "RetargetEditor/IKRetargetEditorStyle.h"
#include "Retargeter/IKRetargetOps.h"
#include "Retargeter/IKRetargetProfile.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "SRetargetOpStack"

float SRetargetOpItem::OpHorizontalPadding = 3.0f;;
float SRetargetOpItem::OpVerticalPadding = 2.0f;

TSharedRef<ITableRow> FRetargetOpStackElement::MakeListRowWidget(
	const TSharedRef<STableViewBase>& InOwnerTable,
	TSharedRef<FRetargetOpStackElement> InStackElement,
	TSharedPtr<SRetargetOpList> InOpListWidget)
{
	if (InStackElement.Get().bCanHaveChildren)
	{
		return SNew(SParentRetargetOpItem, InOwnerTable)
			.InStackElement(InStackElement)
			.InOpListWidget(InOpListWidget);	
	}
	
	return SNew(SRetargetOpSingleItem, InOwnerTable)
			.InStackElement(InStackElement)
			.InOpListWidget(InOpListWidget);
}

FName FRetargetOpStackElement::GetName() const
{
	FIKRetargetEditorController* EditorController = OpListWidget.Pin()->EditorController.Pin().Get();
	return EditorController->AssetController->GetOpName(IndexInStack);
}

const UScriptStruct* FRetargetOpStackElement::GetType() const
{
	const FIKRetargetEditorController* EditorController = OpListWidget.Pin()->EditorController.Pin().Get();
	const FInstancedStruct* OpStruct = EditorController->AssetController->GetRetargetOpStructAtIndex(IndexInStack);
	return OpStruct ? OpStruct->GetScriptStruct() : nullptr;
}

void SRetargetOpItem::Construct(const FArguments& InArgs)
{
	ListView = InArgs._InOpListWidget;
	Element = InArgs._InStackElement;

	ChildSlot
	[
		SNew(SBorder)
		 .BorderImage(FAppStyle::GetBrush("NoBorder")) // Invisible but hit-testable
		 .Visibility(EVisibility::Visible)
		 .OnMouseButtonUp_Lambda([this](const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) -> FReply
		 {
			 if (MouseEvent.GetEffectingButton() != EKeys::RightMouseButton)
			 {
				return FReply::Unhandled();
			 }

			// build a menu for this op
			FMenuBuilder MenuBuilder(true, nullptr);
			GenerateRightClickMenu(MenuBuilder);

			// summon the menu at the cursor's location
			FSlateApplication::Get().PushMenu(
				SharedThis(this),
				MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath(),
				MenuBuilder.MakeWidget(),
				MouseEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

			 return FReply::Handled();
		 })
		[
			 SNew(SHorizontalBox)
			.ToolTipText(TAttribute<FText>::CreateLambda([this]()
			{
				const UScriptStruct* OpType = Element.Pin()->GetType();
				if (!OpType)
				{
					return FText::FromString(TEXT("Unknown Op type. Resave asset to clean."));
				}
					
				return FText::Format(
				FText::FromString(TEXT("Op Type: {0} \nIndex in Stack: {1}\n")),
				FText::FromString(OpType->GetStructCPPName()),
				FText::AsNumber(Element.Pin()->GetIndexInStack()) );
			}))
			
			// drag Icon
			+ SHorizontalBox::Slot()
			.MaxWidth(18)
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(OpHorizontalPadding, OpVerticalPadding))
			[
				SNew(SImage)
				.Image(FIKRigEditorStyle::Get().GetBrush("IKRig.DragSolver"))
			]

			// enable Checkbox
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(OpHorizontalPadding, OpVerticalPadding))
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() -> ECheckBoxState
				{
					FIKRetargetEditorController* EditorController = GetEditorController();
					bool bEnabled = EditorController->AssetController->GetRetargetOpEnabled(Element.Pin()->GetIndexInStack());
					return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
				{
					FIKRetargetEditorController* EditorController = GetEditorController();
					const bool bIsChecked = InCheckBoxState == ECheckBoxState::Checked;
					EditorController->AssetController->SetRetargetOpEnabled( Element.Pin()->GetIndexInStack(), bIsChecked);
					EditorController->ReinitializeRetargeterNoUIRefresh();
				})
			]

			// toggle debug draw button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(OpHorizontalPadding, OpVerticalPadding))
			[
				SNew(SButton)
				.NormalPaddingOverride(FMargin(2, 2))
				.PressedPaddingOverride(FMargin(2, 2))
				.ToolTipText(LOCTEXT("ToggleOpDebugDraw", "Toggle this Ops debug drawing in the viewport."))
				.Visibility_Lambda([this]()
				{
					FIKRetargetEditorController* EditorController = GetEditorController();
					FIKRetargetOpBase* Op = EditorController->AssetController->GetRetargetOpByIndex(Element.Pin()->GetIndexInStack());
					if (!ensure(Op))
					{
						return EVisibility::Collapsed;
					}
					return Op->HasDebugDrawing() ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.OnClicked_Lambda([this]() -> FReply
				{
					FIKRetargetEditorController* EditorController = GetEditorController();
					FIKRetargetOpBase* Op = EditorController->AssetController->GetRetargetOpByIndex(Element.Pin()->GetIndexInStack());
					if (!ensure(Op))
					{
						return FReply::Handled();
					}

					Op->GetSettings()->bDebugDraw = !Op->GetSettings()->bDebugDraw;
					return FReply::Handled();
				})
				.Content()
				[
					SNew(SImage)
					.Image(FIKRetargetEditorStyle::Get().GetBrush("IKRetarget.Debug"))
					.ColorAndOpacity_Lambda([this]()
					{
						FIKRetargetEditorController* EditorController = GetEditorController();
						FIKRetargetOpBase* Op = EditorController->AssetController->GetRetargetOpByIndex(Element.Pin()->GetIndexInStack());
						if (!Op)
						{
							return FLinearColor::White;
						}
						return Op->GetSettings()->bDebugDraw ?FLinearColor(0,1,0,1.0) : FLinearColor::White;
					})
				]
			]

			// display Name
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FMargin(OpHorizontalPadding, OpVerticalPadding))
			[
				SAssignNew(EditNameWidget, SInlineEditableTextBlock)
				.Text(this, &SRetargetOpItem::GetName)
				.OnVerifyTextChanged_Lambda([](const FText& InText, FText& OutErrorMessage)
				{
					static FString IllegalNameCharacters = "^<>:\"/\\|?*";
					return FName::IsValidXName(InText.ToString(), FString(IllegalNameCharacters), &OutErrorMessage);
				})
				.OnTextCommitted(this, &SRetargetOpItem::OnNameCommitted)
				.MultiLine(false)
			]

			// spacer
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.FillWidth(1.0f)
			[
				SNew(SSpacer)
				.Size(FVector2D(0.0f, 1.0f))
			]

			// profiling
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(OpHorizontalPadding, OpVerticalPadding))
			[
				SNew(STextBlock)
				.Text_Lambda([this]() -> FText
				{
					FIKRetargetEditorController* EditorController = GetEditorController();
					FIKRetargetOpBase* Op = EditorController->AssetController->GetRetargetOpByIndex(Element.Pin()->GetIndexInStack());
					const double Milliseconds = (Op && Op->EditorInstance ? Op->EditorInstance->Timer.GetRunningAverage() : 0.0f) * 1000.0;
					FString Formatted = FString::Printf(TEXT("%.3f ms"), Milliseconds);
					return FText::FromString(Formatted);
				})
				.IsEnabled(false)
				.Visibility_Lambda([this]()
				{
					const bool Enabled = ListView.Pin()->EditorController.Pin()->AssetController->GetAsset()->bProfileOps;
					return Enabled ? EVisibility::Visible : EVisibility::Collapsed;
				})
			]

			// add sub-op button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(OpHorizontalPadding, OpVerticalPadding))
			[
				SNew(SPositiveActionButton)
				.Visibility(Element.Pin()->GetCanHaveChildren() ? EVisibility::Visible : EVisibility::Collapsed)
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				.ToolTipText(LOCTEXT("AddChildToolTip", "Add a child op to run before this op."))
				.OnGetMenuContent_Lambda([this]()
				{
					return ListView.Pin()->CreateAddNewOpMenu();	
				})
			]

			// spin Box
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(OpHorizontalPadding, OpVerticalPadding))
			[
				SNew(SSpinBox<double>)
				.Value_Lambda([this]() -> double
				{
					return 1.0f;							// TODO add support for alpha value on op
				})
				.IsEnabled(false)					// TODO enable this once alpha is supported
				.Visibility(EVisibility::Collapsed)	// TODO enable this once alpha is supported
				.MinValue(TOptional<double>())
				.MaxValue(TOptional<double>())
				.Style(&FAppStyle::Get().GetWidgetStyle<FSpinBoxStyle>("Sequencer.HyperlinkSpinBox"))
			]

			// delete Button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(FMargin(OpHorizontalPadding, OpVerticalPadding))
			[
				SNew(SButton)
				.NormalPaddingOverride(FMargin(2, 2))
				.PressedPaddingOverride(FMargin(2, 2))
				.ToolTipText(LOCTEXT("DeleteOp", "Delete retarget op and remove from stack."))
				.OnClicked_Lambda([this]() -> FReply
				{
					ListView.Pin()->DeleteRetargetOp(Element.Pin());
					return FReply::Handled();
				})
				.Content()
				[
					SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
						.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]   
		]
	];

	// bind the rename callback to enter text editing mode when item is slow double-clicked
	Element.Pin()->OnRenameRequested.BindSP(EditNameWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
}

void SRetargetOpItem::OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	FName NewName = FName(InText.ToString());
	FIKRetargetEditorController* EditorController = GetEditorController();
	EditorController->AssetController->SetOpName(NewName, Element.Pin()->GetIndexInStack());
}

FText SRetargetOpItem::GetName() const
{
	FName OpName = ListView.Pin()->EditorController.Pin()->AssetController->GetOpName(Element.Pin()->GetIndexInStack());
	return FText::FromName(OpName);
}

void SRetargetOpItem::GenerateRightClickMenu(FMenuBuilder& InMenuBuilder)
{
	FIKRetargetEditorController* EditorController = GetEditorController();
	if (!EditorController)
	{
		return;
	}
	TObjectPtr<UIKRetargeterController> AssetController = EditorController->AssetController;
	FIKRetargetOpBase* Op = AssetController->GetRetargetOpByIndex(Element.Pin()->GetIndexInStack());
	if (!ensure(Op))
	{
		return;
	}

	const FName SelectedOverrideSet = EditorController->GetSelectedOverrideSet();
	const FName SelectedOpName = Op->GetName();

	if (SelectedOverrideSet == NAME_None)
	{
		InMenuBuilder.BeginSection("WarningSection", LOCTEXT("WarningHeading", "Notice"));
		{
			InMenuBuilder.AddWidget(
				SNew(STextBlock)
				.Text(LOCTEXT("NoSelectionWarning", "No profile selected to add overrides to. \nOpen the Retarget Profiles tab to edit a profile."))
				.ColorAndOpacity(FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"))
				.Margin(FMargin(16.0f, 4.0f))
				.Font(FAppStyle::GetFontStyle("BoldFont")),
				FText::GetEmpty(),
				true
			);
		}
		InMenuBuilder.EndSection();
		return;
	}
	
	InMenuBuilder.BeginSection("ProfileActions", FText::Format(LOCTEXT("ProfileHeading", "Edit Profile: {0}"), FText::FromName(SelectedOverrideSet)));

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("AddChangedLabel", "Add Changed Properties"),
		LOCTEXT("AddChangedTooltip", "Record all modifications to the op settings into the currently selected override set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
		FUIAction(FExecuteAction::CreateLambda([this, SelectedOverrideSet, SelectedOpName]()
			{
				GetEditorController()->AssetController->OverrideAllEditedProperties(SelectedOverrideSet, SelectedOpName);
			})));

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("AddAllLabel", "Add All Properties"),
		LOCTEXT("AddAllTooltip", "Record all the op settings into the currently selected override set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
		FUIAction(FExecuteAction::CreateLambda([this, SelectedOverrideSet, SelectedOpName]()
			{
				GetEditorController()->AssetController->OverrideAllProperties(SelectedOverrideSet, SelectedOpName);
			})));

	InMenuBuilder.AddMenuEntry(
		LOCTEXT("ClearAllLabel", "Clear All Overrides"),
		LOCTEXT("ClearAllTooltip", "Removes all sparse overrides from this overrise set."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Delete"),
		FUIAction(FExecuteAction::CreateLambda([this, SelectedOverrideSet, SelectedOpName]()
			{
				GetEditorController()->AssetController->RemoveOpOverrides(SelectedOverrideSet, SelectedOpName);
			})));

	TSharedPtr<FRetargetPropertyOverrideNode> RootNode;
	Op->GetOverrideableProperties(RootNode);
	if (RootNode.IsValid())
	{
		InMenuBuilder.EndSection();
		InMenuBuilder.BeginSection("OpProperties", LOCTEXT("ProfileOverrideHeading", "Add Overrides to Profile"));
		MakeOverrideSubMenu(InMenuBuilder, RootNode);
		InMenuBuilder.EndSection();
	}
}

FIKRetargetEditorController* SRetargetOpItem::GetEditorController() const
{
	return ListView.IsValid() ? ListView.Pin()->EditorController.Pin().Get() : nullptr;
}

void SRetargetOpItem::MakeOverrideSubMenu(FMenuBuilder& SubMenuBuilder, TSharedPtr<FRetargetPropertyOverrideNode> Node)
{
	const FName ProfileName = GetEditorController()->GetSelectedOverrideSet();
	FName OpName = GetEditorController()->AssetController->GetOpName(Element.Pin()->GetIndexInStack());
	
	// add leaf properties first
	for (const FRetargetPropertyOverrideNode::FRetargetPropertyOverrideEntry& Entry : Node->LeafProperties)
	{
		SubMenuBuilder.AddMenuEntry(
			Entry.DisplayName,
			FText::Format(LOCTEXT("OverrideTipLabel", "Add or Remove {0} override in current override set."), FText::FromString(Entry.PropertyPath)),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, ProfileName, OpName, Entry]()
				{
					TObjectPtr<UIKRetargeterController> AssetController = GetEditorController()->AssetController;
					if (AssetController->HasPropertyOverride(ProfileName, OpName, Entry.PropertyPath))
					{
						// remove it
						AssetController->RemovePropertyOverrideFromOp(ProfileName, OpName, Entry.PropertyPath);
					}
					else
					{
						// add it
						AssetController->AddPropertyOverrideToOp(ProfileName, OpName, Entry.PropertyPath);	
					}
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, ProfileName, OpName, Entry]()
				{
					TObjectPtr<UIKRetargeterController> AssetController = GetEditorController()->AssetController;
					return AssetController->HasPropertyOverride(ProfileName, OpName, Entry.PropertyPath);
				})
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	// add sub-nodes (nested submenus)
	for (TTuple<FName, TSharedPtr<FRetargetPropertyOverrideNode>>& Pair : Node->SubNodes)
	{
		SubMenuBuilder.AddSubMenu(
			FText::FromName(Pair.Key),
			FText::GetEmpty(),
			FNewMenuDelegate::CreateSP(this, &SRetargetOpItem::MakeOverrideSubMenu, Pair.Value)
		);
	}
}

bool SRetargetOpItem::IsOpEnabled() const
{
	if (const FIKRetargetOpBase* Op = GetRetargetOp())
	{
		return Op->IsEnabled();
	}
	
	return false;
}

FIKRetargetOpBase* SRetargetOpItem::GetRetargetOp() const
{
	if (!(Element.IsValid() && ListView.IsValid()))
	{
		return nullptr;
	}
	FIKRetargetEditorController* EditorController = GetEditorController();
	if (!EditorController)
	{
		return nullptr;
	}
	const int32 OpIndex = Element.Pin()->GetIndexInStack();
	return EditorController->AssetController->GetRetargetOpByIndex(OpIndex);
}

void SRetargetOpList::Construct(const FArguments& InArgs)
{
	EditorController = InArgs._InEditorController;
	ParentElement = InArgs._InParentElement;

	CacheOpTypeMetaData();
	
	SListView::FArguments SuperArgs;
	SuperArgs.ListItemsSource(&Elements);
	SuperArgs.SelectionMode(ESelectionMode::Single);
	SuperArgs.OnGenerateRow(this, &SRetargetOpList::MakeListRowWidget);
	SuperArgs.IsEnabled(this, &SRetargetOpList::IsEnabled);
	SuperArgs.OnSelectionChanged(this, &SRetargetOpList::OnSelectionChanged);
	SuperArgs.OnMouseButtonClick(this, &SRetargetOpList::OnItemClicked);
	
	SListView::Construct(SuperArgs);
}

void SRetargetOpList::CacheOpTypeMetaData()
{
	AllOpsMetaData.Reset();

	// instantiate all types of ops to cache their meta data
	TArray<FInstancedStruct> TempInstancedOps;
	const UStruct* BaseStruct = FIKRetargetOpBase::StaticStruct();
	for (TObjectIterator<UStruct> StructIt; StructIt; ++StructIt)
	{
		UStruct* Struct = *StructIt;
		if (!Struct->IsChildOf(BaseStruct) || Struct == BaseStruct)
		{
			continue;
		}

		UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Struct);
		if (!ScriptStruct)
		{
			continue;
		}
		
		TempInstancedOps.Emplace(ScriptStruct);
	}

	// what type of op is the parent of this list? (determines which type of children can be added)
	// this is nullptr if we are the top level (top-level ops don't need a parent)
	const UScriptStruct* ListType = ParentElement.IsValid() ? ParentElement.Pin()->GetType() : nullptr;

	// store meta data for each type of op
	for (const FInstancedStruct& TempOpStruct : TempInstancedOps)
	{
		const FIKRetargetOpBase* TempOp = TempOpStruct.GetPtr<FIKRetargetOpBase>();
		const UScriptStruct* TempOpParentType = TempOp->GetParentOpType();

		// filter ops from the sub-lists if they require a different parent type
		if (ListType != nullptr && TempOpParentType != ListType)
		{
			continue;
		}

		// filter ops from main list if they require a special parent type
		if (ListType == nullptr && TempOpParentType != nullptr)
		{
			continue;
		}

		// filter deprecated ops
		static const FName DeprecatedStruct = TEXT("Deprecated");
		if (TempOp->GetType()->HasMetaData(DeprecatedStruct))
		{
			continue;
		}

		// disallow adding abstract types to the stack
		if (TempOp->IsAbstract())
		{
			continue;
		}
		
		FIKRetargetOpMetaData OpMetaData;
		OpMetaData.NiceName = TempOp->GetDefaultName();
		OpMetaData.OpType = TempOpStruct.GetScriptStruct();
		OpMetaData.ParentType = TempOp->GetParentOpType();
		OpMetaData.bIsSingleton = TempOp->IsSingleton();
		AllOpsMetaData.Add(OpMetaData);
	}
}

TSharedRef<ITableRow> SRetargetOpList::MakeListRowWidget(
	TSharedPtr<FRetargetOpStackElement> InElement,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return InElement->MakeListRowWidget(OwnerTable,InElement.ToSharedRef(),SharedThis(this));
}

FReply SRetargetOpList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	TArray<TSharedPtr<FRetargetOpStackElement>> SelectedOps = GetSelectedItems();
	if (!SelectedOps.IsEmpty() && InKeyEvent.GetKey() == EKeys::Delete)
	{
		DeleteRetargetOp(SelectedOps.Last());
		return FReply::Handled();
	}
	
	return SListView::OnKeyDown(MyGeometry, InKeyEvent);
}

FReply SRetargetOpList::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TArray<TSharedPtr<FRetargetOpStackElement>> CurrentSelectedItems = GetSelectedItems();
	if (CurrentSelectedItems.Num() != 1)
	{
		return FReply::Unhandled();
	}
	
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TSharedPtr<FRetargetOpStackElement> DraggedElement = CurrentSelectedItems[0];
		const TSharedRef<FRetargetOpStackDragDropOp> DragDropOp = FRetargetOpStackDragDropOp::New(DraggedElement);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SRetargetOpList::OnCanAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
	TSharedPtr<FRetargetOpStackElement> TargetElement)
{
	TOptional<EItemDropZone> ReturnedDropZone;
	const TSharedPtr<FRetargetOpStackDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FRetargetOpStackDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return ReturnedDropZone;
	}

	const TSharedPtr<FRetargetOpStackElement>& DraggedElement = DragDropOp.Get()->Element.Pin();
	if (!DraggedElement)
	{
		return ReturnedDropZone;
	}

	// if this is child element, it can only be re-ordered with siblings
	if (const FRetargetOpStackElement* ParentOfDraggedItem = DraggedElement->GetParent())
	{
		// only allow dropping a child element on a sibling
		if (!ParentOfDraggedItem->GetChildren().Contains(TargetElement.Get()))
		{
			return ReturnedDropZone;
		}
	}
	else
	{
		// only allow dropping a top-level element on another top-level element
		if (TargetElement->GetParent() != nullptr)
		{
			return ReturnedDropZone;
		}
	}

	// validate index to move to
	const int32 IndexToMoveTo = FRetargetOpStackDragDropOp::GetIndexToMoveTo(DraggedElement, TargetElement, DropZone);

	// GetIndexToMoveTo returns index none for invalid moves
	if (IndexToMoveTo == INDEX_NONE)
	{
		return ReturnedDropZone;
	}
	
	// all good!
	ReturnedDropZone = DropZone;
	return ReturnedDropZone;
}

FReply SRetargetOpList::OnAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
	TSharedPtr<FRetargetOpStackElement> TargetElement)
{
	const TSharedPtr<FRetargetOpStackDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FRetargetOpStackDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}
	
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return FReply::Handled();
	}

	const TSharedPtr<FRetargetOpStackElement>& DraggedElement = DragDropOp.Get()->Element.Pin();
	const int32 IndexToMoveTo = FRetargetOpStackDragDropOp::GetIndexToMoveTo(DraggedElement, TargetElement, DropZone);
	if (IndexToMoveTo == INDEX_NONE)
	{
		// don't do anything if the drop location is invalid
		return FReply::Unhandled();
	}
	
	const UIKRetargeterController* AssetController = Controller->AssetController;
	const bool bWasReparented = AssetController->MoveRetargetOpInStack(DraggedElement->GetIndexInStack(),IndexToMoveTo);
	
	return FReply::Handled();
}

FReply SRetargetOpList::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	LastClickCycles = FPlatformTime::Cycles();
	return SListView::OnFocusReceived(MyGeometry, InFocusEvent);
}

void SRetargetOpList::OnSelectionChanged(TSharedPtr<FRetargetOpStackElement> InItem, ESelectInfo::Type SelectInfo) const
{
	// adds support for keyboard navigation of op stack
	if (SelectInfo == ESelectInfo::OnNavigation || SelectInfo == ESelectInfo::Direct)
	{
		if (EditorController.IsValid() && InItem.IsValid())
		{
			EditorController.Pin()->SetOpSelected(InItem->GetIndexInStack());
		}
		
		return;
	}

	if (!InItem.IsValid())
	{
		EditorController.Pin()->ClearSelection();
	}
}

void SRetargetOpList::OnItemClicked(TSharedPtr<FRetargetOpStackElement> InItem)
{
	// to rename an item, you have to select it first, then click on it again within a time limit (slow double click)
	const bool ClickedOnSameItem = LastSelectedElement.Pin().Get() == InItem.Get();
	const uint32 CurrentCycles = FPlatformTime::Cycles();
	const double SecondsPassed = static_cast<double>(CurrentCycles - LastClickCycles) * FPlatformTime::GetSecondsPerCycle();
	if (ClickedOnSameItem && SecondsPassed > 0.25f && SecondsPassed < 0.75f)
	{
		RegisterActiveTimer(0.f,
			FWidgetActiveTimerDelegate::CreateLambda([this](double, float)
			{
				RequestRenameSelectedOp();
				return EActiveTimerReturnType::Stop;
			}));
	}

	LastClickCycles = CurrentCycles;
	LastSelectedElement = InItem;

	if (EditorController.IsValid() && InItem.IsValid())
	{
		EditorController.Pin()->SetOpSelected(InItem->GetIndexInStack());
	}
	else
	{
		EditorController.Pin()->ClearSelection();
	}
}

bool SRetargetOpList::IsEnabled() const
{
	if (!EditorController.IsValid())
	{
		return false; 
	}

	if (const FIKRetargetProcessor* Processor = EditorController.Pin()->GetRetargetProcessor())
	{
		return Processor->IsInitialized();
	}
	
	return false;
}

void SRetargetOpList::RequestRenameSelectedOp() const
{
	if (!LastSelectedElement.IsValid())
	{
		return;
	}

	LastSelectedElement.Pin()->OnRenameRequested.ExecuteIfBound();
}

TSharedRef<SWidget> SRetargetOpList::CreateAddNewOpMenu()
{
	constexpr bool bCloseMenuAfterSelection = true;
	FMenuBuilder MenuBuilder(bCloseMenuAfterSelection, nullptr);

	MenuBuilder.BeginSection("AddNewRetargetOp", LOCTEXT("AddOperations", "Add New Op"));

	// add menu option to create each retarget op type
	const TObjectPtr<UIKRetargeterController> AssetController = EditorController.Pin()->AssetController;
	for (FIKRetargetOpMetaData& OpMetaData : AllOpsMetaData)
	{
		FUIAction Action = FUIAction(
			FExecuteAction::CreateSP(this, &SRetargetOpList::AddNewRetargetOp, OpMetaData.OpType),
			FCanExecuteAction::CreateLambda([AssetController, OpMetaData]()
			{
				if (!OpMetaData.bIsSingleton)
				{
					return true;
				}

				// check if another instance of this type already exists in the op stack
				const int32 NumOps = AssetController->GetNumRetargetOps();
				for (int32 OpIndex = 0; OpIndex < NumOps; ++OpIndex)
				{
					FIKRetargetOpBase* Op = AssetController->GetRetargetOpByIndex(OpIndex);
					if (Op->GetType() == OpMetaData.OpType)
					{
						return false; // can only have one instance of this op type
					}
				}
				
				return true;
			}));
			
		MenuBuilder.AddMenuEntry(FText::FromName(OpMetaData.NiceName), FText::GetEmpty(), FSlateIcon(), Action);
	}
	
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SRetargetOpList::AddNewRetargetOp(const UScriptStruct* ScriptStruct)
{
	if (!EditorController.IsValid())
	{
		return; 
	}

	const UIKRetargeterController* AssetController = EditorController.Pin()->AssetController;
	if (!AssetController)
	{
		return;
	}

	// add a new op to the stack
	const FName ParentOpName = ParentElement.IsValid() ? ParentElement.Pin()->GetName() : NAME_None;
	int32 NewOpIndex = AssetController->AddRetargetOp(ScriptStruct, ParentOpName);
}

void SRetargetOpList::DeleteRetargetOp(TSharedPtr<FRetargetOpStackElement> OpToDelete)
{
	if (!OpToDelete.IsValid())
	{
		return;
	}
	
	EditorController.Pin()->AssetController->RemoveRetargetOp(OpToDelete->GetIndexInStack());
}

void SRetargetOpList::RefreshAndRestore()
{
	FIKRetargetEditorController* Controller = EditorController.Pin().Get();
	if (!ensure(Controller))
	{
		return;
	}

	RequestListRefresh();
	
	auto GetElementOfLastSelectedOp = [this, Controller]() -> TSharedPtr<FRetargetOpStackElement>
	{
		TSharedPtr<FRetargetOpStackElement> ElementToSelect;

		if (Controller->GetSelectionState().LastSelectedType != ERetargetSelectionType::OP)
		{
			return TSharedPtr<FRetargetOpStackElement>();
		}
	
		const FName LastSelectedOpName = Controller->GetSelectedOpName();
		if (LastSelectedOpName == NAME_None)
		{
			return TSharedPtr<FRetargetOpStackElement>();
		}

		for (TSharedPtr<FRetargetOpStackElement> Element : Elements)
		{
			if (Element->GetName() == LastSelectedOpName)
			{
				return Element;
			}
		}

		return TSharedPtr<FRetargetOpStackElement>();
	};
	
	// restore selection to the last selected element
	TSharedPtr<FRetargetOpStackElement> ElementToRestore = GetElementOfLastSelectedOp();
	if (ElementToRestore.IsValid())
	{
		// restore selection
		ElementToRestore.Get()->GetOpList().Pin()->SetSelection(ElementToRestore);
		Controller->SetOpSelected(ElementToRestore->GetIndexInStack());
	}
	else
	{
		// if an op was selected but it's no longer valid, then clear the selection
		if (Controller->GetSelectionState().LastSelectedType == ERetargetSelectionType::OP)
		{
			FIKRetargetOpBase* SelectedOp = Controller->AssetController->GetRetargetOpByName(Controller->GetSelectedOpName());
			if (!SelectedOp)
			{
				Controller->ClearSelection();	
			}
		}
	}
}

void SParentRetargetOpItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
{
	ParentStackElement = InArgs._InStackElement;
	OpListWidget = InArgs._InOpListWidget;
	
	// create the list view ahead of time so it can be passed to the op item
	ChildrenListView = MakeShared<SRetargetOpList>();
	ChildrenListView->Construct(
	SRetargetOpList::FArguments()
		.InEditorController(OpListWidget.Pin()->EditorController)
		.InParentElement(ParentStackElement));
	
	STableRow::Construct(
		STableRow::FArguments()
		.OnDragDetected(OpListWidget.Pin().Get(), &SRetargetOpList::OnDragDetected)
		.OnCanAcceptDrop(OpListWidget.Pin().Get(), &SRetargetOpList::OnCanAcceptDrop)
		.OnAcceptDrop(OpListWidget.Pin().Get(), &SRetargetOpList::OnAcceptDrop)
		.ShowSelection(false)
		.Padding(FMargin(6.f, 3.f))
		.Content()
		[
			SNew(SBorder)
			.BorderImage_Lambda([this]()
			{
				const FName SelectedOpName = OpListWidget.Pin()->EditorController.Pin()->GetSelectedOpName();
				if (ParentStackElement.Pin()->GetName() == SelectedOpName)
				{
					return FIKRetargetEditorStyle::Get().GetBrush("IKRetarget.OpBorderSelected");
				}
				return FIKRetargetEditorStyle::Get().GetBrush("IKRetarget.OpBorder");
			})
			.Padding(FMargin(SRetargetOpItem::OpHorizontalPadding, SRetargetOpItem::OpVerticalPadding))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				.Padding(0.0f)
				[
					SNew(SRetargetOpItem)
					.InOpListWidget(ChildrenListView)
					.InStackElement(ParentStackElement)
				]

				+SVerticalBox::Slot()
				.Padding(5.0f)
				[
					SNew(SBorder)
					.Visibility_Lambda([this]()
					{
						return ChildrenListView->Elements.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
					})
					.BorderImage(FIKRetargetEditorStyle::Get().GetBrush("IKRetarget.OpGroupBorder"))
					.Padding(0.f, SRetargetOpItem::OpVerticalPadding)
					[
						ChildrenListView.ToSharedRef()
					]
				]
			]
			
		], OwnerTable);

	RefreshListView();
}

void SParentRetargetOpItem::RefreshListView() const
{
	const FIKRetargetEditorController* Controller = OpListWidget.Pin()->EditorController.Pin().Get();
	if (!Controller)
	{
		return; 
	}

	// generate all list elements
	ChildrenListView->Elements.Reset();
	UIKRetargeterController* AssetController = Controller->AssetController;
	TArray<int32> ChildIndices = AssetController->GetChildOpIndices(ParentStackElement.Pin()->GetIndexInStack());
	for (int32 ChildOpIndex : ChildIndices)
	{
		// create a new op element for the child
		constexpr bool bCanOpHaveChildren = false; // we only allow 1 level of nesting
		TSharedPtr<FRetargetOpStackElement> ChildElement = FRetargetOpStackElement::Make(ChildOpIndex, bCanOpHaveChildren, ChildrenListView);
		ChildrenListView->Elements.Add(ChildElement);
		
		// store pointer to parent element on child
		ChildElement->SetParent(ParentStackElement.Pin().Get());
		
		// store pointer to child element on parent
		ParentStackElement.Pin()->AddChild(ChildElement.Get());
	}

	ChildrenListView->RefreshAndRestore();
}

void SRetargetOpSingleItem::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable)
{
	StackElement = InArgs._InStackElement;
	OpListWidget = InArgs._InOpListWidget;
	
	STableRow::Construct(
        STableRow::FArguments()
        .OnDragDetected(OpListWidget.Pin().Get(), &SRetargetOpList::OnDragDetected)
        .OnCanAcceptDrop(OpListWidget.Pin().Get(), &SRetargetOpList::OnCanAcceptDrop)
        .OnAcceptDrop(OpListWidget.Pin().Get(), &SRetargetOpList::OnAcceptDrop)
        .ShowSelection(false)
        .Padding(FMargin(6.f, 3.f))
        .Content()
        [
        	SNew(SBorder)
        	.BorderImage_Lambda([this]()
			{
				const FName SelectedOpName = OpListWidget.Pin()->EditorController.Pin()->GetSelectedOpName();
				if (StackElement.Pin()->GetName() == SelectedOpName)
				{
					return FIKRetargetEditorStyle::Get().GetBrush("IKRetarget.OpBorderSelected");
				}
				return FIKRetargetEditorStyle::Get().GetBrush("IKRetarget.OpBorder");
			})
			.Padding(FMargin(SRetargetOpItem::OpHorizontalPadding, SRetargetOpItem::OpVerticalPadding))
			[
				SNew(SRetargetOpItem)
				.InOpListWidget(OpListWidget)
				.InStackElement(StackElement)
			]
        ], OwnerTable);	
}

bool SRetargetOpSingleItem::GetWarningMessage(FText& Message) const
{
	const FIKRetargetProcessor* Processor = OpListWidget.Pin()->EditorController.Pin()->GetRetargetProcessor();
	if (!(Processor && Processor->IsInitialized()))
	{
		return false;
	}

	const TArray<FInstancedStruct>& RetargetOps = Processor->GetRetargetOps();
	const int32 OpIndex = StackElement.Pin()->GetIndexInStack();
	if (RetargetOps.IsValidIndex(OpIndex))
	{
		const FIKRetargetOpBase* Op = RetargetOps[OpIndex].GetPtr<FIKRetargetOpBase>();
		Message = Op->GetWarningMessage();
		return true;
	}
	
	return false;
}

bool SRetargetOpSingleItem::IsOpEnabled() const
{
	if (const FIKRetargetOpBase* Op = GetRetargetOp())
	{
		return Op->IsEnabled();
	}
	
	return false;
}

FIKRetargetOpBase* SRetargetOpSingleItem::GetRetargetOp() const
{
	if (!(StackElement.IsValid() && OpListWidget.IsValid()))
	{
		return nullptr;
	}
	FIKRetargetEditorController* EditorController = OpListWidget.Pin()->EditorController.Pin().Get();
	if (!EditorController)
	{
		return nullptr;
	}
	const int32 OpIndex = StackElement.Pin()->GetIndexInStack();
	return EditorController->AssetController->GetRetargetOpByIndex(OpIndex);
}

void SRetargetOpStack::Construct(
	const FArguments& InArgs,
	const TWeakPtr<FIKRetargetEditorController>& InEditorController)
{
	EditorController = InEditorController;
	EditorController.Pin()->SetOpStackView(SharedThis(this));

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		.Padding(0.0f)
		[
			SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(2.0f, 4.0f)
					[
						SNew(SPositiveActionButton)
						.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
						.Text(LOCTEXT("AddNewRetargetOpLabel", "Add New Op"))
						.ToolTipText(LOCTEXT("AddNewToolTip", "Add a new operation to run as part of the retargeter."))
						.OnGetMenuContent_Lambda([this]()
						{
							return ListView->CreateAddNewOpMenu();	
						})
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(2.0f, 4.0f)
					[
						SNew(SPositiveActionButton)
						.Icon(FIKRetargetEditorStyle::Get().GetBrush("IKRetarget.PostSettingsSmall"))
						.Text(LOCTEXT("AddDefaultOpsLabel", "Add Default Ops"))
						.ToolTipText(LOCTEXT("AddDefaultToolTip", "Add Pelvis Motion, IK/FK Chains and Root Motion Ops. Runs auto-setup on each op."))
						.OnClicked_Lambda([this]()
						{
							EditorController.Pin()->AssetController->AddDefaultOps();
							return FReply::Handled();
						})
					]

					// spacer
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.FillWidth(1.0f)
					[
						SNew(SSpacer).Size(FVector2D(0.0f, 1.0f))
					]

					// profiling
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					.Padding(FMargin(2.0f, 4.0f))
					[
						SNew(STextBlock)
						.Text_Lambda([this]() -> FText
						{
							UIKRetargetAnimInstance* AnimInstance = EditorController.Pin()->GetAnimInstance(ERetargetSourceOrTarget::Target);
							const FAnimNode_RetargetPoseFromMesh& AnimNode = AnimInstance->GetRetargetAnimNode();
							const double Milliseconds = AnimNode.GetAverageExecutionTime() * 1000.0;
							FString Formatted = FString::Printf(TEXT("Total: %.3f ms"), Milliseconds);
							return FText::FromString(Formatted);
						})
						.IsEnabled(false)
						.Visibility_Lambda([this]()
						{
							const bool Enabled = EditorController.Pin()->AssetController->GetAsset()->bProfileOps;
							return Enabled ? EVisibility::Visible : EVisibility::Collapsed;
						})
					]
				]
			]
		]

		+SVerticalBox::Slot()
		.Padding(0.0f)
		[
			SAssignNew( ListView, SRetargetOpList )
			.InEditorController(EditorController)
			.InParentElement(nullptr)
		]
	];

	RefreshStackView();
}

void SRetargetOpStack::RefreshStackView() const
{
	const TSharedPtr<FIKRetargetEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	// empty old elements
	ListView->Elements.Reset();

	// rebuild list of top level elements
	UIKRetargeterController* AssetController = Controller->AssetController;
	const int32 NumOps = AssetController->GetNumRetargetOps();
	for (int32 OpIndex=0; OpIndex<NumOps; ++OpIndex)
	{
		// skip ops that are children of another op
		const int32 ParentOpIndex = AssetController->GetParentOpIndex(OpIndex);
		if (ParentOpIndex != INDEX_NONE)
		{
			continue;
		}

		// is this a group with children
		const bool bCanOpHaveChildren = AssetController->GetCanOpHaveChildren(OpIndex);

		// add op to main stack
		TSharedPtr<FRetargetOpStackElement> StackElement = FRetargetOpStackElement::Make(OpIndex, bCanOpHaveChildren, ListView);
		ListView->Elements.Add(StackElement);
	}

	// refresh the list and restore the selection
	ListView->RefreshAndRestore();
}

TSharedRef<FRetargetOpStackDragDropOp> FRetargetOpStackDragDropOp::New(TWeakPtr<FRetargetOpStackElement> InElement)
{
	TSharedRef<FRetargetOpStackDragDropOp> Operation = MakeShared<FRetargetOpStackDragDropOp>();
	Operation->Element = InElement;
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FRetargetOpStackDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.Visibility(EVisibility::Visible)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		[
			SNew(STextBlock)
			.Text(FText::FromName(Element.Pin()->GetName()))
		];
}

int32 FRetargetOpStackDragDropOp::GetIndexToMoveTo(
	const TSharedPtr<FRetargetOpStackElement>& InDraggedElement,
	const TSharedPtr<FRetargetOpStackElement>& InTargetElement,
	const EItemDropZone InDropZone)
{
	// disallow dropping on self
	const int32 DraggedItemIndex = InDraggedElement->GetIndexInStack();
	const int32 TargetItemIndex = InTargetElement->GetIndexInStack();
	if (DraggedItemIndex == TargetItemIndex)
	{
		return INDEX_NONE;
	}

	// disallow dropping in a place that would not change the order (like below the one above, or above the one below)
	int32 IndexToMoveDragItemTo = INDEX_NONE;
	bool bDraggingDown = DraggedItemIndex <= TargetItemIndex;
	switch (InDropZone)
	{
	case EItemDropZone::AboveItem:
		IndexToMoveDragItemTo = bDraggingDown ? TargetItemIndex - 1 : TargetItemIndex;
		break;
	case EItemDropZone::BelowItem:
		IndexToMoveDragItemTo = bDraggingDown ? TargetItemIndex : TargetItemIndex + 1;
		break;
	case EItemDropZone::OntoItem:
		IndexToMoveDragItemTo = TargetItemIndex;
		break;
	default:
		checkNoEntry();
	}
	
	if (DraggedItemIndex == IndexToMoveDragItemTo)
	{
		return INDEX_NONE;
	}

	return IndexToMoveDragItemTo;
}

#undef LOCTEXT_NAMESPACE
