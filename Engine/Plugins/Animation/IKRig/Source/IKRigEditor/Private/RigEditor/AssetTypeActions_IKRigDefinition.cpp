// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/AssetTypeActions_IKRigDefinition.h"
#include "Rig/IKRigDefinition.h"
#include "RigEditor/IKRigToolkit.h"
#include "ThumbnailRendering/SceneThumbnailInfo.h"
#include "ContentBrowserMenuContexts.h"
#include "RigEditor/IKRigDefinitionFactory.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "Engine/SkeletalMesh.h"
#include "ToolMenus.h"
#include "AssetRegistry/AssetData.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"


UClass* FAssetTypeActions_IKRigDefinition::GetSupportedClass() const
{
	return UIKRigDefinition::StaticClass();
}

UThumbnailInfo* FAssetTypeActions_IKRigDefinition::GetThumbnailInfo(UObject* Asset) const
{
	UIKRigDefinition* IKRig = CastChecked<UIKRigDefinition>(Asset);
	return NewObject<USceneThumbnailInfo>(IKRig, NAME_None, RF_Transactional);
}

void FAssetTypeActions_IKRigDefinition::ExtendSkeletalMeshMenuToMakeIKRig()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.SkeletalMesh.CreateSkeletalMeshSubmenu");
	if (Menu == nullptr)
	{
		return;
	}

	FToolMenuSection& Section = Menu->AddSection("IKRig", LOCTEXT("IKRigSectionName", "IK Rig"));
	Section.AddDynamicEntry("CreateIKRig", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
		if (Context)
		{
			if (Context->SelectedAssets.Num() > 0)
			{
				InSection.AddMenuEntry(
					"CreateIKRig",
					LOCTEXT("CreateIKRig", "IK Rig"),
					LOCTEXT("CreateIKRig_ToolTip", "Creates an IK rig for this skeletal mesh."),
					FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(), "IKRig", "ClassIcon.IKRigDefinition"),
					FExecuteAction::CreateLambda([InSelectedAssets = Context->SelectedAssets]()
					{
						TArray<UObject*> SelectedObjects;
						SelectedObjects.Reserve(InSelectedAssets.Num());

						for (const FAssetData& Asset : InSelectedAssets)
						{
							if (UObject* LoadedAsset = Asset.GetAsset())
							{
								SelectedObjects.Add(LoadedAsset);
							}
						}

						for (UObject* SelectedObject : SelectedObjects)
						{
							CreateNewIKRigFromSkeletalMesh(SelectedObject);
						}
					})
				);
			}
		}
	}));
}

void FAssetTypeActions_IKRigDefinition::CreateNewIKRigFromSkeletalMesh(UObject* InSelectedObject)
{
	// validate skeletal mesh
	USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InSelectedObject);
	if(!SkeletalMesh)
	{
		UE_LOGF(LogTemp, Error, "CreateNewIKRigFromSkeletalMesh: Provided object has to be a SkeletalMesh.");
		return;
	}

	// remove the standard "SK_" prefix which is standard for these asset types (convenience)
	FString AssetName = InSelectedObject->GetName();
	if (AssetName.Len() > 3)
	{
		AssetName.RemoveFromStart("SK_");	
	}

	// create unique package and asset names
	FString UniquePackageName;
	FString UniqueAssetName = FString::Printf(TEXT("IK_%s"), *AssetName);
	
	FString PackagePath = InSelectedObject->GetPathName(); 
	int32 LastSlashPos = INDEX_NONE;
	if (PackagePath.FindLastChar('/', LastSlashPos))
	{
		PackagePath = PackagePath.Left(LastSlashPos);
	}
	
	PackagePath = PackagePath / UniqueAssetName;
	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(PackagePath, TEXT(""), UniquePackageName, UniqueAssetName);
	if (UniquePackageName.EndsWith(UniqueAssetName))
	{
		UniquePackageName = UniquePackageName.LeftChop(UniqueAssetName.Len() + 1);
	}
	
	// create the new IK Rig asset
	UIKRigDefinitionFactory* Factory = NewObject<UIKRigDefinitionFactory>();
	UObject* NewAsset = AssetToolsModule.Get().CreateAsset(*UniqueAssetName, *UniquePackageName, nullptr, Factory);
	const UIKRigDefinition* IKRig = Cast<UIKRigDefinition>(NewAsset);

	// assign skeletal mesh to the ik rig
	const UIKRigController* Controller = UIKRigController::GetController(IKRig);
	Controller->SetSkeletalMesh(SkeletalMesh);
}

void FAssetTypeActions_IKRigDefinition::OpenAssetEditor(
	const TArray<UObject*>& InObjects,
	TSharedPtr<IToolkitHost> EditWithinLevelEditor)
{
	const EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;
    
    for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
    {
    	if (UIKRigDefinition* Asset = Cast<UIKRigDefinition>(*ObjIt))
    	{
    		TSharedRef<FIKRigEditorToolkit> NewEditor(new FIKRigEditorToolkit());
    		NewEditor->InitAssetEditor(Mode, EditWithinLevelEditor, Asset);
    	}
    }
}

const TArray<FText>& FAssetTypeActions_IKRigDefinition::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		LOCTEXT("AnimRetargetingSubMenu", "Retargeting")
	};
	return SubMenus;
}

#undef LOCTEXT_NAMESPACE
