// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigAnimInstance.h"
#include "RigEditor/IKRigAnimInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigAnimInstance)

UIKRigAnimInstance::UIKRigAnimInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseMultiThreadedAnimationUpdate = false;
}

void UIKRigAnimInstance::SetIKRigAsset(UIKRigDefinition* InIKRigAsset)
{
	FIKRigAnimInstanceProxy& Proxy = GetProxyOnGameThread<FIKRigAnimInstanceProxy>();
	Proxy.SetIKRigAsset(InIKRigAsset);
}

void UIKRigAnimInstance::SetProcessorNeedsInitialized()
{
	IKRigNode.SetProcessorNeedsInitialized();
}

FIKRigProcessor* UIKRigAnimInstance::GetCurrentlyRunningProcessor()
{
	return IKRigNode.GetIKRigProcessor();
}

FAnimInstanceProxy* UIKRigAnimInstance::CreateAnimInstanceProxy()
{
	return new FIKRigAnimInstanceProxy(this, &IKRigNode);
}

