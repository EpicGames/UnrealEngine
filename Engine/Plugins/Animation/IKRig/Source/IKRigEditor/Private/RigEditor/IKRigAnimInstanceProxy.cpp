// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigAnimInstanceProxy.h"
#include "RigEditor/IKRigAnimInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigAnimInstanceProxy)


FIKRigAnimInstanceProxy::FIKRigAnimInstanceProxy(UAnimInstance* InAnimInstance, FAnimNode_IKRig* InIKRigNode)
	: FAnimPreviewInstanceProxy(InAnimInstance),
	IKRigNode(InIKRigNode)
{
}

void FIKRigAnimInstanceProxy::Initialize(UAnimInstance* InAnimInstance)
{
	Super::Initialize(InAnimInstance);
	IKRigNode->bDriveWithSourceAsset = true; // force this instance of the IK Rig evaluation to copy settings from the source IK Rig asset 
}

bool FIKRigAnimInstanceProxy::Evaluate(FPoseContext& Output)
{
	Super::Evaluate(Output);
	IKRigNode->Evaluate_AnyThread(Output);
	return true;
}

FAnimNode_Base* FIKRigAnimInstanceProxy::GetCustomRootNode()
{
	return IKRigNode;
}

void FIKRigAnimInstanceProxy::GetCustomNodes(TArray<FAnimNode_Base*>& OutNodes)
{
	OutNodes.Add(IKRigNode);
}

void FIKRigAnimInstanceProxy::UpdateAnimationNode(const FAnimationUpdateContext& InContext)
{
	Super::UpdateAnimationNode(InContext);
	IKRigNode->Update_AnyThread(InContext);
}

void FIKRigAnimInstanceProxy::SetIKRigAsset(UIKRigDefinition* InIKRigAsset)
{
	IKRigNode->RigDefinitionAsset = InIKRigAsset;
}


