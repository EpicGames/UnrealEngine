// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigEditor/IKRigAssetBrowserTabSummoner.h"

#include "IDocumentation.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/SIKRigAssetBrowser.h"

#define LOCTEXT_NAMESPACE "RetargetAssetBrowserTabSummoner"

const FName FIKRigAssetBrowserTabSummoner::TabID(TEXT("AssetBrowser"));

FIKRigAssetBrowserTabSummoner::FIKRigAssetBrowserTabSummoner(const TSharedRef<FIKRigEditorToolkit>& InRetargetEditor)
	: FWorkflowTabFactory(TabID, InRetargetEditor),
	IKRigEditor(InRetargetEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRigAssetBrowserTabLabel", "Asset Browser");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.TabIcon");

	ViewMenuDescription = LOCTEXT("IKRigAssetBrowser_ViewMenu_Desc", "Asset Browser");
	ViewMenuTooltip = LOCTEXT("IKRigAssetBrowser_ViewMenu_ToolTip", "Show the Asset Browser Tab");
}

TSharedPtr<SToolTip> FIKRigAssetBrowserTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRigAssetBrowserTooltip", "Select an animation asset to preview the IK results."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRigAssetBrowser_Window"));
}

TSharedRef<SWidget> FIKRigAssetBrowserTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SIKRigAssetBrowser, IKRigEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
