// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigAutoFBIK.h"

#include "IKRigEditor.h"
#include "Rig/Solvers/IKRigFullBodyIK.h"
#include "RigEditor/IKRigAutoCharacterizer.h"
#include "RigEditor/IKRigController.h"
#include "Templates/SubclassOf.h"

#define LOCTEXT_NAMESPACE "AutoFBIK"

void FAutoFBIKCreator::CreateFBIKSetup(const UIKRigController& IKRigController, FAutoFBIKResults& Results) const
{
	// ensure we have a mesh to operate on
	USkeletalMesh* Mesh = IKRigController.GetSkeletalMesh();
	if (!Mesh)
	{
		Results.Outcome = EAutoFBIKResult::MissingMesh;
		return;
	}

	// auto generate a retarget definition
	FAutoCharacterizeResults CharacterizeResults;
	IKRigController.AutoGenerateRetargetDefinition(CharacterizeResults);
	if (!CharacterizeResults.bUsedTemplate)
	{
		Results.Outcome = EAutoFBIKResult::UnknownSkeletonType;
		return;
	}
	
	// create all the goals in the template
	TArray<FName> GoalNames;
	const TArray<FBoneChain>& ExpectedChains = CharacterizeResults.AutoRetargetDefinition.RetargetDefinition.BoneChains;
	for (const FBoneChain& ExpectedChain : ExpectedChains)
	{
		if (ExpectedChain.IKGoalName == NAME_None)
		{
			continue;
		}
		
		const FBoneChain* Chain = IKRigController.GetRetargetChainByName(ExpectedChain.ChainName);
		FName GoalName = (Chain && Chain->IKGoalName != NAME_None) ? Chain->IKGoalName : ExpectedChain.IKGoalName;
		const UIKRigEffectorGoal* ChainGoal = IKRigController.GetGoal(GoalName);
		if (!ChainGoal)
		{
			// create new goal
			GoalName = IKRigController.AddNewGoal(GoalName, ExpectedChain.EndBone.BoneName);
			if (Chain)
			{
				IKRigController.SetRetargetChainGoal(Chain->ChainName, GoalName);
			}
			else
			{
				UE_LOGF(LogIKRigEditor, Warning, "Auto FBIK created goal for a limb, but it did not have a retarget chain. %ls", *ExpectedChain.ChainName.ToString());
			}
		}

		GoalNames.Add(GoalName);
	}

	// create IK solver and attach all the goals to it
	const int32 SolverIndex = IKRigController.AddSolver(FIKRigFullBodyIKSolver::StaticStruct());
	for (const FName& GoalName : GoalNames)
	{
		IKRigController.ConnectGoalToSolver(GoalName, SolverIndex);	
	}

	// set the root of the solver
	const bool bSetRoot = IKRigController.SetStartBone(CharacterizeResults.AutoRetargetDefinition.RetargetDefinition.PelvisBone, SolverIndex);
	if (!bSetRoot)
	{
		Results.Outcome = EAutoFBIKResult::MissingPelvisBone;
		return;
	}

	// update solver settings for retargeting
	FIKRigFullBodyIKSolver* Solver = static_cast<FIKRigFullBodyIKSolver*>(IKRigController.GetSolverAtIndex(SolverIndex));
	// set the root behavior to "free", allows pelvis motion only when needed to reach goals
	Solver->Settings.RootBehavior = EPBIKRootBehavior::Free;
	// removing pull chain alpha on all goals "calms" the motion down, especially when retargeting arms
	Solver->Settings.GlobalPullChainAlpha = 0.0f;
	// sub-iterations for chain-level sub-solves before each main iteration
	Solver->Settings.SubIterations = 10;

	// set chain depth on all goals
	for (FIKRigFBIKGoalSettings& GoalSettings : Solver->AllGoalSettings)
	{
		GoalSettings.ChainDepth = 2;
	}

	// assign bone settings from template
	const FAutoCharacterizer& Characterizer = IKRigController.GetAutoCharacterizer();
	const FTemplateHierarchy* TemplateHierarchy = Characterizer.GetKnownTemplateHierarchy(CharacterizeResults.BestTemplateName);
	if (!ensureAlways(TemplateHierarchy))
	{
		return;
	}
	const FAbstractHierarchy MeshAbstractHierarchy(Mesh);
	const TArray<FBoneSettingsForIK>& AllBoneSettings = TemplateHierarchy->AutoRetargetDefinition.BoneSettingsForIK.GetBoneSettings();
	for (const FBoneSettingsForIK& BoneSettings : AllBoneSettings)
	{
		// templates use "clean" names, free from prefixes, so we need to resolve this onto the actual skeletal mesh being setup
		const int32 BoneIndex = MeshAbstractHierarchy.GetBoneIndex(BoneSettings.BoneToApplyTo, ECleanOrFullName::Clean);
		if (BoneIndex == INDEX_NONE)
		{
			UE_LOGF(LogIKRigEditor, Warning, "Auto FBIK using a template with settings for a bone that is not in this skeletal mesh: %ls", *BoneSettings.BoneToApplyTo.ToString());
			continue;
		}
		const FName BoneFullName = MeshAbstractHierarchy.GetBoneName(BoneIndex, ECleanOrFullName::Full);
		
		// optionally exclude bones
		if (BoneSettings.bExcluded)
		{
			IKRigController.SetBoneExcluded(BoneFullName, true /*bExclude*/);
		}

		// get the bone settings for this bone for the FBIK solver (or add one if there is none)
		auto GetOrAddBoneSettings = [SolverIndex, &IKRigController, BoneFullName]() -> FIKRigFBIKBoneSettings*
		{
			if (!IKRigController.GetSettingsForBone(BoneFullName, SolverIndex))
			{
				IKRigController.AddBoneSetting(BoneFullName, SolverIndex);
			}
			
			return reinterpret_cast<FIKRigFBIKBoneSettings*>(IKRigController.GetSettingsForBone(BoneFullName, SolverIndex));
		};

		// apply rotational stiffness
		if (BoneSettings.RotationStiffness > 0.f)
		{
			FIKRigFBIKBoneSettings* Settings = GetOrAddBoneSettings();
			Settings->RotationStiffness = FMath::Clamp(BoneSettings.RotationStiffness, 0.f, 1.0f);
		}

		// apply position stiffness
		if (BoneSettings.PositionStiffness > 0.f)
		{
			FIKRigFBIKBoneSettings* Settings = GetOrAddBoneSettings();
			Settings->PositionStiffness = FMath::Clamp(BoneSettings.PositionStiffness, 0.f, 1.0f);
		}

		// apply preferred angles
		if (BoneSettings.PreferredAxis != EPreferredAxis::None)
		{
			FIKRigFBIKBoneSettings* Settings = GetOrAddBoneSettings();
			Settings->bUsePreferredAngles = true;
			Settings->PreferredAngles = BoneSettings.GetPreferredAxisAsAngles();
		}
		
		// apply locked axes
		if (BoneSettings.bIsHinge)
		{
			FIKRigFBIKBoneSettings* Settings = GetOrAddBoneSettings();
			BoneSettings.LockNonPreferredAxes(Settings->X, Settings->Y, Settings->Z);	
		}
	}
}

#undef LOCTEXT_NAMESPACE
