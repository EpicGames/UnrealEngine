// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigCommands.h"

#define LOCTEXT_NAMESPACE "IKRigCommands"

void FIKRigCommands::RegisterCommands()
{
	UI_COMMAND(Reset, "Reset", "Reset state of the rig and goals to initial pose.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AutoRetargetChains, "Auto Create Retarget Chains", "Compares the skeleton against known templates to automatically split it into retarget chains.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AutoSetupFBIK, "Auto Create IK", "Compares the skeleton against known templates to automatically setup a Full-Body IK for feet and hands.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowAssetSettings, "Asset Settings", "Show the settings for this IK Rig asset.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
