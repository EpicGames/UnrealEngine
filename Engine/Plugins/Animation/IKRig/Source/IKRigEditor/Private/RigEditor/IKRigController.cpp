// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigController.h"

#include "IKRigEditor.h"
#include "Rig/IKRigDefinition.h"
#include "Rig/IKRigProcessor.h"
#include "Rig/Solvers/IKRigSolverBase.h"

#include "Engine/SkeletalMesh.h"
#include "ScopedTransaction.h"
#include "RigEditor/IKRigAutoCharacterizer.h"
#include "RigEditor/IKRigAutoFBIK.h"
#include "RigEditor/IKRigStructViewer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigController)

#define LOCTEXT_NAMESPACE "IKRigController"

UIKRigController::UIKRigController()
{
	AutoCharacterizer = MakeUnique<FAutoCharacterizer>();
	AutoFBIKCreator = MakeUnique<FAutoFBIKCreator>();
	StructViewerPool.Init(this);
}

UIKRigController* UIKRigController::GetController(const UIKRigDefinition* InIKRigDefinition)
{
	if (!InIKRigDefinition)
	{
		return nullptr;
	}

	if (!InIKRigDefinition->Controller)
	{
		UIKRigController* Controller = NewObject<UIKRigController>();
		Controller->Asset = const_cast<UIKRigDefinition*>(InIKRigDefinition);
		Controller->Asset->Controller = Controller;
	}

	return Cast<UIKRigController>(InIKRigDefinition->Controller);
}

UIKRigDefinition* UIKRigController::GetAsset() const
{
	return Asset;
}

TObjectPtr<UIKRigDefinition>& UIKRigController::GetAssetPtr()
{
	return Asset;
}

bool UIKRigController::AddBoneSetting(const FName BoneName, int32 SolverIndex) const
{
	check(Asset->SolverStack.IsValidIndex(SolverIndex))

	FText ErrorMessage;
	if (!CanAddBoneSetting(BoneName, SolverIndex, &ErrorMessage))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Bone settings refused by solver. Reason: %ls", *ErrorMessage.ToString());
		return false; // prerequisites not met
	}
	
	FScopedTransaction Transaction(LOCTEXT("AddBoneSetting_Label", "Add Bone Setting"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	Solver->AddSettingsToBone(BoneName);
	return true;
}

bool UIKRigController::CanAddBoneSetting(const FName BoneName, int32 SolverIndex, FText* OutError) const
{
	const FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		if (OutError)
		{
			*OutError = FText::Format(
				NSLOCTEXT("IKRig", "NoSolverAtIndex", "Solver does not exist at index: {0}."),
				FText::AsNumber(SolverIndex));
		}
		return false;
	}

	if (!Asset || Asset->Skeleton.GetBoneIndexFromName(BoneName) == INDEX_NONE)
	{
		if (OutError)
		{
			*OutError = FText::Format(
				NSLOCTEXT("IKRig", "BoneMissing", "Bone does not exist: {0}."),
				FText::FromName(BoneName));
		}
		return false;
	}

	if (!Solver->UsesCustomBoneSettings())
	{
		if (OutError)
		{
			*OutError = NSLOCTEXT("IKRig", "NoCustomBoneSettings", "Solver does not support bone settings.");
		}
		return false;
	}

	if (Solver->HasSettingsOnBone(BoneName))
	{
		if (OutError)
		{
			*OutError = FText::Format(
				NSLOCTEXT("IKRig", "AlreadyHasSettings", "Solver already has settings on bone: {0}."),
				FText::FromName(BoneName));
		}
		return false;
	}

	if (OutError)
	{
		*OutError = FText::GetEmpty(); // no error
	}
	
	return true;
}

bool UIKRigController::RemoveBoneSetting(const FName BoneName, int32 SolverIndex) const
{
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Solver does not exist at index: %d.", SolverIndex);
		return false; // solver doesn't exist
	}

	if (Asset->Skeleton.GetBoneIndexFromName(BoneName) == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Cannot remove setting on unknown bone, %ls.", *BoneName.ToString());
		return false; // bone doesn't exist
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveBoneSetting_Label", "Remove Bone Setting"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	
	Solver->RemoveSettingsOnBone(BoneName);

	return true;
}

bool UIKRigController::CanRemoveBoneSetting(const FName BoneName, int32 SolverIndex) const
{
	const FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		return false; // solver doesn't exist
	}

	if (!Solver->UsesCustomBoneSettings())
	{
		return false; // solver doesn't use bone settings
	}

	if (Asset->Skeleton.GetBoneIndexFromName(BoneName) == INDEX_NONE)
	{
		return false; // bone doesn't exist
	}

	if (!Solver->HasSettingsOnBone(BoneName))
	{
		return false; // solver doesn't have any settings for this bone
	}

	return true;
}

UObject* UIKRigController::GetBoneSettings(const FName BoneName, int32 SolverIndex) const
{
	return nullptr;
}

FIKRigBoneSettingsBase* UIKRigController::GetSettingsForBone(const FName BoneName, int32 SolverIndex) const
{
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		return nullptr;
	}

	return Solver->GetBoneSettings(BoneName);
}

bool UIKRigController::DoesBoneHaveSettings(const FName BoneName) const
{
	if (Asset->Skeleton.GetBoneIndexFromName(BoneName) == INDEX_NONE)
	{
		return false; // bone doesn't exist (do not spam here)
	}
	
	for (int32 SolverIndex=0; SolverIndex < Asset->SolverStack.Num(); ++SolverIndex)
	{
		const FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
		if (Solver->HasSettingsOnBone(BoneName))
		{
			return true;
		}
	}

	return false;
}

FName UIKRigController::AddRetargetChain(const FName ChainName, const FName StartBoneName, const FName EndBoneName, const FName GoalName) const
{
	return AddRetargetChainInternal(FBoneChain(ChainName, StartBoneName, EndBoneName, GoalName));
}

FName UIKRigController::AddRetargetChainInternal(const FBoneChain& BoneChain) const
{
	if (BoneChain.StartBone.BoneName != NAME_None && Asset->Skeleton.GetBoneIndexFromName(BoneChain.StartBone.BoneName) == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Could not create retarget chain. Start Bone does not exist, %ls.", *BoneChain.StartBone.BoneName.ToString());
		return NAME_None; // start bone doesn't exist
	}

	if (BoneChain.EndBone.BoneName != NAME_None && Asset->Skeleton.GetBoneIndexFromName(BoneChain.EndBone.BoneName) == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Could not create retarget chain. End Bone does not exist, %ls.", *BoneChain.EndBone.BoneName.ToString());
		return NAME_None; // end bone doesn't exist
	}

	// uniquify the chain name
	FBoneChain ChainToAdd = BoneChain;
	ChainToAdd.ChainName = GetUniqueRetargetChainName(BoneChain.ChainName);
	
	FScopedTransaction Transaction(LOCTEXT("AddRetargetChain_Label", "Add Retarget Chain"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	const int32 NewChainIndex = Asset->RetargetDefinition.BoneChains.Emplace(ChainToAdd);
	RetargetChainAdded.Broadcast(Asset);
	return Asset->RetargetDefinition.BoneChains[NewChainIndex].ChainName;
}

bool UIKRigController::RemoveRetargetChain(const FName ChainName) const
{
	FScopedTransaction Transaction(LOCTEXT("RemoveRetargetChain_Label", "Remove Retarget Chain"));
	Asset->Modify();
	
	auto Pred = [ChainName](const FBoneChain& Element)
	{
		if (Element.ChainName == ChainName)
		{
			return true;
		}

		return false;
	};
	
	if (Asset->RetargetDefinition.BoneChains.RemoveAll(Pred) > 0)
	{
		RetargetChainRemoved.Broadcast(Asset, ChainName);
		FScopedReinitializeIKRig Reinitialize(this);
		return true;
	}

	UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
	return false;
}

FName UIKRigController::RenameRetargetChain(const FName ChainName, const FName NewChainName) const
{
	FBoneChain* Chain = Asset->RetargetDefinition.GetEditableBoneChainByName(ChainName);
	if (!Chain)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
		return ChainName; // chain doesn't exist to rename
	}

	// make sure it's unique
	const FName UniqueChainName = GetUniqueRetargetChainName(NewChainName);
	
	FScopedTransaction Transaction(LOCTEXT("RenameRetargetChain_Label", "Rename Retarget Chain"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	Chain->ChainName = UniqueChainName;
	RetargetChainRenamed.Broadcast(GetAsset(), ChainName, UniqueChainName);
	
	return UniqueChainName;
}

bool UIKRigController::SetRetargetChainStartBone(const FName ChainName, const FName StartBoneName) const
{
	if (Asset->Skeleton.GetBoneIndexFromName(StartBoneName) == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Bone does not exist, %ls.", *StartBoneName.ToString());
		return false; // bone doesn't exist
	}
	
	if (FBoneChain* BoneChain = Asset->RetargetDefinition.GetEditableBoneChainByName(ChainName))
	{
		FScopedTransaction Transaction(LOCTEXT("SetRetargetChainStartBone_Label", "Set Retarget Chain Start Bone"));
		FScopedReinitializeIKRig Reinitialize(this);
		Asset->Modify();
		BoneChain->StartBone = StartBoneName;
		return true;
	}

	UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
	return false; // no bone chain with that name
}

bool UIKRigController::SetRetargetChainEndBone(const FName ChainName, const FName EndBoneName) const
{
	if (Asset->Skeleton.GetBoneIndexFromName(EndBoneName) == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Bone does not exist, %ls.", *EndBoneName.ToString());
		return false; // bone doesn't exist
	}
	
	if (FBoneChain* BoneChain = Asset->RetargetDefinition.GetEditableBoneChainByName(ChainName))
	{
		FScopedTransaction Transaction(LOCTEXT("SetRetargetChainEndBone_Label", "Set Retarget Chain End Bone"));
		FScopedReinitializeIKRig Reinitialize(this);
		Asset->Modify();
		BoneChain->EndBone = EndBoneName;
		return true;
	}

	UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
	return false; // no bone chain with that name
}

bool UIKRigController::SetRetargetChainGoal(const FName ChainName, const FName GoalName) const
{
	FBoneChain* BoneChain = Asset->RetargetDefinition.GetEditableBoneChainByName(ChainName);
	if (!BoneChain)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
		return false; // no bone chain with that name
	}
	
	FName GoalNameToUse = GoalName;
	if (!GetGoal(GoalName))
	{
		GoalNameToUse = NAME_None; // no goal with that name, that's ok we set it to None
	}

	FScopedTransaction Transaction(LOCTEXT("SetRetargetChainGoal_Label", "Set Retarget Chain Goal"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	BoneChain->IKGoalName = GoalNameToUse;
	return true;
}

FName UIKRigController::GetRetargetChainGoal(const FName ChainName) const
{
	check(Asset)
	const FBoneChain* Chain = GetRetargetChainByName(ChainName);
	if (!Chain)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
		return NAME_None;
	}
	
	return Chain->IKGoalName;
}

FName UIKRigController::GetRetargetChainStartBone(const FName ChainName) const
{
	check(Asset)
	const FBoneChain* Chain = GetRetargetChainByName(ChainName);
	if (!Chain)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
		return NAME_None;
	}
	
	return Chain->StartBone.BoneName;
}

FName UIKRigController::GetRetargetChainEndBone(const FName ChainName) const
{
	check(Asset)
	const FBoneChain* Chain = GetRetargetChainByName(ChainName);
	if (!Chain)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
		return NAME_None;
	}
	
	return Chain->EndBone.BoneName;
}

const FBoneChain* UIKRigController::GetRetargetChainByName(const FName ChainName) const
{
	for (const FBoneChain& Chain : Asset->RetargetDefinition.BoneChains)
	{
		if (Chain.ChainName == ChainName)
		{
			return &Chain;
		}
	}

	UE_LOGF(LogIKRigEditor, Warning, "Retarget chain not found: %ls.", *ChainName.ToString());
	return nullptr;
}

const TArray<FBoneChain>& UIKRigController::GetRetargetChains() const
{
	check(Asset)
	return Asset->GetRetargetChains();
}

bool UIKRigController::SetRetargetRoot(const FName InPelvisBoneName) const
{
	check(Asset)

	FName NewPelvisBone = InPelvisBoneName;
	if (InPelvisBoneName != NAME_None && Asset->Skeleton.GetBoneIndexFromName(InPelvisBoneName) == INDEX_NONE)
	{
		NewPelvisBone = NAME_None;
	}

	FScopedTransaction Transaction(LOCTEXT("SetPelvisBone_Label", "Set Pelvis Bone"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	Asset->RetargetDefinition.PelvisBone = NewPelvisBone;
	return true;
}

FName UIKRigController::GetRetargetRoot() const
{
	check(Asset)

	return Asset->RetargetDefinition.PelvisBone;
}

bool UIKRigController::SetRootMotionBone(const FName InRootBoneName) const
{
	check(Asset)

	FName NewRootBone = InRootBoneName;
	if (InRootBoneName != NAME_None && Asset->Skeleton.GetBoneIndexFromName(InRootBoneName) == INDEX_NONE)
	{
		NewRootBone = NAME_None;
	}

	FScopedTransaction Transaction(LOCTEXT("SetRootBone_Label", "Set Root Bone"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	Asset->RetargetDefinition.RootBone = NewRootBone;
	return true;
}

FName UIKRigController::GetRootMotionBone() const
{
	check(Asset);
	return Asset->RetargetDefinition.RootBone;
}

void UIKRigController::SetRetargetDefinition(const FRetargetDefinition& RetargetDefinition) const
{
	check(Asset)
	
	FScopedTransaction Transaction(LOCTEXT("SetRetargetDefinition_Label", "Set Retarget Definition"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();

	Asset->RetargetDefinition = RetargetDefinition;
}

bool UIKRigController::ApplyAutoGeneratedRetargetDefinition()
{
	check(Asset)
	
	USkeletalMesh* Mesh = GetSkeletalMesh();
	if (!Mesh)
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("ApplyAutoCharacterization_Label", "Apply Auto Characterization"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	
	// apply an auto generated retarget definition
	FAutoCharacterizeResults Results;
	AutoCharacterizer.Get()->GenerateRetargetDefinitionFromMesh(Mesh, Results);
	if (!Results.bUsedTemplate)
	{
		return false;
	}
	
	SetRetargetDefinition(Results.AutoRetargetDefinition.RetargetDefinition);
	return true;
}

bool UIKRigController::ApplyAutoFBIK()
{
	check(Asset)
	
	USkeletalMesh* Mesh = GetSkeletalMesh();
	if (!Mesh)
	{
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("ApplyAutoFBIK_Label", "Apply Auto FBIK"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	
	// apply an auto generated FBIK setup
	FAutoFBIKResults Results;
	AutoFBIKCreator.Get()->CreateFBIKSetup(*this, Results);
	return Results.Outcome == EAutoFBIKResult::AllOk;
}

void UIKRigController::AutoGenerateRetargetDefinition(FAutoCharacterizeResults& Results) const
{
	USkeletalMesh* Mesh = GetSkeletalMesh();
	if (!Mesh)
	{
		return;
	}
	
	AutoCharacterizer.Get()->GenerateRetargetDefinitionFromMesh(Mesh, Results);
}

void UIKRigController::AutoGenerateFBIK(FAutoFBIKResults& Results) const
{
	FScopedTransaction Transaction(LOCTEXT("AutoFBIK_Label", "Auto Setup FBIK"));
	FScopedReinitializeIKRig Reinitialize(this);
	AutoFBIKCreator.Get()->CreateFBIKSetup(*this, Results);
}

const FAutoCharacterizer& UIKRigController::GetAutoCharacterizer() const
{
	return *AutoCharacterizer.Get();
}

void UIKRigController::SortRetargetChains() const
{
	Asset->RetargetDefinition.BoneChains.Sort([this](const FBoneChain& A, const FBoneChain& B)
	{
		const int32 IndexA = Asset->Skeleton.GetBoneIndexFromName(A.StartBone.BoneName);
		const int32 IndexB = Asset->Skeleton.GetBoneIndexFromName(B.StartBone.BoneName);
		if (IndexA == IndexB)
		{
			// fallback to sorting alphabetically
			return A.ChainName.LexicalLess(B.ChainName);
		}
		return IndexA < IndexB;
	});
}

FName UIKRigController::GetUniqueRetargetChainName(FName NameToMakeUnique) const
{
	auto IsNameBeingUsed = [this](const FName NameToTry)->bool
	{
		for (const FBoneChain& Chain : Asset->RetargetDefinition.BoneChains)
		{
			if (Chain.ChainName == NameToTry)
			{
				return true;
			}
		}
		return false;
	};

	// if no name specified, use a default
	if (NameToMakeUnique == NAME_None)
	{
		NameToMakeUnique = FName("DefaultChainName");
	}

	// check if name is already unique
	if (!IsNameBeingUsed(NameToMakeUnique))
	{
		return NameToMakeUnique; 
	}
	
	// keep concatenating an incremented integer suffix until name is unique
	int32 Number = NameToMakeUnique.GetNumber() + 1;
	while(IsNameBeingUsed(FName(NameToMakeUnique, Number)))
	{
		Number++;
	}

	return FName(NameToMakeUnique, Number);
}

bool UIKRigController::ValidateChain(
	const FName ChainName,
	const FIKRigSkeleton* OptionalSkeleton,
	TSet<int32>& OutChainIndices) const
{
	const FBoneChain* Chain = GetRetargetChainByName(ChainName);
	if (!Chain)
	{
		return false; // chain doesn't exist
	}

	const FIKRigSkeleton &Skeleton = OptionalSkeleton ? *OptionalSkeleton : GetIKRigSkeleton();
	const int32 StartBoneIndex = Skeleton.GetBoneIndexFromName(Chain->StartBone.BoneName);
	const int32 EndBoneIndex = Skeleton.GetBoneIndexFromName(Chain->EndBone.BoneName);

	const bool bHasStartBone = StartBoneIndex != INDEX_NONE;
	const bool bHasEndBone = EndBoneIndex != INDEX_NONE;

	// chain has neither start nor end bone
	if (!bHasStartBone && !bHasEndBone)
	{
		return false;
	}

	// has only a start bone, this is a single bone "chain" which is fine
	if (bHasStartBone && !bHasEndBone)
	{
		OutChainIndices.Add(StartBoneIndex);
		return true; 
	}

	// has only a end bone, not valid
	if (!bHasStartBone && bHasEndBone)
	{
		OutChainIndices.Add(EndBoneIndex);
		return false; 
	}
	
	// this chain has a start AND an end bone so we must verify that end bone is child of start bone
	int32 NextBoneIndex = EndBoneIndex;
	while (true)
	{
		OutChainIndices.Add(NextBoneIndex);
		if (StartBoneIndex == NextBoneIndex)
		{
			return true;
		}
		
		NextBoneIndex = Skeleton.GetParentIndex(NextBoneIndex);
		if (NextBoneIndex == INDEX_NONE)
		{
			// oops, we walked all the way past the root without finding the start bone
			OutChainIndices.Reset();
			OutChainIndices.Add(EndBoneIndex);
			OutChainIndices.Add(StartBoneIndex);
			return false;
		}
	}
}

TArray<UIKRigStructWrapperBase*> UIKRigController::GetStructWrappers(const int32 InNum, TSubclassOf<UIKRigStructWrapperBase> InType)
{
	return StructViewerPool.GetStructWrappers(InNum, InType);
}

FName UIKRigController::GetRetargetChainFromBone(const FName BoneName, const FIKRigSkeleton* OptionalSkeleton) const
{
	const FIKRigSkeleton& Skeleton = OptionalSkeleton ? *OptionalSkeleton : GetIKRigSkeleton();
	const int32 BoneIndex = Skeleton.GetBoneIndexFromName(BoneName);

	if (BoneName == GetRetargetRoot())
	{
		return FName("Pelvis");
	}
	
	const TArray<FBoneChain>& Chains = GetRetargetChains();
	TSet<int32> OutBoneIndices;
	for (const FBoneChain& Chain : Chains)
	{
		OutBoneIndices.Reset();
		if (ValidateChain(Chain.ChainName, OptionalSkeleton, OutBoneIndices))
		{
			if (OutBoneIndices.Contains(BoneIndex))
			{
				return Chain.ChainName;
			}
		}
	}

	return NAME_None;
}

FName UIKRigController::GetRetargetChainFromGoal(const FName GoalName) const
{
	if (GoalName == NAME_None)
	{
		return NAME_None;
	}


	
	const TArray<FBoneChain>& Chains = GetRetargetChains();
	for (const FBoneChain& Chain : Chains)
	{
		if (Chain.IKGoalName == GoalName)
		{
			return Chain.ChainName;
		}
	}

	return NAME_None;
}

// -------------------------------------------------------
// SKELETON
//

bool UIKRigController::SetSkeletalMesh(USkeletalMesh* SkeletalMesh) const
{
	// first determine runtime compatibility between the IK Rig asset and the skeleton we're trying to run it on
	if (!IsSkeletalMeshCompatible(SkeletalMesh))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to initialize IKRig with a Skeleton that is missing required bones. See output log. %ls", *Asset->GetName());
		FScopedReinitializeIKRig Reinitialize(this);
		return false;
	}

	const bool bShouldActuallyTransact = Asset->PreviewSkeletalMesh != SkeletalMesh;
	FScopedTransaction Transaction(LOCTEXT("SetSkeletalMesh_Label", "Set Skeletal Mesh"), bShouldActuallyTransact);
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	// update stored skeletal mesh used for previewing results
	Asset->PreviewSkeletalMesh = SkeletalMesh;
	// copy skeleton data from the actual skeleton we want to run on
	const FIKRigInputSkeleton InputSkeleton = FIKRigInputSkeleton(SkeletalMesh);
	Asset->Skeleton.SetInputSkeleton(InputSkeleton, GetAsset()->Skeleton.ExcludedBones);
	return true;
}

bool UIKRigController::IsSkeletalMeshCompatible(USkeletalMesh* SkeletalMeshToCheck) const
{
	const FIKRigInputSkeleton InputSkeleton = FIKRigInputSkeleton(SkeletalMeshToCheck);
	return FIKRigProcessor::IsIKRigCompatibleWithSkeleton(Asset, InputSkeleton, nullptr);
}

USkeletalMesh* UIKRigController::GetSkeletalMesh() const
{
	return Asset->PreviewSkeletalMesh.LoadSynchronous();
}

const FIKRigSkeleton& UIKRigController::GetIKRigSkeleton() const
{
	return Asset->Skeleton;
}

int32 UIKRigController::AddSolver(const FString InIKRigSolverType) const
{
	UScriptStruct* SolverType = FindObject<UScriptStruct>(nullptr, *InIKRigSolverType);
	if (!SolverType)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Specified solver type was not found, %ls.", *InIKRigSolverType);
		return INDEX_NONE;
	}

	return AddSolver(SolverType);
}

bool UIKRigController::SetBoneExcluded(const FName BoneName, const bool bExclude) const
{
	// does bone exist?
	const int32 BoneIndex = Asset->Skeleton.GetBoneIndexFromName(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to exclude non-existant bone, %ls.", *BoneName.ToString());
		return false;
	}

	// already excluded?
	const bool bIsExcluded = Asset->Skeleton.ExcludedBones.Contains(BoneName);
	if (bIsExcluded == bExclude)
	{
		return false; // (don't spam warning)
	}

	FScopedTransaction Transaction(LOCTEXT("SetBoneExcluded_Label", "Set Bone Excluded"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	if (bExclude)
	{
		Asset->Skeleton.ExcludedBones.Add(BoneName);
	}
	else
	{
		Asset->Skeleton.ExcludedBones.Remove(BoneName);
	}
	return true;
}

bool UIKRigController::GetBoneExcluded(const FName BoneName) const
{
	return Asset->Skeleton.ExcludedBones.Contains(BoneName);
}

FTransform UIKRigController::GetRefPoseTransformOfBone(const FName BoneName) const
{	
	const int32 BoneIndex = Asset->Skeleton.GetBoneIndexFromName(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Tried to get the ref pose of bone that is not loaded into this rig.");
		return FTransform::Identity;
	}
	
	return Asset->Skeleton.RefPoseGlobal[BoneIndex];
}

// -------------------------------------------------------
// SOLVERS
//

int32 UIKRigController::AddSolver(UScriptStruct* InIKRigSolverType) const
{
	check(Asset)

	if (!InIKRigSolverType)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Could not add solver to IK Rig. No solver type specified.");
		return INDEX_NONE;
	}

	if (!InIKRigSolverType->IsChildOf(FIKRigSolverBase::StaticStruct()))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Could not add solver to IK Rig. Invalid solver type specified. Must be child of FIKRigSolverBase.");
		return INDEX_NONE;
	}

	FScopedTransaction Transaction(LOCTEXT("AddSolver_Label", "Add Solver"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	const int32 NewSolverIndex = Asset->SolverStack.Emplace(InIKRigSolverType);
	return NewSolverIndex;
}

bool UIKRigController::RemoveSolver(const int32 SolverIndex) const
{
	check(Asset)
	
	if (!Asset->SolverStack.IsValidIndex(SolverIndex))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Solver not removed. Invalid index, %d.", SolverIndex);
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveSolver_Label", "Remove Solver"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	Asset->SolverStack.RemoveAt(SolverIndex);
	return true;
}

bool UIKRigController::MoveSolverInStack(int32 SolverToMoveIndex, int32 TargetSolverIndex) const
{
	if (!Asset->SolverStack.IsValidIndex(SolverToMoveIndex))
	{
		UE_LOGF(LogIKRigEditor, Warning, "Solver not moved. Invalid source index, %d.", SolverToMoveIndex);
		return false;
	}

	// allow a target 1 greater than the last element (for dragging below last element of list)
	if (TargetSolverIndex > Asset->SolverStack.Num() || TargetSolverIndex < 0)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Solver not moved. Invalid target index, %d.", TargetSolverIndex);
		return false;
	}

	if (SolverToMoveIndex == TargetSolverIndex)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Solver not moved. Source and target index cannot be the same.");
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("ReorderSolver_Label", "Reorder Solvers"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	
	// extract the element to move
	FInstancedStruct MovedSolver = MoveTemp(Asset->SolverStack[SolverToMoveIndex]);
	// Remove the element without shrinking the array
	Asset->SolverStack.RemoveAt(SolverToMoveIndex, 1, EAllowShrinking::No);
	// Insert at the corrected TargetIndex
	Asset->SolverStack.Insert(MoveTemp(MovedSolver), TargetSolverIndex);
	
	return true;
}

bool UIKRigController::SetSolverEnabled(int32 SolverIndex, bool bIsEnabled) const
{
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Solver not enabled or disabled. Invalid index, %d.", SolverIndex);
		return false;
	}
	
	FScopedTransaction Transaction(LOCTEXT("SetSolverEndabled_Label", "Enable/Disable Solver"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	Solver->SetEnabled(bIsEnabled);
	return true;
}

bool UIKRigController::GetSolverEnabled(int32 SolverIndex) const
{
	const FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Invalid solver index, %d.", SolverIndex);
		return false;
	}

	return Solver->IsEnabled();
}

bool UIKRigController::SetStartBone(const FName RootBoneName, int32 SolverIndex) const
{
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Start bone not set. Invalid solver index, %d.", SolverIndex);
		return false; // solver doesn't exist
	}

	if (Asset->Skeleton.GetBoneIndexFromName(RootBoneName) == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Start bone not set. Invalid bone specified, %ls.", *RootBoneName.ToString());
		return false; // bone doesn't exist
	}

	FScopedTransaction Transaction(LOCTEXT("SetStartBone_Label", "Set Start Bone"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	Solver->SetStartBone(RootBoneName);
	return true;
}

FName UIKRigController::GetStartBone(int32 SolverIndex) const
{
	const FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Could not query root bone. Invalid solver index, %d.", SolverIndex);
		return NAME_None; // solver doesn't exist
	}

	return Solver->GetStartBone();
}

bool UIKRigController::SetEndBone(const FName EndBoneName, int32 SolverIndex) const
{
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "End bone not set. Invalid solver index, %d.", SolverIndex);
		return false; // solver doesn't exist
	}

	if (Asset->Skeleton.GetBoneIndexFromName(EndBoneName) == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "End bone not set. Invalid bone specified, %ls.", *EndBoneName.ToString());
		return false; // bone doesn't exist
	}

	if (!Solver->UsesEndBone())
	{
		UE_LOGF(LogIKRigEditor, Warning, "End bone not set. Specified solver does not support end bones.");
		return false; //
	}

	FScopedTransaction Transaction(LOCTEXT("SetEndBone_Label", "Set End Bone"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	Solver->SetEndBone(EndBoneName);
	return true;
}

FName UIKRigController::GetEndBone(int32 SolverIndex) const
{
	const FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "End bone not queried. Invalid solver index, %d.", SolverIndex);
		return NAME_None; // solver doesn't exist
	}

	return Solver->GetEndBone();
}

UIKRigSolverControllerBase* UIKRigController::GetSolverController(int32 SolverIndex)
{
	FIKRigSolverBase* SolverAtIndex = GetSolverAtIndex(SolverIndex);
	if (!SolverAtIndex)
	{
		return nullptr;
	}

	return SolverAtIndex->GetSolverController(this);
}

int32 UIKRigController::GetIndexOfSolver(UIKRigSolverControllerBase* InController)
{
	if (!InController)
	{
		return INDEX_NONE;
	}
	
	for (int32 SolverIndex = 0; SolverIndex<Asset->SolverStack.Num(); SolverIndex++)
	{
		if (GetSolverAtIndex(SolverIndex) == InController->SolverToControl)
		{
			return SolverIndex;
		}
	}
	
	return INDEX_NONE;
}

const TArray<FIKRigSolverBase*> UIKRigController::GetSolverArray() const
{
	TArray<FIKRigSolverBase*> SolverPointers;
	for (int32 SolverIndex = 0; SolverIndex<Asset->SolverStack.Num(); SolverIndex++)
	{
		SolverPointers.Add(GetSolverAtIndex(SolverIndex));
	}
	return MoveTemp(SolverPointers);
}

FString UIKRigController::GetSolverUniqueName(int32 SolverIndex) const
{
	check(Asset)
	const FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		checkNoEntry()
		return "";
	}

	return FString::FromInt(SolverIndex+1) + " - " + Solver->GetNiceName().ToString();
		
}

int32 UIKRigController::GetNumSolvers() const
{
	check(Asset)
	return Asset->SolverStack.Num();
}

FIKRigSolverBase* UIKRigController::GetSolverAtIndex(int32 Index) const
{
	check(Asset)

	if (Asset->SolverStack.IsValidIndex(Index))
	{
		return Asset->SolverStack[Index].GetMutablePtr<FIKRigSolverBase>();
	}
	
	return nullptr;
}

FInstancedStruct* UIKRigController::GetSolverStructAtIndex(int32 Index) const
{
	check(Asset)

	if (Asset->SolverStack.IsValidIndex(Index))
	{
		return &Asset->SolverStack[Index];
	}
	
	return nullptr;
}

int32 UIKRigController::GetIndexOfSolver(FIKRigSolverBase* Solver) const
{
	for (int32 SolverIndex = 0; SolverIndex<Asset->SolverStack.Num(); SolverIndex++)
	{
		if (GetSolverAtIndex(SolverIndex) == Solver)
		{
			return SolverIndex;
		}
	}
	
	return INDEX_NONE;
}

// -------------------------------------------------------
// GOALS
//

FName UIKRigController::AddNewGoal(const FName GoalName, const FName BoneName) const
{
	// does goal already exist?
	if (GetGoalIndex(GoalName) != INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to create a Goal that already exists, %ls.", *GoalName.ToString());
		return NAME_None;
	}

	// does this bone exist?
	const int32 BoneIndex = Asset->Skeleton.GetBoneIndexFromName(BoneName);
	if (BoneIndex == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to create Goal on unknown bone, %ls.", *BoneName.ToString());
		return NAME_None;
	}
	
	FScopedTransaction Transaction(LOCTEXT("AddNewGoal_Label", "Add New Goal"));
	FScopedReinitializeIKRig Reinitialize(this, true /*bGoalsChanged*/);
	Asset->Modify();

	UIKRigEffectorGoal* NewGoal = NewObject<UIKRigEffectorGoal>(Asset, UIKRigEffectorGoal::StaticClass(), NAME_None, RF_Transactional);
	NewGoal->BoneName = BoneName;
	NewGoal->GoalName = GoalName;
	Asset->Goals.Add(NewGoal);

	// set initial transform
	NewGoal->InitialTransform = GetRefPoseTransformOfBone(NewGoal->BoneName);
	NewGoal->CurrentTransform = NewGoal->InitialTransform;
	
	return NewGoal->GoalName;
}

bool UIKRigController::RemoveGoal(const FName GoalName) const
{
	const int32 GoalIndex = GetGoalIndex(GoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return false; // can't remove goal we don't have
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveGoal_Label", "Remove Goal"));
	FScopedReinitializeIKRig Reinitialize(this, true /*bGoalsChanged*/);
	Asset->Modify();
	
	// remove from all the solvers
	const FName GoalToRemove = Asset->Goals[GoalIndex]->GoalName;
	for (int32 SolverIndex=0; SolverIndex<Asset->SolverStack.Num(); SolverIndex++)
	{
		FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
		Solver->OnGoalRemoved(GoalToRemove);
	}

	// remove from core system
	Asset->Goals.RemoveAt(GoalIndex);

	// clean any retarget chains that might reference the missing goal
	for (FBoneChain& BoneChain : Asset->RetargetDefinition.BoneChains)
	{
		if (BoneChain.IKGoalName == GoalName)
		{
			BoneChain.IKGoalName = NAME_None;
		}
	}

	return true;
}

FName UIKRigController::RenameGoal(const FName OldName, const FName PotentialNewName) const
{
	if (OldName == PotentialNewName)
	{
		return OldName; // skipping renaming the same name
	}
	
	const int32 GoalIndex = GetGoalIndex(OldName);
	if (GoalIndex == INDEX_NONE)
	{
		return NAME_None; // can't rename goal we don't have
	}
	
	// sanitize the potential new name
	FString CleanName = PotentialNewName.ToString();
	SanitizeGoalName(CleanName);
	// make the name unique
	const FName NewName = GetUniqueGoalName(FName(CleanName));
	
	FScopedTransaction Transaction(LOCTEXT("RenameGoal_Label", "Rename Goal"));
	FScopedReinitializeIKRig Reinitialize(this, true /*bGoalsChanged*/);
	Asset->Modify();

	// rename in core
	Asset->Goals[GoalIndex]->Modify();
	Asset->Goals[GoalIndex]->GoalName = NewName;

	// update any retarget chains that might reference the goal
	for (FBoneChain& BoneChain : Asset->RetargetDefinition.BoneChains)
	{
		if (BoneChain.IKGoalName == OldName)
		{
			BoneChain.IKGoalName = NewName;
		}
	}

	// rename in solvers
	for (int32 SolverIndex=0; SolverIndex<Asset->SolverStack.Num(); SolverIndex++)
	{
		FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
		Solver->OnGoalRenamed(OldName, NewName);
	}

	return NewName;
}

FName UIKRigController::GetUniqueGoalName(const FName NameToMakeUnique) const
{
	auto IsNameBeingUsed = [this](const FName NameToTry) -> bool
	{
		// check if this goal already exists (case sensitive)
		int32 ExistingGoalIndex = GetGoalIndex(NameToTry, ENameCase::IgnoreCase);
		return ExistingGoalIndex != INDEX_NONE;
	};

	// check if name is already unique
	if (!IsNameBeingUsed(NameToMakeUnique))
	{
		return NameToMakeUnique; 
	}
	
	// keep concatenating an incremented integer suffix until name is unique
	int32 Number = NameToMakeUnique.GetNumber() + 1;
	while(IsNameBeingUsed(FName(NameToMakeUnique, Number)))
	{
		Number++;
	}

	return FName(NameToMakeUnique, Number);
}

bool UIKRigController::ModifyGoal(const FName GoalName) const
{
	if (UIKRigEffectorGoal* Goal = GetGoal(GoalName))
	{
		Goal->Modify();
		return true;
	}
	
	return false;
}

bool UIKRigController::SetGoalBone(const FName GoalName, const FName NewBoneName) const
{
	const int32 GoalIndex = GetGoalIndex(GoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return false; // goal doesn't exist in the rig
	}

	const int32 BoneIndex = GetIKRigSkeleton().GetBoneIndexFromName(NewBoneName);
	if (BoneIndex == INDEX_NONE)
	{
		return false; // bone does not exist in the skeleton
	}

	if (GetBoneForGoal(GoalName) == NewBoneName)
	{
		return false; // goal is already using this bone
	}
	
	FScopedTransaction Transaction(LOCTEXT("SetGoalBone_Label", "Set Goal Bone"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();

	// update goal's bone name and it's initial transform
	TObjectPtr<UIKRigEffectorGoal> Goal = Asset->Goals[GoalIndex]; 
	Goal->Modify();
	Goal->BoneName = NewBoneName;
	Goal->InitialTransform = GetRefPoseTransformOfBone(NewBoneName);
	
	// update in solvers
	for (int32 SolverIndex=0; SolverIndex<Asset->SolverStack.Num(); SolverIndex++)
	{
		FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
		Solver->OnGoalMovedToDifferentBone(GoalName, NewBoneName);
	}

	return true;
}

FName UIKRigController::GetBoneForGoal(const FName GoalName) const
{
	for (const UIKRigEffectorGoal* Goal : Asset->Goals)
	{
		if (Goal->GoalName == GoalName)
		{
			return Goal->BoneName;
		}
	}
	
	return NAME_None;
}

FName UIKRigController::GetGoalNameForBone(const FName BoneName) const
{
	const TArray<UIKRigEffectorGoal*>& AllGoals = GetAllGoals();
	for (UIKRigEffectorGoal* Goal : AllGoals)
	{
		if (Goal->BoneName == BoneName)
		{
			return Goal->GoalName;
		}
	}

	return NAME_None;
}

bool UIKRigController::ConnectGoalToSolver(const FName GoalName, int32 SolverIndex) const
{
	// get index of the goal with this name
	const int32 GoalIndex = GetGoalIndex(GoalName);
	
	// can't add goal that is not present
	if (GoalIndex == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to connect unknown Goal, %ls to a solver.", *GoalName.ToString());
		return false;
	}
	
	// can't add goal to a solver with an invalid index
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to connect Goal, %ls to a unknown solver with index, %d.", *GoalName.ToString(), SolverIndex);
		return false;
	}

	FScopedTransaction Transaction(LOCTEXT("ConnectGoalSolver_Label", "Connect Goal to Solver"));
	FScopedReinitializeIKRig Reinitialize(this);
	const UIKRigEffectorGoal& Goal = *GetGoal(GoalName);
	Solver->AddGoal(Goal);
	return true;
}

bool UIKRigController::DisconnectGoalFromSolver(const FName GoalToRemove, int32 SolverIndex) const
{
	// can't remove goal that is not present in the core
	if (GetGoalIndex(GoalToRemove) == INDEX_NONE)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to disconnect unknown Goal, %ls.", *GoalToRemove.ToString());
		return false;
	}
	
	// can't remove goal from a solver with an invalid index
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		UE_LOGF(LogIKRigEditor, Warning, "Trying to disconnect Goal, %ls from an unknown solver with index, %d.", *GoalToRemove.ToString(), SolverIndex);
		return false;
	}

    FScopedTransaction Transaction(LOCTEXT("DisconnectGoalSolver_Label", "Disconnect Goal from Solver"));
	FScopedReinitializeIKRig Reinitialize(this);
	Asset->Modify();
	
	Solver->OnGoalRemoved(GoalToRemove);
	return true;
}

bool UIKRigController::IsGoalConnectedToSolver(const FName GoalName, int32 SolverIndex) const
{
	const FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		return false;
	}

	TSet<FName> RequiredGoals;
	Solver->GetRequiredGoals(RequiredGoals);

	return RequiredGoals.Contains(GoalName);
}

bool UIKRigController::IsGoalConnectedToAnySolver(const FName GoalName) const
{
	TSet<FName> RequiredGoals;
	for (int32 SolverIndex=0; SolverIndex<Asset->SolverStack.Num(); SolverIndex++)
	{
		GetSolverAtIndex(SolverIndex)->GetRequiredGoals(RequiredGoals);
	}

	return RequiredGoals.Contains(GoalName);
}

const TArray<UIKRigEffectorGoal*>& UIKRigController::GetAllGoals() const
{
	return Asset->Goals;
}

UIKRigEffectorGoal* UIKRigController::GetGoal(const FName GoalName) const
{
	const int32 GoalIndex = GetGoalIndex(GoalName);
	if (GoalIndex == INDEX_NONE)
	{
		return nullptr;
	}
	
	return Asset->Goals[GoalIndex];
}

UObject* UIKRigController::GetGoalSettingsForSolver(const FName GoalName, int32 SolverIndex) const
{
	return nullptr;
}

bool UIKRigController::DoesSolverHaveCustomGoalSettings(const FName GoalName, int32 SolverIndex) const
{
	FIKRigSolverBase* Solver = GetSolverAtIndex(SolverIndex);
	if (!Solver)
	{
		return false;
	}
	
	return Solver->GetGoalSettings(GoalName) != nullptr;
}

FTransform UIKRigController::GetGoalCurrentTransform(const FName GoalName) const
{
	if(const UIKRigEffectorGoal* Goal = GetGoal(GoalName))
	{
		return Goal->CurrentTransform;
	}
	
	return FTransform::Identity; // no goal with that name
}

void UIKRigController::SetGoalCurrentTransform(const FName GoalName, const FTransform& Transform) const
{
	UIKRigEffectorGoal* Goal = GetGoal(GoalName);
	check(Goal);

	Goal->CurrentTransform = Transform;
}

void UIKRigController::ResetGoalTransforms() const
{
	FScopedTransaction Transaction(LOCTEXT("ResetGoalTransforms", "Reset All Goal Transforms"));
	
	for (UIKRigEffectorGoal* Goal : Asset->Goals)
    {
		Goal->Modify();
    	const FTransform InitialTransform = GetRefPoseTransformOfBone(Goal->BoneName);
    	Goal->InitialTransform = InitialTransform;
		Goal->CurrentTransform = InitialTransform;
    }
}

void UIKRigController::ResetInitialGoalTransforms() const
{
	for (UIKRigEffectorGoal* Goal : Asset->Goals)
	{
		// record the current delta relative to the current bone
		FTransform Delta = Goal->CurrentTransform.GetRelativeTransform(Goal->InitialTransform);
		// get the initial transform based on the ref pose of the bone it's attached to
		const FTransform NewInitialTransform = GetRefPoseTransformOfBone(Goal->BoneName);
		// update the initial transform
		Goal->InitialTransform = NewInitialTransform;
		// reapply the delta
		Goal->CurrentTransform = Delta * NewInitialTransform;
	}
}

void UIKRigController::SanitizeGoalName(FString& InOutName)
{
	for (int32 i = 0; i < InOutName.Len(); ++i)
	{
		TCHAR& C = InOutName[i];

		const bool bGoodChar =
            ((C >= 'A') && (C <= 'Z')) || ((C >= 'a') && (C <= 'z')) ||		// A-Z (upper and lowercase) anytime
            (C == '_') || (C == '-') || (C == '.') ||						// _  - . anytime
            ((i > 0) && (C >= '0') && (C <= '9'));							// 0-9 after the first character

		if (!bGoodChar)
		{
			C = '_';
		}
	}

	// FIXME magic numbers should actually mean something
	static constexpr int32 MaxNameLength = 100;
	if (InOutName.Len() > MaxNameLength)
	{
		InOutName.LeftChopInline(InOutName.Len() - MaxNameLength);
	}
}

int32 UIKRigController::GetGoalIndex(const FName InGoalName, const ENameCase CompareMethod) const
{
	return Asset->Goals.IndexOfByPredicate([&](const TObjectPtr<UIKRigEffectorGoal>& Goal)
	{
		return Goal->GoalName.IsEqual(InGoalName, CompareMethod); 
	});
}

void UIKRigController::BroadcastGoalsChange() const
{
	if(ensure(Asset))
	{
		static const FName GoalsPropName = GET_MEMBER_NAME_CHECKED(UIKRigDefinition, Goals);
		if (FProperty* GoalProperty = UIKRigDefinition::StaticClass()->FindPropertyByName(GoalsPropName) )
		{
			FPropertyChangedEvent GoalPropertyChangedEvent(GoalProperty);
			FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(Asset, GoalPropertyChangedEvent);
		}	
	}
}

void UIKRigController::BroadcastNeedsReinitialized() const
{
	// initialize all solvers
	const FIKRigSkeleton& IKRigSkeleton = GetIKRigSkeleton();
	const TArray<FIKRigSolverBase*> Solvers = GetSolverArray(); 
	for (FIKRigSolverBase* Solver: Solvers)
	{
		Solver->Initialize(IKRigSkeleton);
	}

	// ensure goals are using initial transforms from the current mesh 
	ResetInitialGoalTransforms();

	// inform outside systems
	IKRigNeedsInitialized.Broadcast(GetAsset());
}

#undef LOCTEXT_NAMESPACE
