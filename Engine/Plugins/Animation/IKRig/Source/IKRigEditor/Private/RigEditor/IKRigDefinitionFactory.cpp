// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigDefinitionFactory.h"

#include "AssetToolsModule.h"
#include "Rig/IKRigDefinition.h"
#include "AssetTypeCategories.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigDefinitionFactory)

#define LOCTEXT_NAMESPACE "IKRigDefinitionFactory"


UIKRigDefinitionFactory::UIKRigDefinitionFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = UIKRigDefinition::StaticClass();
}

UObject* UIKRigDefinitionFactory::FactoryCreateNew(
	UClass* Class,
	UObject* InParent,
	FName InName,
	EObjectFlags InFlags,
	UObject* Context, 
	FFeedbackContext* Warn)
{
	// create the IK Rig asset
	return NewObject<UIKRigDefinition>(InParent, InName, InFlags | RF_Transactional);
}

bool UIKRigDefinitionFactory::ShouldShowInNewMenu() const
{
	return true;
}

FText UIKRigDefinitionFactory::GetDisplayName() const
{
	return LOCTEXT("IKRigDefinition_DisplayName", "IK Rig");
}

uint32 UIKRigDefinitionFactory::GetMenuCategories() const
{
	return EAssetTypeCategories::Animation;
}

FText UIKRigDefinitionFactory::GetToolTip() const
{
	return LOCTEXT("IKRigDefinition_Tooltip", "Defines a set of IK Solvers and Effectors to pose a skeleton with Goals.");
}

FString UIKRigDefinitionFactory::GetDefaultNewAssetName() const
{
	return FString(TEXT("IK_NewIKRig"));
}

UIKRigDefinition* UIKRigDefinitionFactory::CreateNewIKRigAsset(const FString& InPackagePath, const FString& InAssetName)
{
	// add a trailing forward slash in case user forgot
	FString DesiredPackagePath = InPackagePath;
	if (!DesiredPackagePath.EndsWith(TEXT("/")))
	{
		DesiredPackagePath.Append(TEXT("/"));
	}
	DesiredPackagePath = DesiredPackagePath / InAssetName;
	
	// create unique package and asset names
	FString UniqueAssetName = InAssetName;
	FString UniquePackageName;
	const FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(DesiredPackagePath, TEXT(""), UniquePackageName, UniqueAssetName);
	if (UniquePackageName.EndsWith(UniqueAssetName))
	{
		UniquePackageName = UniquePackageName.LeftChop(UniqueAssetName.Len() + 1);
	}
	
	// create the new IK Rig asset
	UIKRigDefinitionFactory* Factory = NewObject<UIKRigDefinitionFactory>();
	UObject* NewAsset = AssetToolsModule.Get().CreateAsset(*UniqueAssetName, *UniquePackageName, nullptr, Factory);
	UIKRigDefinition* IKRig = Cast<UIKRigDefinition>(NewAsset);
	return IKRig;
}

#undef LOCTEXT_NAMESPACE

