// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigDetailCustomizations.h"

#include "Editor.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailPropertyRow.h"
#include "IKRig.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Retargeter/RetargetOps/PinBoneOp.h"

#define LOCTEXT_NAMESPACE "IKRigDetailCustomizations"

namespace IKRigDetailCustomizationsConstants
{
	static const float ItemWidth = 125.0f;
}

const TArray<FText>& FIKRigGenericDetailCustomization::GetButtonLabels()
{
	static const TArray<FText> ButtonLabels
	{
		LOCTEXT("CurrentTransform", "Current"),
		LOCTEXT("ReferenceTransform", "Reference")
	};
	return ButtonLabels;
}

const TArray<EIKRigTransformType::Type>& FIKRigGenericDetailCustomization::GetTransformTypes()
{
	static const TArray<EIKRigTransformType::Type> TransformTypes
	{
		EIKRigTransformType::Current,
		EIKRigTransformType::Reference
	};
	return TransformTypes;
}

template<typename ClassToCustomize>
TArray<ClassToCustomize*> GetCustomizedObjects(const TArray<TWeakObjectPtr<UObject>>& InObjectsBeingCustomized)
{
	TArray<ClassToCustomize*> CustomizedObjects;
	CustomizedObjects.Reserve(InObjectsBeingCustomized.Num());

	auto IsOfCustomType = [](TWeakObjectPtr<UObject> InObject) { return InObject.Get() && InObject->IsA<ClassToCustomize>(); };
	auto CastAsCustomType = [](TWeakObjectPtr<UObject> InObject) { return Cast<ClassToCustomize>(InObject); };
	Algo::TransformIf(InObjectsBeingCustomized, CustomizedObjects, IsOfCustomType, CastAsCustomType);
	
	return CustomizedObjects;
}

void FIKRigGenericDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized = DetailBuilder.GetSelectedObjects();
	ObjectsBeingCustomized.RemoveAll( [](const TWeakObjectPtr<UObject>& InObject) { return !InObject.IsValid(); } );

	// if no selected objects
	if (ObjectsBeingCustomized.IsEmpty())
	{
		return;
	}
	
	// make sure all types are of the same class
	const UClass* DetailsClass = ObjectsBeingCustomized[0]->GetClass();
	const int32 NumObjects = ObjectsBeingCustomized.Num();
	for (int32 Index = 1; Index < NumObjects; Index++)
	{
		if (ObjectsBeingCustomized[Index]->GetClass() != DetailsClass)
		{
			// multiple different things - fallback to default details panel behavior
			return;
		}
	}

	// assuming the classes are all the same
	if (ObjectsBeingCustomized[0]->IsA<UIKRigBoneDetails>())
	{
		return CustomizeDetailsForClass<UIKRigBoneDetails>(DetailBuilder, ObjectsBeingCustomized);
	}

	if (ObjectsBeingCustomized[0]->IsA<UIKRigEffectorGoal>())
	{
		return CustomizeDetailsForClass<UIKRigEffectorGoal>(DetailBuilder, ObjectsBeingCustomized);
	}
}

template <>
void FIKRigGenericDetailCustomization::CustomizeDetailsForClass<UIKRigBoneDetails>(
	IDetailLayoutBuilder& DetailBuilder,
	const TArray<TWeakObjectPtr<UObject>>& InObjectsBeingCustomized)
{
	const TArray<UIKRigBoneDetails*> Bones = GetCustomizedObjects<UIKRigBoneDetails>(InObjectsBeingCustomized);
	if (Bones.IsEmpty())
	{
		return;
	}

	const TArray<FText>& ButtonLabels = GetButtonLabels();
	const TArray<EIKRigTransformType::Type>& TransformTypes = GetTransformTypes();
	
	static const TArray<FText> ButtonTooltips
	{
		LOCTEXT("CurrentBoneTransformTooltip", "The current transform of the bone"),
		LOCTEXT("ReferenceBoneTransformTooltip", "The reference transform of the bone")
	};
	
	static const TAttribute<TArray<EIKRigTransformType::Type>> VisibleTransforms =
		TArray<EIKRigTransformType::Type>({EIKRigTransformType::Current});

	const TArray<TSharedRef<IPropertyHandle>> Properties
	{
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRigBoneDetails, CurrentTransform)),
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRigBoneDetails, ReferenceTransform))
	};

	for (TSharedRef<IPropertyHandle> Property : Properties)
	{
		DetailBuilder.HideProperty(Property);
	}

	TSharedPtr<SSegmentedControl<EIKRigTransformType::Type>> TransformChoiceWidget =
		SSegmentedControl<EIKRigTransformType::Type>::Create(
			TransformTypes,
			ButtonLabels,
			ButtonTooltips,
			VisibleTransforms
		);

	DetailBuilder.EditCategory(TEXT("Selection")).SetSortOrder(1);

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("Transforms"));
	CategoryBuilder.SetSortOrder(2);
	CategoryBuilder.AddCustomRow(FText::FromString(TEXT("TransformType")))
	.ValueContent()
	.MinDesiredWidth(375.f)
	.MaxDesiredWidth(375.f)
	.HAlign(HAlign_Left)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			TransformChoiceWidget.ToSharedRef()
		]
	];

	SAdvancedTransformInputBox<FTransform>::FArguments TransformWidgetArgs = SAdvancedTransformInputBox<FTransform>::FArguments()
	.IsEnabled(false)
	.DisplayRelativeWorld(true)
	.DisplayScaleLock(false)
	.AllowEditRotationRepresentation(true)
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.UseQuaternionForRotation(true);
	
	for(int32 PropertyIndex=0;PropertyIndex<Properties.Num();PropertyIndex++)
	{
		const EIKRigTransformType::Type TransformType = (EIKRigTransformType::Type)PropertyIndex; 

		// get/set relative
		TransformWidgetArgs.OnGetIsComponentRelative_Lambda( [Bones, TransformType](ESlateTransformComponent::Type InComponent)
		{
			return Bones.ContainsByPredicate( [&](const UIKRigBoneDetails* Bone)
			{
				return Bone->IsComponentRelative(InComponent, TransformType);
			} );
		})
		.OnIsComponentRelativeChanged_Lambda( [Bones, TransformType](ESlateTransformComponent::Type InComponent, bool bIsRelative)
		{
			for (UIKRigBoneDetails* Bone: Bones)
			{
				Bone->OnComponentRelativeChanged(InComponent, bIsRelative, TransformType);
			}
		} );

		// get bones transforms
		TransformWidgetArgs.Transform_Lambda([Bones, TransformType]()
		{
			TOptional<FTransform> Value = Bones[0]->GetTransform(TransformType);
			if (Value)
			{
				for (int32 Index = 1; Index < Bones.Num(); Index++)
				{
					const TOptional<FTransform> CurrentValue = Bones[Index]->GetTransform(TransformType);
					if (CurrentValue)
					{
						if (!Value->Equals( *CurrentValue))
						{
							return TOptional<FTransform>();
						}
					}
				}
			}
			return Value;
		});
		
		// copy/paste bones transforms
		TransformWidgetArgs.OnCopyToClipboard_UObject(Bones[0], &UIKRigBoneDetails::OnCopyToClipboard, TransformType);
		// FIXME we must connect paste capabilities here otherwise copy is not enabled...
		// see DetailSingleItemRow::OnContextMenuOpening for more explanations
		TransformWidgetArgs.OnPasteFromClipboard_UObject(Bones[0], &UIKRigBoneDetails::OnPasteFromClipboard, TransformType);

		TransformWidgetArgs.Visibility_Lambda([TransformChoiceWidget, TransformType]() -> EVisibility
		{
			return TransformChoiceWidget->HasValue(TransformType) ? EVisibility::Visible : EVisibility::Collapsed;
		});

		SAdvancedTransformInputBox<FTransform>::ConstructGroupedTransformRows(
			CategoryBuilder, 
			ButtonLabels[PropertyIndex], 
			ButtonTooltips[PropertyIndex], 
			TransformWidgetArgs);
	}
}

template <>
void FIKRigGenericDetailCustomization::CustomizeDetailsForClass<UIKRigEffectorGoal>(
	IDetailLayoutBuilder& DetailBuilder,
	const TArray<TWeakObjectPtr<UObject>>& InObjectsBeingCustomized)
{
	const TArray<UIKRigEffectorGoal*> Goals = GetCustomizedObjects<UIKRigEffectorGoal>(InObjectsBeingCustomized);
	if (Goals.IsEmpty())
	{
		return;
	}

	const TArray<FText>& ButtonLabels = GetButtonLabels();
	const TArray<EIKRigTransformType::Type>& TransformTypes = GetTransformTypes();	

	static const TArray<FText> ButtonTooltips
	{
		LOCTEXT("CurrentGoalTransformTooltip", "The current transform of the goal"),
		LOCTEXT("ReferenceGoalTransformTooltip", "The reference transform of the goal")
	};

	static TAttribute<TArray<EIKRigTransformType::Type>> VisibleTransforms =
		TArray<EIKRigTransformType::Type>({EIKRigTransformType::Current});

	const TArray<TSharedRef<IPropertyHandle>> Properties
	{
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRigEffectorGoal, CurrentTransform)),
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UIKRigEffectorGoal, InitialTransform))
	};

	for (TSharedRef<IPropertyHandle> Property : Properties)
	{
		DetailBuilder.HideProperty(Property);
	}

	TSharedPtr<SSegmentedControl<EIKRigTransformType::Type>> TransformChoiceWidget =
		SSegmentedControl<EIKRigTransformType::Type>::Create(
			TransformTypes,
			ButtonLabels,
			ButtonTooltips,
			VisibleTransforms
		);

	DetailBuilder.EditCategory(TEXT("Goal Settings")).SetSortOrder(1);
	DetailBuilder.EditCategory(TEXT("Viewport Goal Settings")).SetSortOrder(3);

	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(TEXT("Transforms"));
	CategoryBuilder.SetSortOrder(2);
	CategoryBuilder.AddCustomRow(FText::FromString(TEXT("TransformType")))
	.ValueContent()
	.MinDesiredWidth(375.f)
	.MaxDesiredWidth(375.f)
	.HAlign(HAlign_Left)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			TransformChoiceWidget.ToSharedRef()
		]
	];

	SAdvancedTransformInputBox<FTransform>::FArguments TransformWidgetArgs = SAdvancedTransformInputBox<FTransform>::FArguments()
	.IsEnabled(true)
	.DisplayRelativeWorld(false)
	.DisplayScaleLock(true)
	.AllowEditRotationRepresentation(true)
	.Font(IDetailLayoutBuilder::GetDetailFont())
	.UseQuaternionForRotation(true);
	
	for(int32 PropertyIndex=0;PropertyIndex<Properties.Num();PropertyIndex++)
	{
		const EIKRigTransformType::Type TransformType = (EIKRigTransformType::Type)PropertyIndex;

		if(PropertyIndex > 0)
		{
			TransformWidgetArgs
			.IsEnabled(false)
			.DisplayScaleLock(false);
		}

		// get goals transforms
		TransformWidgetArgs.OnGetNumericValue_Lambda( [Goals, TransformType](ESlateTransformComponent::Type Component,
														ESlateRotationRepresentation::Type Representation,
														ESlateTransformSubComponent::Type SubComponent)
		{
			TOptional<FVector::FReal> Value = Goals[0]->GetNumericValue(Component, Representation, SubComponent, TransformType);
			if (Value)
			{
				for (int32 Index = 1; Index < Goals.Num(); Index++)
				{
					const TOptional<FVector::FReal> CurrentValue = Goals[Index]->GetNumericValue(Component, Representation, SubComponent, TransformType);
					if (CurrentValue)
					{
						if (!FMath::IsNearlyEqual(*Value, *CurrentValue))
						{
							return TOptional<FVector::FReal>();
						}
					}
				}
			}
			return Value;
		} );

		// set goals transforms
		TransformWidgetArgs.OnNumericValueChanged_Lambda([this, Goals, TransformType]( ESlateTransformComponent::Type InComponent,
															  ESlateRotationRepresentation::Type InRepresentation,
															  ESlateTransformSubComponent::Type InSubComponent,
															  FVector::FReal InValue)
		{
			FTransform CurrentTransform, UpdatedTransform;
			for (UIKRigEffectorGoal* Goal: Goals)
			{
				Tie(CurrentTransform, UpdatedTransform) =
					Goal->PrepareNumericValueChanged(InComponent, InRepresentation, InSubComponent, InValue, TransformType);

				// don't do anything if the transform has not been changed
				if (!UpdatedTransform.Equals(CurrentTransform))
				{
					// prepare transaction if needed
					if (!ValueChangedTransaction.IsValid())
					{
						ValueChangedTransaction = MakeShareable(new FScopedTransaction(LOCTEXT("ChangeNumericValue", "Change Numeric Value")));
					}
					
					Goal->SetTransform(UpdatedTransform, TransformType);
				}
			}
		})
		.OnNumericValueCommitted_Lambda([this, Goals, TransformType](ESlateTransformComponent::Type InComponent,
											ESlateRotationRepresentation::Type InRepresentation,
											ESlateTransformSubComponent::Type InSubComponent,
											FVector::FReal InValue,
											ETextCommit::Type InCommitType)
		{
			if (!ValueChangedTransaction.IsValid())
			{
				ValueChangedTransaction = MakeShareable(new FScopedTransaction(LOCTEXT("ChangeNumericValue", "Change Numeric Value")));
			}
			
			FTransform CurrentTransform, UpdatedTransform;
			for (UIKRigEffectorGoal* Goal: Goals)
			{
				Tie(CurrentTransform, UpdatedTransform) = Goal->PrepareNumericValueChanged( InComponent, InRepresentation, InSubComponent, InValue, TransformType);

				// don't do anything if the transform has not been changed
				if (!UpdatedTransform.Equals(CurrentTransform))
				{
					Goal->SetTransform(UpdatedTransform, TransformType);
				}
			}
			
			ValueChangedTransaction.Reset();
		});

		// copy/paste values
		TransformWidgetArgs.OnCopyToClipboard_UObject(Goals[0], &UIKRigEffectorGoal::OnCopyToClipboard, TransformType)
		.OnPasteFromClipboard_Lambda([Goals, TransformType](ESlateTransformComponent::Type InComponent)
		{
			FScopedTransaction Transaction(LOCTEXT("PasteTransform", "Paste Transform"));
			for (UIKRigEffectorGoal* Goal: Goals)
			{
				Goal->OnPasteFromClipboard(InComponent, TransformType);
			};
		});

		// reset to default
		const TSharedPtr<IPropertyHandle> PropertyHandle = Properties[PropertyIndex];
		TransformWidgetArgs.DiffersFromDefault_Lambda([Goals,PropertyHandle](ESlateTransformComponent::Type InComponent)
		{
			return Goals.ContainsByPredicate( [&](const UIKRigEffectorGoal* Goal)
			{
				return Goal->TransformDiffersFromDefault(InComponent, PropertyHandle);
			} );
		} )
		.OnResetToDefault_Lambda([Goals,PropertyHandle](ESlateTransformComponent::Type InComponent)
		{
			for (UIKRigEffectorGoal* Goal: Goals)
			{
				Goal->ResetTransformToDefault(InComponent, PropertyHandle);
			}
		} );

		// visibility
		TransformWidgetArgs.Visibility_Lambda([TransformChoiceWidget, TransformType]() -> EVisibility
		{
			return TransformChoiceWidget->HasValue(TransformType) ? EVisibility::Visible : EVisibility::Collapsed;
		});

		SAdvancedTransformInputBox<FTransform>::ConstructGroupedTransformRows(
			CategoryBuilder, 
			ButtonLabels[PropertyIndex], 
			ButtonTooltips[PropertyIndex], 
			TransformWidgetArgs);
	}
}

#undef LOCTEXT_NAMESPACE
