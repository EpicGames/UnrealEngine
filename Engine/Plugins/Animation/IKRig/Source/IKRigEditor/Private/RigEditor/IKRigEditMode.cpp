// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigEditMode.h"

#include "EditorViewportClient.h"
#include "AssetEditorModeManager.h"
#include "IPersonaPreviewScene.h"
#include "SkeletalDebugRendering.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "RigEditor/IKRigAnimInstance.h"
#include "RigEditor/IKRigHitProxies.h"
#include "RigEditor/IKRigToolkit.h"
#include "IKRigDebugRendering.h"
#include "Preferences/PersonaOptions.h"

#define LOCTEXT_NAMESPACE "IKRetargeterEditMode"

FName FIKRigEditMode::ModeName("IKRigAssetEditMode");

FIKRigEditMode::FIKRigEditMode()
{
}

bool FIKRigEditMode::GetCameraTarget(FSphere& OutTarget) const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	
	// collect selected points
	TArray<FVector> TargetPoints;
	
	// get selected goal locations
	TArray<FName> OutGoalNames;
	Controller->GetSelectedGoalNames(OutGoalNames);
	for (const FName& GoalName : OutGoalNames)
	{			
		TargetPoints.Add(Controller->AssetController->GetGoal(GoalName)->CurrentTransform.GetLocation());
	}

	// get selected bone locations
	TArray<FName> OutBoneNames;
	Controller->GetSelectedBoneNames(OutBoneNames);
	if (!OutBoneNames.IsEmpty())
	{
		const FIKRigSkeleton* CurrentIKRigSkeleton = Controller->GetCurrentIKRigSkeleton();
		const FIKRigSkeleton& Skeleton = CurrentIKRigSkeleton ? *CurrentIKRigSkeleton : Controller->AssetController->GetIKRigSkeleton();
		for (const FName& BoneName : OutBoneNames)
		{
			const int32 BoneIndex = Skeleton.GetBoneIndexFromName(BoneName);
			if (BoneIndex != INDEX_NONE)
			{
				TArray<int32> Children;
				Skeleton.GetChildIndices(BoneIndex, Children);
				for (const int32 ChildIndex: Children)
				{
					TargetPoints.Add(Skeleton.CurrentPoseGlobal[ChildIndex].GetLocation());
				}
				TargetPoints.Add(Skeleton.CurrentPoseGlobal[BoneIndex].GetLocation());
			}
		}		
	}

	// target union of selected goals and bones
	if (!TargetPoints.IsEmpty())
	{
		// create a sphere that contains all the target points
		OutTarget = FSphere(&TargetPoints[0], TargetPoints.Num());
		return true;
	}

	// target skeletal mesh
	if (Controller->SkelMeshComponent)
	{
		OutTarget = Controller->SkelMeshComponent->Bounds.GetSphere();
		return true;
	}
	
	return false;
}

IPersonaPreviewScene& FIKRigEditMode::GetAnimPreviewScene() const
{
	return *static_cast<IPersonaPreviewScene*>(static_cast<FAssetEditorModeManager*>(Owner)->GetPreviewScene());
}

void FIKRigEditMode::GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const
{
	// todo: provide warnings from solvers
}

void FIKRigEditMode::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	FEdMode::Render(View, Viewport, PDI);
	RenderGoals(PDI);
	RenderBones(PDI);
}

void FIKRigEditMode::RenderGoals(FPrimitiveDrawInterface* PDI)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	const UIKRigController* AssetController = Controller->AssetController;
	const UIKRigDefinition* IKRigAsset = AssetController->GetAsset();
	if (!IKRigAsset->DrawGoals)
	{
		return;
	}

	TArray<UIKRigEffectorGoal*> Goals = AssetController->GetAllGoals();
	for (const UIKRigEffectorGoal* Goal : Goals)
	{
		const bool bIsSelected = Controller->IsGoalSelected(Goal->GoalName);
		const float Size = IKRigAsset->GoalSize * Goal->SizeMultiplier;
		const float Thickness = IKRigAsset->GoalThickness * Goal->ThicknessMultiplier;
		const FLinearColor Color = bIsSelected ? FLinearColor::Green : FLinearColor::Yellow;
		PDI->SetHitProxy(new HIKRigEditorGoalProxy(Goal->GoalName));
		IKRigDebugRendering::DrawWireCube(PDI, Goal->CurrentTransform, Color, Size, Thickness);
		PDI->SetHitProxy(NULL);
	}
}

void FIKRigEditMode::RenderBones(FPrimitiveDrawInterface* PDI)
{
	// get the controller
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	// is the ik rig available?
	const FIKRigProcessor* ProcessorPtr = Controller->GetIKRigProcessor();
	if (!ProcessorPtr)
	{
		return;
	}

	// is the ik rig initialized?
	const FIKRigProcessor& CurrentProcessor = *ProcessorPtr;
	if (!CurrentProcessor.IsInitialized())
	{
		return;
	}
	
	const UDebugSkelMeshComponent* MeshComponent = Controller->SkelMeshComponent;
	const FTransform ComponentTransform = MeshComponent->GetComponentTransform();
	const FReferenceSkeleton& RefSkeleton = MeshComponent->GetReferenceSkeleton();
	const int32 NumBones = RefSkeleton.GetNum();

	// get world transforms of bones
	TArray<FBoneIndexType> RequiredBones;
	RequiredBones.AddUninitialized(NumBones);
	TArray<FTransform> WorldTransforms;
	WorldTransforms.AddUninitialized(NumBones);
	for (int32 Index=0; Index<NumBones; ++Index)
	{
		RequiredBones[Index] = static_cast<FBoneIndexType>(Index);
		WorldTransforms[Index] = MeshComponent->GetBoneTransform(Index, ComponentTransform);
	}
	
	UPersonaOptions* ConfigOption = UPersonaOptions::StaticClass()->GetDefaultObject<UPersonaOptions>();
	
	FSkelDebugDrawConfig DrawConfig;
	DrawConfig.BoneDrawMode = (EBoneDrawMode::Type)ConfigOption->DefaultBoneDrawSelection;
	DrawConfig.BoneDrawSize = Controller->AssetController->GetAsset()->BoneSize;
	DrawConfig.bAddHitProxy = true;
	DrawConfig.bForceDraw = false;
	DrawConfig.DefaultBoneColor = GetMutableDefault<UPersonaOptions>()->DefaultBoneColor;
	DrawConfig.AffectedBoneColor = GetMutableDefault<UPersonaOptions>()->AffectedBoneColor;
	DrawConfig.SelectedBoneColor = GetMutableDefault<UPersonaOptions>()->SelectedBoneColor;
	DrawConfig.ParentOfSelectedBoneColor = GetMutableDefault<UPersonaOptions>()->ParentOfSelectedBoneColor;

	TArray<TRefCountPtr<HHitProxy>> HitProxies;
	HitProxies.Reserve(NumBones);
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		HitProxies.Add(new HIKRigEditorBoneProxy(RefSkeleton.GetBoneName(Index)));
	}
	
	// get selected bones
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBoneItems;
	Controller->GetSelectedBones(SelectedBoneItems);
	TArray<int32> SelectedBones;
	for (TSharedPtr<FIKRigTreeElement> SelectedBone: SelectedBoneItems)
	{
		const int32 BoneIndex = RefSkeleton.FindBoneIndex(SelectedBone->BoneName);
		SelectedBones.Add(BoneIndex);
	}

	// get bone colors
	TArray<FLinearColor> BoneColors;
	const bool bUseBoneColors = GetDefault<UPersonaOptions>()->bShowBoneColors;
	GetBoneColors(Controller.Get(), CurrentProcessor, RefSkeleton, bUseBoneColors, BoneColors);

	SkeletalDebugRendering::DrawBones(
		PDI,
		ComponentTransform.GetLocation(),
		RequiredBones,
		RefSkeleton,
		WorldTransforms,
		SelectedBones,
		BoneColors,
		HitProxies,
		DrawConfig
	);
}

void FIKRigEditMode::GetBoneColors(
	FIKRigEditorController* Controller,
	const FIKRigProcessor& Processor,
	const FReferenceSkeleton& RefSkeleton,
	bool bUseMultiColorAsDefaultColor,
	TArray<FLinearColor>& OutBoneColors) const
{
	const FLinearColor DefaultColor = GetMutableDefault<UPersonaOptions>()->DefaultBoneColor;
	const FLinearColor HighlightedColor = FLinearColor::Blue;
	const FLinearColor ErrorColor = FLinearColor::Red;

	// set all to default color
	if (bUseMultiColorAsDefaultColor)
	{
		SkeletalDebugRendering::FillWithMultiColors(OutBoneColors, RefSkeleton.GetNum());
	}
	else
	{
		OutBoneColors.Init(DefaultColor, RefSkeleton.GetNum());		
	}
	
	// highlight bones of the last selected UI element (could be solver or retarget chain) 
	switch (Controller->GetLastSelectedType())
	{
		case EIKRigSelectionType::Hierarchy:
		// handled by SkeletalDebugRendering::DrawBones()
		break;
		
		case EIKRigSelectionType::SolverStack:
		{
			// get selected solver
			TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
			Controller->GetSelectedSolvers(SelectedSolvers);
			if (SelectedSolvers.IsEmpty())
			{
				return;
			}

			// record which bones in the skeleton are affected by this solver
			const FIKRigSkeleton& Skeleton = Processor.GetSkeleton();
			const UIKRigController* AssetController = Controller->AssetController;
			if(const FIKRigSolverBase* SelectedSolver = AssetController->GetSolverAtIndex(SelectedSolvers[0].Get()->IndexInStack))
			{
				for (int32 BoneIndex=0; BoneIndex < Skeleton.BoneNames.Num(); ++BoneIndex)
				{
					const FName& BoneName = Skeleton.BoneNames[BoneIndex];
					if (SelectedSolver->IsBoneAffectedBySolver(BoneName, Skeleton))
					{
						OutBoneColors[BoneIndex] = HighlightedColor;
					}
				}
			}
		}
		break;
		
		case EIKRigSelectionType::RetargetChains:
		{
			const TArray<FName> SelectedChainNames = Controller->GetSelectedChains();
			if (SelectedChainNames.IsEmpty())
			{
				return;
			}

			const FIKRigSkeleton& Skeleton = Processor.GetSkeleton();
			for (const FName& SelectedChainName : SelectedChainNames)
			{
				TSet<int32> OutChainBoneIndices;
				const bool bIsValid = Controller->AssetController->ValidateChain(SelectedChainName, &Skeleton, OutChainBoneIndices);
				for (const int32 IndexOfBoneInChain : OutChainBoneIndices)
				{
					OutBoneColors[IndexOfBoneInChain] = bIsValid ? HighlightedColor : ErrorColor;
				}
			}
		}
		break;
		
		default:
			checkNoEntry();
	}
}

bool FIKRigEditMode::AllowWidgetMove()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	return Controller->GetNumSelectedGoals() > 0;
}

bool FIKRigEditMode::ShouldDrawWidget() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	return Controller->GetNumSelectedGoals() > 0;
}

bool FIKRigEditMode::UsesTransformWidget() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	return Controller->GetNumSelectedGoals() > 0;
}

bool FIKRigEditMode::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	return Controller->GetNumSelectedGoals() > 0;
}

FVector FIKRigEditMode::GetWidgetLocation() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return FVector::ZeroVector;
	}
	
	TArray<FName> OutGoalNames;
	Controller->GetSelectedGoalNames(OutGoalNames);
	if (OutGoalNames.IsEmpty())
	{
		return FVector::ZeroVector; 
	}

	return Controller->AssetController->GetGoalCurrentTransform(OutGoalNames.Last()).GetTranslation();
}

bool FIKRigEditMode::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	// check for selections
	if (Click.GetKey() == EKeys::LeftMouseButton)
	{
		// draw bones based on the hierarchy when clicking in viewport
		EditorController.Pin()->SetLastSelectedType(EIKRigSelectionType::Hierarchy);
		
		// clicking in empty space clears selection and shows empty details
		if (!HitProxy)
		{
			Controller->ClearSelection();
			return false;
		}
		
		// selected goal
		if (HitProxy->IsA(HIKRigEditorGoalProxy::StaticGetType()))
		{
			HIKRigEditorGoalProxy* GoalProxy = static_cast<HIKRigEditorGoalProxy*>(HitProxy);
			const bool bReplaceSelection = !(InViewportClient->IsCtrlPressed() || InViewportClient->IsShiftPressed());
			Controller->HandleGoalSelectedInViewport(GoalProxy->GoalName, bReplaceSelection);
			return true;
		}
		// selected bone
		if (HitProxy->IsA(HIKRigEditorBoneProxy::StaticGetType()))
		{
			HIKRigEditorBoneProxy* BoneProxy = static_cast<HIKRigEditorBoneProxy*>(HitProxy);
			const bool bReplaceSelection = !(InViewportClient->IsCtrlPressed() || InViewportClient->IsShiftPressed());
			Controller->HandleBoneSelectedInViewport(BoneProxy->BoneName, bReplaceSelection);
			return true;
		}
	}
	
	return false;
}

bool FIKRigEditMode::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	const EAxisList::Type CurrentAxis = InViewportClient->GetCurrentWidgetAxis();
	if (CurrentAxis == EAxisList::None)
	{
		return false; // not manipulating a required axis
	}

	return HandleBeginTransform();
}

bool FIKRigEditMode::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	return HandleEndTransform();
}

bool FIKRigEditMode::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	if (!Controller->bManipulatingGoals)
	{
		return false; // not handled
	}

	TArray<FName> SelectedGoalNames;
	Controller->GetSelectedGoalNames(SelectedGoalNames);
	const UIKRigController* AssetController = Controller->AssetController;

	// translate goals
	if(InViewportClient->GetWidgetMode() == UE::Widget::WM_Translate)
	{
		for (const FName& GoalName : SelectedGoalNames)
		{
			FTransform CurrentTransform = AssetController->GetGoalCurrentTransform(GoalName);
			CurrentTransform.AddToTranslation(InDrag);
			AssetController->SetGoalCurrentTransform(GoalName, CurrentTransform);
		}
	}

	// rotate goals
	if(InViewportClient->GetWidgetMode() == UE::Widget::WM_Rotate)
	{
		for (const FName& GoalName : SelectedGoalNames)
		{
			FTransform CurrentTransform = AssetController->GetGoalCurrentTransform(GoalName);
			FQuat CurrentRotation = CurrentTransform.GetRotation();
			CurrentRotation = (InRot.Quaternion() * CurrentRotation);
			CurrentTransform.SetRotation(CurrentRotation);
			AssetController->SetGoalCurrentTransform(GoalName, CurrentTransform);
		}
	}
	
	return true;
}

bool FIKRigEditMode::BeginTransform(const FGizmoState& InState)
{
	return HandleBeginTransform();
}

bool FIKRigEditMode::EndTransform(const FGizmoState& InState)
{
	return HandleEndTransform();
}

bool FIKRigEditMode::HandleBeginTransform() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	TArray<FName> SelectedGoalNames;
	Controller->GetSelectedGoalNames(SelectedGoalNames);
	if (SelectedGoalNames.IsEmpty())
	{
		return false; // no goals selected to manipulate
	}

	for (const FName& SelectedGoal : SelectedGoalNames)
	{
		(void)Controller->AssetController->ModifyGoal(SelectedGoal);
	}
	Controller->bManipulatingGoals = true;

	return true;
}

bool FIKRigEditMode::HandleEndTransform() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	if (!Controller->bManipulatingGoals)
	{
		return false; // not handled
	}

	Controller->bManipulatingGoals = false;
	return true;
}


bool FIKRigEditMode::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	
	TArray<FName> SelectedGoalNames;
	Controller->GetSelectedGoalNames(SelectedGoalNames);
	if (SelectedGoalNames.IsEmpty())
	{
		return false; // nothing selected to manipulate
	}

	if (const UIKRigEffectorGoal* Goal = Controller->AssetController->GetGoal(SelectedGoalNames[0]))
	{
		InMatrix = Goal->CurrentTransform.ToMatrixNoScale().RemoveTranslation();
		return true;
	}

	return false;
}

bool FIKRigEditMode::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

bool FIKRigEditMode::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	if (FEdMode::InputKey(ViewportClient, Viewport, Key, Event))
	{
		return false;
	}
		
	if (Key == EKeys::Delete || Key == EKeys::BackSpace)
	{
		if (EditorController.IsValid())
		{
			EditorController.Pin()->HandleDeleteSelectedElements();
			return true;
		}
	}

	return false;
}

void FIKRigEditMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);
}

void FIKRigEditMode::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
	FEdMode::DrawHUD(ViewportClient, Viewport, View, Canvas);
}

#undef LOCTEXT_NAMESPACE
