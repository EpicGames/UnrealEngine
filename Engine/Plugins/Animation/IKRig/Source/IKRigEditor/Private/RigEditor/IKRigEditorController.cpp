// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigEditorController.h"

#include "RigEditor/IKRigAutoCharacterizer.h"
#include "RigEditor/IKRigController.h"
#include "RigEditor/SIKRigHierarchy.h"
#include "RigEditor/SIKRigSolverStack.h"
#include "RigEditor/IKRigAnimInstance.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/SIKRigAssetBrowser.h"
#include "RigEditor/SIKRigOutputLog.h"
#include "Rig/Solvers/IKRigFullBodyIK.h"
#include "Widgets/Input/SComboBox.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "IPersonaToolkit.h"
#include "SKismetInspector.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Dialog/SCustomDialog.h"
#include "ScopedTransaction.h"
#include "Framework/Notifications/NotificationManager.h"
#include "RigEditor/IKRigAutoFBIK.h"
#include "RigEditor/IKRigDetailCustomizations.h"
#include "RigEditor/IKRigStructViewer.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigEditorController)

#if WITH_EDITOR

#include "HAL/PlatformApplicationMisc.h"

#endif

#define LOCTEXT_NAMESPACE "IKRigEditorController"

bool UIKRigBoneDetails::CurrentTransformRelative[3] = {true, true, true};
bool UIKRigBoneDetails::ReferenceTransformRelative[3] = {true, true, true};

TOptional<FTransform> UIKRigBoneDetails::GetTransform(EIKRigTransformType::Type TransformType) const
{
	if(!AnimInstancePtr.IsValid() || !AssetPtr.IsValid())
	{
		return TOptional<FTransform>();
	}
	
	FTransform LocalTransform = FTransform::Identity;
	FTransform GlobalTransform = FTransform::Identity;
	const bool* IsRelative = nullptr;

	const FIKRigSkeleton& Skeleton = AssetPtr->GetSkeleton();
	const int32 BoneIndex = Skeleton.GetBoneIndexFromName(SelectedBone);
	if(BoneIndex == INDEX_NONE)
	{
		return TOptional<FTransform>();
	}

	switch(TransformType)
	{
		case EIKRigTransformType::Current:
		{
			IsRelative = CurrentTransformRelative;
			
			USkeletalMeshComponent* SkeletalMeshComponent = AnimInstancePtr->GetSkelMeshComponent();
			const bool IsSkelMeshValid = SkeletalMeshComponent != nullptr &&
										SkeletalMeshComponent->GetSkeletalMeshAsset() != nullptr;
			if (IsSkelMeshValid)
			{
				GlobalTransform = SkeletalMeshComponent->GetBoneTransform(BoneIndex);
				const TArray<FTransform>& LocalTransforms = SkeletalMeshComponent->GetBoneSpaceTransforms();
				LocalTransform = LocalTransforms.IsValidIndex(BoneIndex) ? LocalTransforms[BoneIndex] : FTransform::Identity;
			}
			else
			{
				GlobalTransform = Skeleton.CurrentPoseGlobal[BoneIndex];
				LocalTransform = Skeleton.CurrentPoseLocal[BoneIndex];
			}
			break;
		}
		case EIKRigTransformType::Reference:
		{
			IsRelative = ReferenceTransformRelative;
			GlobalTransform = Skeleton.RefPoseGlobal[BoneIndex];
			LocalTransform = GlobalTransform;
			const int32 ParentBoneIndex = Skeleton.ParentIndices[BoneIndex];
			if(ParentBoneIndex != INDEX_NONE)
			{
				const FTransform ParentTransform = Skeleton.RefPoseGlobal[ParentBoneIndex];;
				LocalTransform = GlobalTransform.GetRelativeTransform(ParentTransform);
			}
			break;
		}
	}
	checkSlow(IsRelative);

	FTransform Transform = LocalTransform;
	if(!IsRelative[0]) Transform.SetLocation(GlobalTransform.GetLocation());
	if(!IsRelative[1]) Transform.SetRotation(GlobalTransform.GetRotation());
	if(!IsRelative[2]) Transform.SetScale3D(GlobalTransform.GetScale3D());
	return Transform;
}

bool UIKRigBoneDetails::IsComponentRelative(
	ESlateTransformComponent::Type Component,
	EIKRigTransformType::Type TransformType) const
{
	switch(TransformType)
	{
		case EIKRigTransformType::Current:
		{
			return CurrentTransformRelative[(int32)Component]; 
		}
		case EIKRigTransformType::Reference:
		{
			return ReferenceTransformRelative[(int32)Component]; 
		}
	}
	return true;
}

void UIKRigBoneDetails::OnComponentRelativeChanged(
	ESlateTransformComponent::Type Component,
	bool bIsRelative,
	EIKRigTransformType::Type TransformType)
{
	switch(TransformType)
	{
		case EIKRigTransformType::Current:
		{
			CurrentTransformRelative[(int32)Component] = bIsRelative;
			break; 
		}
		case EIKRigTransformType::Reference:
		{
			ReferenceTransformRelative[(int32)Component] = bIsRelative;
			break; 
		}
	}
}

#if WITH_EDITOR

void UIKRigBoneDetails::OnCopyToClipboard(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType) const
{
	TOptional<FTransform> Optional = GetTransform(TransformType);
	if(!Optional.IsSet())
	{
		return;
	}

	const FTransform Xfo = Optional.GetValue();
	
	FString Content;
	switch(Component)
	{
	case ESlateTransformComponent::Location:
		{
			GetContentFromData(Xfo.GetLocation(), Content);
			break;
		}
	case ESlateTransformComponent::Rotation:
		{
			GetContentFromData(Xfo.Rotator(), Content);
			break;
		}
	case ESlateTransformComponent::Scale:
		{
			GetContentFromData(Xfo.GetScale3D(), Content);
			break;
		}
	case ESlateTransformComponent::Max:
	default:
		{
			GetContentFromData(Xfo, Content);
			TBaseStructure<FTransform>::Get()->ExportText(Content, &Xfo, &Xfo, nullptr, PPF_None, nullptr);
			break;
		}
	}

	if(!Content.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*Content);
	}
}

void UIKRigBoneDetails::OnPasteFromClipboard(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType)
{
	// paste is not supported yet.
}

#endif

void FRetargetChainAnalyzer::AssignBestGuessName(FBoneChain& Chain, const FIKRigSkeleton& IKRigSkeleton)
{
	// a map of common names used in bone hierarchies, used to derive a best guess name for a retarget chain
	static TMap<FString, TArray<FString>> ChainToBonesMap;
	ChainToBonesMap.Add(FString("Head"), {"head"});
	ChainToBonesMap.Add(FString("Neck"), {"neck"});
	ChainToBonesMap.Add(FString("Leg"), {"leg", "hip", "thigh", "calf", "knee", "foot", "ankle", "toe"});
	ChainToBonesMap.Add(FString("Arm"), {"arm", "clavicle", "shoulder", "elbow", "wrist", "hand"});
	ChainToBonesMap.Add(FString("Spine"), {"spine"});
	
	ChainToBonesMap.Add(FString("Jaw"),{"jaw"});
	ChainToBonesMap.Add(FString("Tail"),{"tail", "tentacle"});
	
	ChainToBonesMap.Add(FString("Thumb"), {"thumb"});
	ChainToBonesMap.Add(FString("Index"), {"index"});
	ChainToBonesMap.Add(FString("Middle"), {"middle"});
	ChainToBonesMap.Add(FString("Ring"), {"ring"});
	ChainToBonesMap.Add(FString("Pinky"), {"pinky"});

	ChainToBonesMap.Add(FString("Root"), {"root"});

	TArray<int32> BonesInChainIndices;
	const bool bIsChainValid = IKRigSkeleton.ValidateChainAndGetBones(Chain, BonesInChainIndices);
	if (!bIsChainValid)
	{
		Chain.ChainName = GetDefaultChainName();
		return;
	}

	// initialize map of chain names with initial scores of 0 for each chain
	TArray<FString> ChainNames;
	ChainToBonesMap.GetKeys(ChainNames);
	TMap<FString, float> ChainScores;
	for (const FString& ChainName : ChainNames)
	{
		ChainScores.Add(*ChainName, 0.f);
	}

	// run text filter on the predefined bone names and record score for each predefined chain name
	for (const int32 ChainBoneIndex : BonesInChainIndices)
	{
		const FName& ChainBoneName = IKRigSkeleton.GetBoneNameFromIndex(ChainBoneIndex);
		const FString ChainBoneNameStr = ChainBoneName.ToString().ToLower();
		for (const FString& ChainName : ChainNames)
		{
			for (const FString& BoneNameToTry : ChainToBonesMap[ChainName])
			{
				TextFilter->SetFilterText(FText::FromString(BoneNameToTry));
				if (TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(ChainBoneNameStr)))
				{
					++ChainScores[ChainName];	
				}
			}
		}
	}

	// find the chain name with the highest score
	float HighestScore = 0.f;
	FString RecommendedChainName = GetDefaultChainName().ToString();
	for (const FString& ChainName : ChainNames)
	{
		if (ChainScores[ChainName] > HighestScore)
		{
			HighestScore = ChainScores[ChainName];
			RecommendedChainName = ChainName;
		}
	}

	// now determine "sidedness" of the chain, and add a suffix if its on the left or right side
	const EChainSide ChainSide = GetSideOfChain(BonesInChainIndices, IKRigSkeleton);
	switch (ChainSide)
	{
	case EChainSide::Left:
		RecommendedChainName = "Left" + RecommendedChainName;
		break;
	case EChainSide::Right:
		RecommendedChainName = "Right" + RecommendedChainName;
		break;
	default:
		break;
	}
	
	Chain.ChainName = FName(RecommendedChainName);
}

FName FRetargetChainAnalyzer::GetDefaultChainName()
{
	static FText NewChainText = LOCTEXT("NewRetargetChainLabel", "NewRetargetChain");
	return FName(*NewChainText.ToString());
}

EChainSide FRetargetChainAnalyzer::GetSideOfChain(
	const TArray<int32>& BoneIndices,
	const FIKRigSkeleton& IKRigSkeleton) const
{
	// check if bones are predominantly named "Left" or "Right"
	auto DoMajorityBonesContainText = [this, &BoneIndices, &IKRigSkeleton](const FString& StrToTest) -> bool
	{
		int32 Score = 0;
		for (const int32 BoneIndex : BoneIndices)
		{
			const FString BoneName = IKRigSkeleton.GetBoneNameFromIndex(BoneIndex).ToString().ToLower();
			if (BoneName.Contains(StrToTest))
			{
				++Score;
			}
		}
		
		return Score > (BoneIndices.Num() / 2);
	};
	
	if (DoMajorityBonesContainText("Right"))
	{
		return EChainSide::Right;
	}
	if (DoMajorityBonesContainText("Left"))
	{
		return EChainSide::Left;
	}

	//
	// bones don't have left/right prefix/suffix, so lets fallback on using a spatial test...
	//
	
	// determine "sidedness" of the chain based on the location of the bones (left, right or center of YZ plane)
	double AverageXPositionOfChain = 0.f;
	for (const int32 BoneIndex : BoneIndices)
	{
		AverageXPositionOfChain += IKRigSkeleton.RefPoseGlobal[BoneIndex].GetLocation().X;
	}
	AverageXPositionOfChain /= static_cast<double>(BoneIndices.Num());

	constexpr double CenterThresholdDistance = 1.0f;
	if (FMath::Abs(AverageXPositionOfChain) < CenterThresholdDistance)
	{
		return EChainSide::Center;
	}

	return AverageXPositionOfChain > 0 ? EChainSide::Left :  EChainSide::Right;
}

void FIKRigEditorController::Initialize(TSharedPtr<FIKRigEditorToolkit> Toolkit, const UIKRigDefinition* IKRigAsset)
{
	EditorToolkit = Toolkit;
	AssetController = UIKRigController::GetController(IKRigAsset);
	BoneDetails.Reset(); // populated on demand in ShowDetailsForBones()
	
	// register callback to be informed when rig asset is modified by editor
	if (!AssetController->OnIKRigNeedsInitialized().IsBoundToObject(this))
	{
		ReinitializeDelegateHandle = AssetController->OnIKRigNeedsInitialized().AddSP(this, &FIKRigEditorController::HandleIKRigNeedsInitialized);
	}
}

void FIKRigEditorController::Close() const
{
	AssetController->OnIKRigNeedsInitialized().Remove(ReinitializeDelegateHandle);
}

FIKRigProcessor* FIKRigEditorController::GetIKRigProcessor() const
{
	if (AnimInstance)
	{
		return AnimInstance->GetCurrentlyRunningProcessor();
	}

	return nullptr;
}

const FIKRigSkeleton* FIKRigEditorController::GetCurrentIKRigSkeleton() const
{
	if (const FIKRigProcessor* Processor = GetIKRigProcessor())
	{
		if (Processor->IsInitialized())
		{
			return &Processor->GetSkeleton();	
		}
	}

	return nullptr;
}

void FIKRigEditorController::HandleIKRigNeedsInitialized(UIKRigDefinition* ModifiedIKRig) const
{
	if (ModifiedIKRig != AssetController->GetAsset())
	{
		return;
	}

	ClearOutputLog();

	// currently running processor needs reinit
	AnimInstance->SetProcessorNeedsInitialized();
	
	// in case the skeletal mesh was swapped out, we need to ensure the preview scene is up-to-date
	USkeletalMesh* NewMesh = AssetController->GetSkeletalMesh();
	const TSharedRef<IPersonaPreviewScene> PreviewScene = EditorToolkit.Pin()->GetPersonaToolkit()->GetPreviewScene();
	const bool bMeshWasSwapped = PreviewScene->GetPreviewMesh() != NewMesh;
	SkelMeshComponent->SetSkeletalMesh(NewMesh);
	if (bMeshWasSwapped)
	{
		PreviewScene->SetPreviewMeshComponent(SkelMeshComponent);
		PreviewScene->SetPreviewMesh(NewMesh);
		AssetController->ResetInitialGoalTransforms();
	}

	// re-initializes the anim instances running in the viewport
	if (AnimInstance)
	{
		// record what anim was playing so we can restore it after reinit
		UAnimationAsset* AnimThatWasPlaying = AnimInstance->GetCurrentAsset();
		const float TimeWhenReset = AnimInstance->GetCurrentTime();
		const bool bWasPlaying = AnimInstance->IsPlaying();
		
		SkelMeshComponent->PreviewInstance = AnimInstance;
		AnimInstance->InitializeAnimation();
		SkelMeshComponent->EnablePreview(true, nullptr);

		// restore previously playing asset
		if (AnimThatWasPlaying)
		{
			AnimInstance->SetAnimationAsset(AnimThatWasPlaying);
			AnimInstance->SetPlaying(bWasPlaying);
			AnimInstance->SetPosition(TimeWhenReset);
		}
	}

	// update all bone details objects so they can pull on the current data
	for (const TObjectPtr<UIKRigBoneDetails>& BoneDetail : BoneDetails)
	{
		BoneDetail->AnimInstancePtr = AnimInstance;
		BoneDetail->AssetPtr = ModifiedIKRig;
	}

	// refresh all views
	{
		if (SolverStackView.IsValid())
		{
			SolverStackView->RefreshStackView();
		}

		if (SkeletonView.IsValid())
		{
			SkeletonView->RefreshTreeView();
		}

		if (DetailsView.IsValid())
		{
			DetailsView->ForceRefresh();
		}

		if (RetargetingView.IsValid())
		{
			RetargetingView->RefreshView();
		}
		
		if (bMeshWasSwapped && AssetBrowserView.IsValid())
		{
			AssetBrowserView.Get()->RefreshView(); // refresh the asset browser to ensure it shows compatible sequences
		}
	}
}

void FIKRigEditorController::HandleDeleteSelectedElements()
{
	FScopedTransaction Transaction(LOCTEXT("DeleteSelectedIKRigElements_Label", "Delete Selected IK Rig Elements"));
	FScopedReinitializeIKRig Reinitialize(AssetController);
	
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = SkeletonView->GetSelectedItems();
	for (const TSharedPtr<FIKRigTreeElement>& SelectedItem : SelectedItems)
	{
		switch(SelectedItem->ElementType)
		{
		case EIKRigTreeElementType::GOAL:
			AssetController->RemoveGoal(SelectedItem->GoalName);
			break;
		case EIKRigTreeElementType::GOAL_SETTINGS:
			AssetController->DisconnectGoalFromSolver(SelectedItem->EffectorGoalName, SelectedItem->EffectorIndex);
			break;
		case EIKRigTreeElementType::BONE_SETTINGS:
			AssetController->RemoveBoneSetting(SelectedItem->BoneSettingBoneName, SelectedItem->BoneSettingsSolverIndex);
			break;
		default:
			break; // can't delete anything else
		}
	}
}

void FIKRigEditorController::Reset() const
{
	SkelMeshComponent->ShowReferencePose(true);
	AssetController->ResetGoalTransforms();
}

void FIKRigEditorController::RefreshTreeView() const
{
	if (SkeletonView.IsValid())
	{
		SkeletonView->RefreshTreeView();
	}
}

void FIKRigEditorController::ClearOutputLog() const
{
	if (OutputLogView.IsValid())
	{
		OutputLogView.Get()->ClearLog();
		if (const FIKRigProcessor* Processor = GetIKRigProcessor())
		{
			Processor->Log.Clear();
		}
	}
}

void FIKRigEditorController::AutoGenerateRetargetChains() const
{
	USkeletalMesh* Mesh = AssetController->GetSkeletalMesh();
	if (!Mesh)
	{
		FNotificationInfo Info(LOCTEXT("NoMeshToAutoCharacterize", "No mesh to auto-characterize. Operation cancelled."));
		Info.ExpireDuration = 3.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
		return;
	}

	// auto generate a retarget definition
	FAutoCharacterizeResults Results;
	AssetController->AutoGenerateRetargetDefinition(Results);
	
	// notify user of the results of the auto characterization
	if (Results.bUsedTemplate)
	{
		// actually apply the auto-generated retarget definition
		// TODO move this outside the condition once procedural retarget definitions are supported
		AssetController->SetRetargetDefinition(Results.AutoRetargetDefinition.RetargetDefinition);
		
		// notify user of which skeleton was detected
		const FText ScoreAsText = FText::AsPercent(Results.BestPercentageOfTemplateScore);
		const FText NameAsText = FText::FromName(Results.BestTemplateName);
		const FText Message = FText::Format(
			LOCTEXT("AutoCharacterizeResults", "Using {0} template. Skeletal structure matches with {1} accuracy."),
			NameAsText, ScoreAsText);
		FNotificationInfo Info(Message);
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);

		// log all the differences between the template the skeleton
		if (FIKRigProcessor* Processor = GetIKRigProcessor())
		{
			FIKRigLogger& Log = Processor->Log;

			// missing bones
			for (const FName& MissingBone : Results.MissingBones)
			{
				Log.LogWarning(FText::Format(
				LOCTEXT("MissingTemplateBone", "{0} was not found in this skeleton, but is used by {1}."),
				FText::FromName(MissingBone),
				FText::FromName(Results.BestTemplateName)));
			}

			// different parents
			for (const FName& MissingParent : Results.BonesWithMissingParent)
			{
				Log.LogWarning(FText::Format(
				LOCTEXT("DifferentParentTemplateBone", "{0} has a different parent in the template: {1}."),
				FText::FromName(MissingParent),
				FText::FromName(Results.BestTemplateName)));
			}

			// inform user if any chains were expanded beyond what the template defined
			for (TPair<FName, int32> ExpandedChain : Results.ExpandedChains)
			{
				if (ExpandedChain.Value > 0)
				{
					Log.LogWarning(FText::Format(
						LOCTEXT("ExpandedChain", "The '{0}' chain was expanded beyond the template by {1} bones."),
						FText::FromName(ExpandedChain.Key),
						FText::AsNumber(ExpandedChain.Value)));
				}
			}
		}
	}
	else
	{
		// notify user that no skeleton template was used
		// TODO change this message once procedurally generated retarget definitions are provided
		FNotificationInfo Info(LOCTEXT("MissingTemplateCharacterizeSkipped", "No matching skeletal template found. Characterization skipped."));
		Info.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
	}
}

void FIKRigEditorController::AutoGenerateFBIK() const
{
	FAutoFBIKResults Results;
	AssetController->AutoGenerateFBIK(Results);

	switch (Results.Outcome)
	{
	case EAutoFBIKResult::AllOk:
		{
			FNotificationInfo Info(LOCTEXT("AutoIKSuccess", "Auto FBIK Successfully Setup."));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
		break;
	case EAutoFBIKResult::MissingMesh:
		{
			FNotificationInfo Info(LOCTEXT("AutoIKNoMesh", "No mesh to create IK for. Auto FBIK skipped."));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
		break;
	case EAutoFBIKResult::MissingChains:
		{
			FNotificationInfo Info(LOCTEXT("AutoIKMissingChains", "Missing retarget chains. Auto FBIK did not find all expected chains. See output."));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
		break;
	case EAutoFBIKResult::UnknownSkeletonType:
		{
			FNotificationInfo Info(LOCTEXT("AutoIKUnknownSkeleton", "Unknown skeleton type. Auto FBIK skipped."));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
		break;
	case EAutoFBIKResult::MissingPelvisBone:
		{
			FNotificationInfo Info(LOCTEXT("AutoIKMissingRootBone", "Auto FBIK setup, but root bone was missing. Please assign a root bone manually."));
			Info.ExpireDuration = 3.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
		}
		break;
	default:
		checkNoEntry();
	}
}

void FIKRigEditorController::AddNewGoals(const TArray<FName>& GoalNames, const TArray<FName>& BoneNames)
{
	check(GoalNames.Num() == BoneNames.Num());

	FScopedTransaction Transaction(LOCTEXT("AddNewGoals_Label", "Add New Goals"));
	FScopedReinitializeIKRig Reinitialize(AssetController);
	AssetController->GetAsset()->Modify();

	// add a default solver if there isn't one already
	const bool bCancelled = PromptToAddDefaultSolver();
	if (bCancelled)
	{
		return;
	}

	// create goals
	FName LastCreatedGoalName = NAME_None;
	for (int32 I=0; I<GoalNames.Num(); ++I)
	{
		const FName& GoalName = GoalNames[I];
		const FName& BoneName = BoneNames[I];

		// create a new goal
		const FName NewGoalName = AssetController->AddNewGoal(GoalName, BoneName);
		if (NewGoalName == NAME_None)
		{
			continue; // already exists
		}

		// ask user if they want to assign this goal to a chain (if there is one on this bone)
		PromptToAssignGoalToChain(NewGoalName);

		// connect the new goal to selected solvers
		TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
		GetSelectedSolvers(SelectedSolvers);
		for (TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
		{
			AssetController->ConnectGoalToSolver(NewGoalName, Solver.Get()->IndexInStack);
		}

		LastCreatedGoalName = GoalName;
	}
	
	// show last created goal in details view
	if (LastCreatedGoalName != NAME_None)
	{
		ShowDetailsForGoals({LastCreatedGoalName});
	}
}

void FIKRigEditorController::ClearSelection() const
{
	if (SkeletonView.IsValid())
	{
		SkeletonView->TreeView->ClearSelection();	
	}
	
	ShowAssetDetails();
}

void FIKRigEditorController::HandleGoalSelectedInViewport(const FName& GoalName, bool bReplace) const
{
	if (SkeletonView.IsValid())
	{
		SkeletonView->AddSelectedItemFromViewport(GoalName, EIKRigTreeElementType::GOAL, bReplace);
		ShowDetailsForElements(SkeletonView->GetSelectedItems());
		return;
	}

	ShowDetailsForGoals({GoalName});
}

void FIKRigEditorController::HandleBoneSelectedInViewport(const FName& BoneName, bool bReplace) const
{
	if (SkeletonView.IsValid())
	{
		SkeletonView->AddSelectedItemFromViewport(BoneName, EIKRigTreeElementType::BONE, bReplace);
		ShowDetailsForElements(SkeletonView->GetSelectedItems());
		return;
	}
	
	ShowDetailsForBone(BoneName);
}

void FIKRigEditorController::GetSelectedSolvers(TArray<TSharedPtr<FSolverStackElement>>& OutSelectedSolvers) const
{
	if (SolverStackView.IsValid())
	{
		OutSelectedSolvers.Reset();
		OutSelectedSolvers.Append(SolverStackView->ListView->GetSelectedItems());
	}
}

int32 FIKRigEditorController::GetSelectedSolverIndex()
{
	if (!SolverStackView.IsValid())
	{
		return INDEX_NONE;
	}
	
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers = SolverStackView->ListView->GetSelectedItems();
	if (SelectedSolvers.IsEmpty())
	{
		return INDEX_NONE;
	}

	return SelectedSolvers[0]->IndexInStack;
}

void FIKRigEditorController::GetSelectedGoalNames(TArray<FName>& OutGoalNames) const
{
	if (!SkeletonView.IsValid())
	{
		return;
	}

	SkeletonView->GetSelectedGoalNames(OutGoalNames);
}

int32 FIKRigEditorController::GetNumSelectedGoals() const
{
	if (!SkeletonView.IsValid())
	{
		return 0;
	}

	return SkeletonView->GetNumSelectedGoals();
}

void FIKRigEditorController::GetSelectedBoneNames(TArray<FName>& OutBoneNames) const
{
	if (!SkeletonView.IsValid())
	{
		return;
	}

	SkeletonView->GetSelectedBoneNames(OutBoneNames);
}

void FIKRigEditorController::GetSelectedBones(TArray<TSharedPtr<FIKRigTreeElement>>& OutBoneItems) const
{
	if (!SkeletonView.IsValid())
	{
		return;
	}

	SkeletonView->GetSelectedBones(OutBoneItems);
}

bool FIKRigEditorController::IsGoalSelected(const FName& GoalName) const
{
	if (!SkeletonView.IsValid())
	{
		return false;
	}

	return SkeletonView->IsGoalSelected(GoalName);
}

TArray<FName> FIKRigEditorController::GetSelectedChains() const
{
	if (!RetargetingView.IsValid())
	{
		return TArray<FName>();
	}

	return RetargetingView->GetSelectedChains();
}

bool FIKRigEditorController::DoesSkeletonHaveSelectedItems() const
{
	if (!SkeletonView.IsValid())
	{
		return false;
	}
	return SkeletonView->HasSelectedItems();
}

void FIKRigEditorController::GetChainsSelectedInSkeletonView(TArray<FBoneChain>& InOutChains)
{
	if (!SkeletonView.IsValid())
	{
		return;
	}

	SkeletonView->GetSelectedBoneChains(InOutChains);
}

void FIKRigEditorController::CreateNewRetargetChains()
{
	// get selected chains from hierarchy view
	TArray<FBoneChain> SelectedBoneChains;
	if (SkeletonView.IsValid())
	{
		SkeletonView->GetSelectedBoneChains(SelectedBoneChains);
	}
	
	FScopedTransaction Transaction(LOCTEXT("AddMultipleRetargetChains_Label", "Add Retarget Chain(s)"));
	FScopedReinitializeIKRig Reinitialize(AssetController);
	
	if (!SelectedBoneChains.IsEmpty())
	{
		// create a chain for each selected chain in hierarchy
		for (FBoneChain& BoneChain : SelectedBoneChains)
		{
			ChainAnalyzer.AssignBestGuessName(BoneChain, AssetController->GetIKRigSkeleton());
			PromptToAddNewRetargetChain(BoneChain);
		}
	}
	else
	{
		// create an empty chain
		FBoneChain Chain(FRetargetChainAnalyzer::GetDefaultChainName(), NAME_None, NAME_None, NAME_None);
		PromptToAddNewRetargetChain(Chain);
	}
}

bool FIKRigEditorController::PromptToAddDefaultSolver() const
{
	if (AssetController->GetNumSolvers() > 0)
	{
		return false;
	}

	TArray<TSharedPtr<FIKRigSolverTypeAndName>> SolverTypes;
	for (TObjectIterator<UStruct> StructIt; StructIt; ++StructIt)
	{
		UScriptStruct* Struct = Cast<UScriptStruct>(*StructIt);
		if (!Struct)
		{
			continue;
		}
		
		if (!Struct->IsNative())
		{
			continue;
		}

		if (!Struct->IsChildOf(FIKRigSolverBase::StaticStruct()))
		{
			continue;
		}

		if (Struct == FIKRigSolverBase::StaticStruct())
		{
			continue; // skip base class
		}

		//const UIKRigSolver* SolverCDO = CastChecked<UIKRigSolver>(Class->GetDefaultObject(false));
		TSharedPtr<FIKRigSolverTypeAndName> SolverType = MakeShared<FIKRigSolverTypeAndName>();
		SolverType->NiceName = FText::FromString(Struct->GetName());
		SolverType->SolverType = Struct;
		SolverTypes.Add(SolverType);
	}

	// select "full body IK" by default
	TSharedPtr<FIKRigSolverTypeAndName> SelectedSolver = SolverTypes[0];
	for (TSharedPtr<FIKRigSolverTypeAndName>& SolverType :SolverTypes)
	{
		if (SolverType->SolverType == FIKRigFullBodyIKSolver::StaticStruct())
		{
			SelectedSolver = SolverType;
			break;
		}
	}
	
	const TSharedRef<SComboBox<TSharedPtr<FIKRigSolverTypeAndName>>> SolverOptionBox = SNew(SComboBox<TSharedPtr<FIKRigSolverTypeAndName>>)
	.OptionsSource(&SolverTypes)
	.OnGenerateWidget_Lambda([](TSharedPtr<FIKRigSolverTypeAndName> Item)
	{
		return SNew(STextBlock).Text(Item->NiceName);
	})
	.OnSelectionChanged_Lambda([&SelectedSolver](TSharedPtr<FIKRigSolverTypeAndName> Item, ESelectInfo::Type)
	{
		SelectedSolver = Item;
	})
	.Content()
	[
		SNew(STextBlock)
		.MinDesiredWidth(200)
		.Text_Lambda([&SelectedSolver]()
		{
			return SelectedSolver->NiceName;
		})
	];
	
	TSharedRef<SCustomDialog> AddSolverDialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("EditorController_IKRigFirstSolver", "Add Default Solver")))
		.Content()
		[
			SolverOptionBox
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("AddSolver", "Add Solver")),
			SCustomDialog::FButton(LOCTEXT("Skip", "Skip"))
	});

	// show window and get result
	const int32 Result = AddSolverDialog->ShowModal();
	const bool bWindowClosed = Result < 0;
	const bool bSkipped = Result == 1;
	
	if (bWindowClosed)
	{
		return true; // window closed
	}

	if (bSkipped)
	{
		return false; // user opted NOT to add a solver
	}

	// add a solver
	if (SelectedSolver->SolverType != nullptr && SolverStackView.IsValid())
	{
		AssetController->AddSolver(SelectedSolver->SolverType);
	}

	// must refresh the view so that subsequent goal operations see a selected solver to connect to
	SolverStackView->RefreshStackView();
	
	return false;
}

void FIKRigEditorController::ShowDetailsForBones(const TArray<FName>& BoneNames) const
{
	// grow the owned pool to match the number of bones to display
	while (BoneDetails.Num() < BoneNames.Num())
	{
		BoneDetails.Add(NewObject<UIKRigBoneDetails>());
	}

	TArray<UObject*> BoneDetailObjects;
	for (int32 i = 0; i < BoneNames.Num(); ++i)
	{
		BoneDetails[i]->SetBone(BoneNames[i]);
		BoneDetails[i]->AnimInstancePtr = AnimInstance;
		BoneDetails[i]->AssetPtr = AssetController->GetAsset();
		BoneDetailObjects.Add(BoneDetails[i]);
	}

	if (BoneDetailObjects.Num() == 1)
	{
		DetailsView->SetObject(BoneDetailObjects[0]);
	}
	else
	{
		DetailsView->SetObjects(BoneDetailObjects);
	}
}

void FIKRigEditorController::ShowDetailsForBoneSettings(const TArray<TSharedPtr<FIKRigTreeElement>>& InItems) const
{
	if (InItems.IsEmpty())
	{
		return;
	}
	
	const UIKRigDefinition* Asset = AssetController->GetAsset();
	
	auto GetTypeOfItem = [Asset](const TSharedPtr<FIKRigTreeElement>& InItem)->TSubclassOf<UIKRigStructWrapperBase>
	{
		const int32 SolverIndex = InItem->BoneSettingsSolverIndex;
		const UIKRigController* Controller = UIKRigController::GetController(Asset);
		FIKRigSolverBase* Solver = Controller->GetSolverAtIndex(SolverIndex);
		if (!Solver)
		{
			return nullptr;
		}
		const UScriptStruct* BoneSettingsType = Solver->GetBoneSettingsType();
		if (!BoneSettingsType)
		{
			return nullptr;
		}

		return UIKRigStructWrapperBase::GetStructWrapperFromMetaData(BoneSettingsType);
	};

	TSubclassOf<UIKRigStructWrapperBase> TypeToDisplay = GetTypeOfItem(InItems[0]);
	if (!ensureMsgf(TypeToDisplay.Get(), TEXT("Selected settings do not have a UI wrapper class. Cannot be displayed.")))
	{
		return;
	}
	
	// can only display homogenous array
	TArray<TSharedPtr<FIKRigTreeElement>> ItemsToDisplay = FilterByFirstItemType(InItems, GetTypeOfItem);
	TArray<UIKRigStructWrapperBase*> StructWrappers = AssetController->GetStructWrappers(ItemsToDisplay.Num(), TypeToDisplay);
	for (int32 Index = 0; Index < ItemsToDisplay.Num(); ++Index)
	{
		TSharedPtr<FIKRigTreeElement> Item = ItemsToDisplay[Index];
		const int32 SolverIndex = Item->BoneSettingsSolverIndex;
		const FName BoneName = Item->BoneSettingBoneName;
		
		auto MemoryProvider = [Asset, SolverIndex, BoneName]() -> uint8*
			{
				const UIKRigController* Controller = UIKRigController::GetController(Asset);
				if (FIKRigSolverBase* Solver = Controller->GetSolverAtIndex(SolverIndex))
				{
					return reinterpret_cast<uint8*>(Solver->GetBoneSettings(BoneName));
				}
		
				return nullptr;
			};

		FInstancedStruct* InstancedStruct = AssetController->GetSolverStructAtIndex(SolverIndex);
		const FIKRigSolverBase* Solver = InstancedStruct->GetPtr<FIKRigSolverBase>();

		FIKRigStructToView StructToView;
		StructToView.Type = Solver->GetBoneSettingsType();
		StructToView.MemoryProvider = MemoryProvider;
		StructToView.Owner = AssetController->GetAsset();
		StructToView.UniqueName = BoneName;
	
		StructWrappers[Index]->Initialize(StructToView, FName("Settings"));
	}
	
	ShowStructWrappersInDetailsView(StructWrappers);
}

void FIKRigEditorController::ShowDetailsForGoals(const TArray<FName>& InGoalNames) const
{
	TArray<UObject*> Goals;
	for (const FName& GoalName : InGoalNames)
	{
		if (UIKRigEffectorGoal* Goal = AssetController->GetGoal(GoalName))
		{
			Goals.Add(Goal);
		}
	}
	
	DetailsView->SetObjects(Goals);
}

void FIKRigEditorController::ShowDetailsForGoalSettings(const TArray<TSharedPtr<FIKRigTreeElement>>& InItems) const
{
	if (InItems.IsEmpty())
	{
		return;
	}
	
	const UIKRigDefinition* Asset = AssetController->GetAsset();
	
	auto GetTypeOfItem = [Asset](const TSharedPtr<FIKRigTreeElement>& InItem)->TSubclassOf<UIKRigStructWrapperBase>
	{
		const int32 SolverIndex = InItem->EffectorIndex;
		const UIKRigController* Controller = UIKRigController::GetController(Asset);
		FIKRigSolverBase* Solver = Controller->GetSolverAtIndex(SolverIndex);
		if (!Solver)
		{
			return nullptr;
		}
		const UScriptStruct* GoalSettingsType = Solver->GetGoalSettingsType();
		if (!GoalSettingsType)
		{
			return nullptr;
		}

		return UIKRigStructWrapperBase::GetStructWrapperFromMetaData(GoalSettingsType);
	};

	TSubclassOf<UIKRigStructWrapperBase> TypeToDisplay = GetTypeOfItem(InItems[0]);
	if (!ensureMsgf(TypeToDisplay.Get(), TEXT("Selected settings do not have a UI wrapper class. Cannot be displayed.")))
	{
		return;
	}
	
	// can only display homogenous array
	TArray<TSharedPtr<FIKRigTreeElement>> ItemsToDisplay = FilterByFirstItemType(InItems, GetTypeOfItem);
	TArray<UIKRigStructWrapperBase*> StructWrappers = AssetController->GetStructWrappers(ItemsToDisplay.Num(), TypeToDisplay);
	for (int32 Index = 0; Index < ItemsToDisplay.Num(); ++Index)
	{
		TSharedPtr<FIKRigTreeElement> Item = ItemsToDisplay[Index];
		const int32 SolverIndex = Item->EffectorIndex;
		const FName GoalName = Item->EffectorGoalName;
		
		auto MemoryProvider = [Asset, SolverIndex, GoalName]() -> uint8*
			{
				const UIKRigController* Controller = UIKRigController::GetController(Asset);
				if (FIKRigSolverBase* Solver = Controller->GetSolverAtIndex(SolverIndex))
				{
					return reinterpret_cast<uint8*>(Solver->GetGoalSettings(GoalName));
				}
		
				return nullptr;
			};

		FInstancedStruct* InstancedStruct = AssetController->GetSolverStructAtIndex(SolverIndex);
		const FIKRigSolverBase* Solver = InstancedStruct->GetPtr<FIKRigSolverBase>();

		FIKRigStructToView StructToView;
		StructToView.Type = Solver->GetGoalSettingsType();
		StructToView.MemoryProvider = MemoryProvider;
		StructToView.Owner = AssetController->GetAsset();
		StructToView.UniqueName = GoalName;
	
		StructWrappers[Index]->Initialize(StructToView, FName("Settings"));
	}
	
	ShowStructWrappersInDetailsView(StructWrappers);
}

void FIKRigEditorController::ShowDetailsForSolvers(const TArray<int32>& InSolverIndices) const
{
	if (InSolverIndices.IsEmpty())
	{
		return;
	}
	
	const UIKRigDefinition* Asset = AssetController->GetAsset();
	
	auto GetTypeOfItem = [Asset](const int32 InSolverIndex)->TSubclassOf<UIKRigStructWrapperBase>
	{
		const UIKRigController* Controller = UIKRigController::GetController(Asset);
		FIKRigSolverBase* Solver = Controller->GetSolverAtIndex(InSolverIndex);
		if (!Solver)
		{
			return nullptr;
		}
		const UScriptStruct* SettingsType = Solver->GetSolverSettingsType();
		if (!SettingsType)
		{
			return nullptr;
		}

		return UIKRigStructWrapperBase::GetStructWrapperFromMetaData(SettingsType);
	};

	TSubclassOf<UIKRigStructWrapperBase> TypeToDisplay = GetTypeOfItem(InSolverIndices[0]);
	if (!ensureMsgf(TypeToDisplay.Get(), TEXT("Selected settings do not have a UI wrapper class. Cannot be displayed.")))
	{
		return;
	}

	auto FilterByFirstType = [](
		const TArray<int32>& InIndices,
		TFunctionRef<TSubclassOf<UObject>(int32)> GetTypeOfItem) -> TArray<int32>
		{
			TArray<int32> Out;
			if (InIndices.IsEmpty())
			{
				return Out;
			}

			// filter by first item type
			TSubclassOf<UObject> RefType = GetTypeOfItem(InIndices[0]);

			// collect only items with the same type
			Out.Reserve(InIndices.Num());
			for (const int32 Index : InIndices)
			{
				const TSubclassOf<UObject> ItemType = GetTypeOfItem(Index);
				if (ItemType == RefType) 
				{
					Out.Add(Index);
				}
			}
	
			return Out;	
		};
	
	// can only display homogenous array
	TArray<int32> SolversToDisplay = FilterByFirstType(InSolverIndices, GetTypeOfItem);
	TArray<UIKRigStructWrapperBase*> StructWrappers = AssetController->GetStructWrappers(SolversToDisplay.Num(), TypeToDisplay);
	for (int32 Index = 0; Index < SolversToDisplay.Num(); ++Index)
	{
		const int32 SolverIndex = SolversToDisplay[Index];
		
		auto MemoryProvider = [Asset, SolverIndex]() -> uint8*
			{
				const UIKRigController* Controller = UIKRigController::GetController(Asset);
				if (FIKRigSolverBase* Solver = Controller->GetSolverAtIndex(SolverIndex))
				{
					return reinterpret_cast<uint8*>(Solver->GetSolverSettings());
				}
		
				return nullptr;
			};

		FInstancedStruct* InstancedStruct = AssetController->GetSolverStructAtIndex(SolverIndex);
		const FIKRigSolverBase* Solver = InstancedStruct->GetPtr<FIKRigSolverBase>();

		FIKRigStructToView StructToView;
		StructToView.Type = Solver->GetSolverSettingsType();
		StructToView.MemoryProvider = MemoryProvider;
		StructToView.Owner = AssetController->GetAsset();
		StructToView.UniqueName = FName(Solver->GetNiceName().ToString());
	
		StructWrappers[Index]->Initialize(StructToView, FName("Settings"));
	}
	
	ShowStructWrappersInDetailsView(StructWrappers);
}

void FIKRigEditorController::ShowAssetDetails() const
{
	DetailsView->SetObject(AssetController->GetAsset());
}

void FIKRigEditorController::ShowDetailsForElements(const TArray<TSharedPtr<FIKRigTreeElement>>& InItems) const
{
	if (InItems.IsEmpty())
	{
		ShowAssetDetails();
		return;
	}

	auto FilterSameTypeAsFirst = [](const TArray<TSharedPtr<FIKRigTreeElement>>& InItems) -> TArray<TSharedPtr<FIKRigTreeElement>>
		{
			TArray<TSharedPtr<FIKRigTreeElement>> Result;

			if (InItems.Num() == 0)
			{
				return Result;
			}

			const EIKRigTreeElementType FirstType = InItems[0]->ElementType;

			for (const TSharedPtr<FIKRigTreeElement>& Item : InItems)
			{
				if (Item->ElementType == FirstType)
				{
					Result.Add(Item);
				}
			}

			return Result;
		};

	// edit the last selected element (first in list) and all selected elements of that same type
	TArray<TSharedPtr<FIKRigTreeElement>> ToEdit = FilterSameTypeAsFirst(InItems);
	const TSharedPtr<FIKRigTreeElement>& LastSelected = InItems[0];
	switch (LastSelected->ElementType)
	{
	case EIKRigTreeElementType::BONE:
		{
			TArray<FName> BoneNames;
			for (const TSharedPtr<FIKRigTreeElement>& Item : ToEdit)
			{
				BoneNames.Add(Item->BoneName);
			}
			ShowDetailsForBones(BoneNames);
		}
		break;
		
	case EIKRigTreeElementType::GOAL:
		{
			TArray<FName> GoalsToEdit;
			for (const TSharedPtr<FIKRigTreeElement>& GoalToEdit : ToEdit)
			{
				GoalsToEdit.Add(GoalToEdit->GoalName);
			}
			ShowDetailsForGoals(GoalsToEdit);	
		}
		break;
		
	case EIKRigTreeElementType::GOAL_SETTINGS:
		ShowDetailsForGoalSettings(ToEdit);
		break;
		
	case EIKRigTreeElementType::BONE_SETTINGS:
		ShowDetailsForBoneSettings(ToEdit);
		break;
		
	default:
		ensure(false);
		break;
	}
}

void FIKRigEditorController::ShowStructWrappersInDetailsView(TArray<UIKRigStructWrapperBase*> InStructWrappers) const
{
	// convert to TArray<UObject*>
	TArray<UObject*> Viewers;
	for (UIKRigStructWrapperBase* StructWrapper : InStructWrappers)
	{
		Viewers.Add(StructWrapper);
	}
	
	constexpr bool bForceRefresh = true; // must forcibly refresh because struct wrapper is a recycled UObject
	DetailsView->SetObjects(Viewers, bForceRefresh);
}

void FIKRigEditorController::OnFinishedChangingDetails(const FPropertyChangedEvent& PropertyChangedEvent) const
{
	const bool bPreviewMeshChanged = PropertyChangedEvent.GetPropertyName() == UIKRigDefinition::GetPreviewMeshPropertyName();
	if (bPreviewMeshChanged)
	{
		USkeletalMesh* Mesh = AssetController->GetSkeletalMesh();
		AssetController->SetSkeletalMesh(Mesh);
	}
}

bool FIKRigEditorController::IsObjectInDetailsView(const UObject* Object) const
{
	if (!DetailsView.IsValid())
	{
		return false;
	}

	if (!Object)
	{
		return false;
	}
	
	TArray<TWeakObjectPtr<UObject>> SelectedObjects = DetailsView->GetSelectedObjects();
	for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
	{
		if (SelectedObject.Get() == Object)
		{
			return true;
		}
	}

	return false;
}

void FIKRigEditorController::SetDetailsView(const TSharedPtr<IDetailsView>& InDetailsView)
{
	DetailsView = InDetailsView;
	DetailsView->OnFinishedChangingProperties().AddSP(this, &FIKRigEditorController::OnFinishedChangingDetails);
	ShowAssetDetails();
}

void FIKRigEditorController::PromptToAssignGoalToChain(const FName NewGoalName) const
{
	const UIKRigEffectorGoal* NewGoal = AssetController->GetGoal(NewGoalName);
	check(NewGoal);
	const TArray<FBoneChain>& AllRetargetChains = AssetController->GetRetargetChains();
	FName ChainToAddGoalTo = NAME_None;
	for (const FBoneChain& Chain : AllRetargetChains)
	{
		if (Chain.EndBone == NewGoal->BoneName)
		{
			ChainToAddGoalTo = Chain.ChainName;
		}
	}

	if (ChainToAddGoalTo == NAME_None)
	{
		return;
	}

	TSharedRef<SCustomDialog> AddGoalToChainDialog = SNew(SCustomDialog)
		.Title(FText(LOCTEXT("AssignGoalToNewChainTitle", "Assign Existing Goal to Retarget Chain")))
		.Content()
		[
			SNew(STextBlock)
			.Text(FText::Format(LOCTEXT("AssignGoalToChainLabel", "Assign goal, {0} to retarget chain, {1}?"), FText::FromName(NewGoal->GoalName), FText::FromName(ChainToAddGoalTo)))
		]
		.Buttons({
			SCustomDialog::FButton(LOCTEXT("AssignGoal", "Assign Goal")),
			SCustomDialog::FButton(LOCTEXT("SkipGoal", "Skip Goal"))
	});

	if (AddGoalToChainDialog->ShowModal() != 0)
	{
		return; // cancel button pressed, or window closed
	}
	
	AssetController->SetRetargetChainGoal(ChainToAddGoalTo, NewGoal->GoalName);
}

FName FIKRigEditorController::PromptToAddNewRetargetChain(FBoneChain& BoneChain) const
{
	TArray<SCustomDialog::FButton> Buttons;
	Buttons.Add(SCustomDialog::FButton(LOCTEXT("AddChain", "Add Chain")));

	const bool bHasExistingGoal = BoneChain.IKGoalName != NAME_None;
	if (bHasExistingGoal)
	{
		Buttons.Add(SCustomDialog::FButton(LOCTEXT("AddChainUsingGoal", "Add Chain using Goal")));
	}
	else
	{
		Buttons.Add(SCustomDialog::FButton(LOCTEXT("AddChainAndGoal", "Add Chain and Goal")));
	}
	
	Buttons.Add(SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel")));

	// ask user if they want to add a goal to the end of this chain
	const TSharedRef<SCustomDialog> AddNewRetargetChainDialog =
		SNew(SCustomDialog)
		.Title(FText(LOCTEXT("AddNewChainTitleLabel", "Add New Retarget Chain")))
		.HAlignContent(HAlign_Center)
		.Content()
		[
			SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 1)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("ChainNameLabel", "Chain Name"))
					]

					+SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
						.MinDesiredWidth(200.0f)
						.OnTextChanged_Lambda([&BoneChain](const FText& InText)
						{
							BoneChain.ChainName = FName(InText.ToString());
						})
						.Text(FText::FromName(BoneChain.ChainName))
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 1)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("StartBoneLabel", "Start Bone"))
					]

					+SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
						.Text(FText::FromName(BoneChain.StartBone.BoneName))
						.IsReadOnly(true)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 1)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("EndBoneLabel", "End Bone"))
					]

					+SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
						.Text(FText::FromName(BoneChain.EndBone.BoneName))
						.IsReadOnly(true)
					]
				]

				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2, 1)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(STextBlock).Text(LOCTEXT("GoalLabel", "Goal"))
					]

					+SHorizontalBox::Slot()
					[
						SNew(SEditableTextBox)
						.Text(FText::FromName(BoneChain.IKGoalName))
						.IsReadOnly(true)
					]
				]
			]
			
		]
		.Buttons(Buttons);

	// show the dialog and handle user choice
	const int32 UserChoice = AddNewRetargetChainDialog->ShowModal();
	const bool bUserClosedWindow = UserChoice < 0;
	const bool bUserAddedChainNoGoal = UserChoice == 0;
	const bool bUserAddedChainWithGoal = UserChoice == 1;
	const bool bUserPressedCancel = UserChoice == 2;

	// cancel button pressed, or window closed ?
	if (bUserClosedWindow || bUserPressedCancel)
	{
		return NAME_None;  
	}

	// user opted not to add a goal, so remove it in case one was in the bone chain
	if (bUserAddedChainNoGoal)
	{
		BoneChain.IKGoalName = NAME_None;
	}

	// add the retarget chain
	const FName NewChainName = AssetController->AddRetargetChainInternal(BoneChain);
	
	// did user choose to assign a goal
	if (bUserAddedChainWithGoal)
	{
		FName GoalName;
		if (bHasExistingGoal)
		{
			// use the existing goal
			GoalName = BoneChain.IKGoalName;
		}
		else
		{
			// add a default solver if there isn't one already
			const bool bCancelled = PromptToAddDefaultSolver();
			if (bCancelled)
			{
				// user cancelled creating a goal
				return NewChainName;
			}
			
			// create new goal
			const FName NewGoalName = FName(FText::Format(LOCTEXT("GoalOnNewChainName", "{0}_Goal"), FText::FromName(BoneChain.ChainName)).ToString());
			GoalName = AssetController->AddNewGoal(NewGoalName, BoneChain.EndBone.BoneName);

			// connect the new goal to selected solvers
			TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
			GetSelectedSolvers(SelectedSolvers);
			for (TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
			{
				AssetController->ConnectGoalToSolver(GoalName, Solver.Get()->IndexInStack);
			}
		}
		
		// assign the existing goal to it
		AssetController->SetRetargetChainGoal(NewChainName, GoalName);
	}
	
	return NewChainName;
}

void FIKRigEditorController::PlayAnimationAsset(UAnimationAsset* AssetToPlay)
{
	if (AssetToPlay && AnimInstance)
	{
		AnimInstance->SetAnimationAsset(AssetToPlay);
	}
}

EIKRigSelectionType FIKRigEditorController::GetLastSelectedType() const
{
	return LastSelectedType;
}

void FIKRigEditorController::SetLastSelectedType(EIKRigSelectionType SelectionType)
{
	LastSelectedType = SelectionType;
}

TObjectPtr<UIKRigBoneDetails> FIKRigEditorController::CreateBoneDetails(const TSharedPtr<FIKRigTreeElement const>& InBoneItem) const
{
	// ensure that the element is related to a bone
	if (InBoneItem->ElementType != EIKRigTreeElementType::BONE)
	{
		return nullptr;
	}
	
	// create and store a new one
	UIKRigBoneDetails* NewBoneDetails = NewObject<UIKRigBoneDetails>(AssetController->GetAsset(), FName(InBoneItem->BoneName), RF_Standalone | RF_Transient );
	NewBoneDetails->SelectedBone = InBoneItem->BoneName;
	NewBoneDetails->AnimInstancePtr = AnimInstance;
	NewBoneDetails->AssetPtr = AssetController->GetAsset();
	
	return NewBoneDetails;
}

TArray<TSharedPtr<FIKRigTreeElement>> FIKRigEditorController::FilterByFirstItemType(
	const TArray<TSharedPtr<FIKRigTreeElement>>& InItems,
	TFunctionRef<TSubclassOf<UObject>(const TSharedPtr<FIKRigTreeElement>&)> GetTypeOfItem) const
{
	TArray<TSharedPtr<FIKRigTreeElement>> Out;
	if (InItems.IsEmpty())
	{
		return Out;
	}

	// filter by first item type
	TSubclassOf<UObject> RefType = GetTypeOfItem(InItems[0]);

	// collect only items with the same type
	Out.Reserve(InItems.Num());
	for (const TSharedPtr<FIKRigTreeElement>& Item : InItems)
	{
		const TSubclassOf<UObject> ItemType = GetTypeOfItem(Item);
		if (ItemType == RefType) 
		{
			Out.Add(Item);
		}
	}
	
	return Out;
}

#undef LOCTEXT_NAMESPACE

