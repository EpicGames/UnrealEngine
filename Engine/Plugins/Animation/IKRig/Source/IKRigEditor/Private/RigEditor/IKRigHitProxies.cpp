// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigHitProxies.h"
#include "GenericPlatform/ICursor.h"

IMPLEMENT_HIT_PROXY(HIKRigEditorBoneProxy,HHitProxy);
IMPLEMENT_HIT_PROXY(HIKRigEditorGoalProxy,HHitProxy);

HIKRigEditorBoneProxy::HIKRigEditorBoneProxy(const FName& InBoneName)
	: HHitProxy(HPP_World)
	, BoneName(InBoneName)
{
}

EMouseCursor::Type HIKRigEditorBoneProxy::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}

bool HIKRigEditorBoneProxy::AlwaysAllowsTranslucentPrimitives() const
{
	return true;
}

HIKRigEditorGoalProxy::HIKRigEditorGoalProxy(const FName& InGoalName)
	: HHitProxy(HPP_World)
	, GoalName(InGoalName)
{
}

EMouseCursor::Type HIKRigEditorGoalProxy::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}

bool HIKRigEditorGoalProxy::AlwaysAllowsTranslucentPrimitives() const
{
	return true;
}

