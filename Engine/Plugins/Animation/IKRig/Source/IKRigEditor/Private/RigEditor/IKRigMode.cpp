// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigMode.h"
#include "RigEditor/IKRigToolkit.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "PersonaModule.h"
#include "ISkeletonEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PersonaTabs.h"
#include "RigEditor/IKRigAssetBrowserTabSummoner.h"
#include "RigEditor/IKRigOutputLogTabSummoner.h"
#include "RigEditor/IKRigSkeletonTabSummoner.h"
#include "RigEditor/IKRigSolverStackTabSummoner.h"
#include "RigEditor/IKRigRetargetChainTabSummoner.h"

#define LOCTEXT_NAMESPACE "IKRigMode"

FIKRigMode::FIKRigMode(
	TSharedRef<FWorkflowCentricApplication> InHostingApp,  
	TSharedRef<IPersonaPreviewScene> InPreviewScene)
	: FApplicationMode(IKRigEditorModes::IKRigEditorMode)
{
	IKRigEditorPtr = StaticCastSharedRef<FIKRigEditorToolkit>(InHostingApp);
	TSharedRef<FIKRigEditorToolkit> IKRigEditor = StaticCastSharedRef<FIKRigEditorToolkit>(InHostingApp);

	FPersonaViewportArgs ViewportArgs(InPreviewScene);
	ViewportArgs.bAlwaysShowTransformToolbar = true;
	ViewportArgs.bShowStats = false;
	ViewportArgs.bShowTurnTable = false;
	ViewportArgs.ContextName = TEXT("IKRigEditor.Viewport");
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(IKRigEditor, &FIKRigEditorToolkit::HandleViewportCreated);

	// register Persona tabs
	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	TabFactories.RegisterFactory(PersonaModule.CreatePersonaViewportTabFactory(InHostingApp, ViewportArgs));
	TabFactories.RegisterFactory(PersonaModule.CreateDetailsTabFactory(InHostingApp, FOnDetailsCreated::CreateSP(&IKRigEditor.Get(), &FIKRigEditorToolkit::HandleDetailsCreated)));
	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InHostingApp, IKRigEditor->GetPersonaToolkit()->GetPreviewScene()));

	// register custom tabs
	TabFactories.RegisterFactory(MakeShared<FIKRigAssetBrowserTabSummoner>(IKRigEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRigSkeletonTabSummoner>(IKRigEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRigSolverStackTabSummoner>(IKRigEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRigRetargetChainTabSummoner>(IKRigEditor));
	TabFactories.RegisterFactory(MakeShared<FIKRigOutputLogTabSummoner>(IKRigEditor));

	// create tab layout
	TabLayout = FTabManager::NewLayout("Standalone_IKRigEditor_Layout_v1.127")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()
				->SetSizeCoefficient(0.9f)
				->SetOrientation(Orient_Horizontal)
				->Split
				(
				    FTabManager::NewSplitter()
				    ->SetSizeCoefficient(0.2f)
				    ->SetOrientation(Orient_Vertical)
				    ->Split
				    (
					    FTabManager::NewStack()
					    ->SetSizeCoefficient(0.6f)
					    ->AddTab(FIKRigSkeletonTabSummoner::TabID, ETabState::OpenedTab)
					)
					->Split
					(
					    FTabManager::NewStack()
					    ->SetSizeCoefficient(0.4f)
					    ->AddTab(FIKRigSolverStackTabSummoner::TabID, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.8f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
					FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->SetHideTabWell(true)
						->AddTab(FPersonaTabs::PreviewViewportID, ETabState::OpenedTab)
					)
					->Split
					(
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->AddTab(FIKRigOutputLogTabSummoner::TabID, ETabState::OpenedTab)
					)
				)
				->Split
				(
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.2f)
                    ->SetOrientation(Orient_Vertical)
                    ->Split
                    (
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->AddTab(FPersonaTabs::DetailsID, ETabState::OpenedTab)
						->AddTab(FPersonaTabs::AdvancedPreviewSceneSettingsID, ETabState::OpenedTab)
						->SetForegroundTab(FPersonaTabs::DetailsID)
                    )
                    ->Split
                    (
						FTabManager::NewStack()
						->SetSizeCoefficient(0.6f)
						->AddTab(FIKRigAssetBrowserTabSummoner::TabID, ETabState::OpenedTab)
						->AddTab(FIKRigRetargetChainTabSummoner::TabID, ETabState::OpenedTab)
						->SetForegroundTab(FIKRigAssetBrowserTabSummoner::TabID)
                    )
				)
			)
		);

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InHostingApp);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
}

void FIKRigMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FIKRigEditorToolkit> IKRigEditor = IKRigEditorPtr.Pin();
	IKRigEditor->RegisterTabSpawners(InTabManager.ToSharedRef());
	IKRigEditor->PushTabFactories(TabFactories);
}

#undef LOCTEXT_NAMESPACE
