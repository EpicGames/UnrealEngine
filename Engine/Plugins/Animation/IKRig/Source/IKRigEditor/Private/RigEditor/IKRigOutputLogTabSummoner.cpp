// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigEditor/IKRigOutputLogTabSummoner.h"

#include "IDocumentation.h"
#include "Rig/IKRigProcessor.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/SIKRigOutputLog.h"

#define LOCTEXT_NAMESPACE "IKRigOutputLogTabSummoner"

const FName FIKRigOutputLogTabSummoner::TabID(TEXT("IKRigOutputLog"));

FIKRigOutputLogTabSummoner::FIKRigOutputLogTabSummoner(
	const TSharedRef<FIKRigEditorToolkit>& InRigEditor)
	: FWorkflowTabFactory(TabID, InRigEditor),
	IKRigEditor(InRigEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRigOutputLogTabLabel", "IK Rig Output Log");
	TabIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.Tabs.CompilerResults");

	ViewMenuDescription = LOCTEXT("IKRigOutputLog_ViewMenu_Desc", "IK Rig Output Log");
	ViewMenuTooltip = LOCTEXT("IKRigOutputLog_ViewMenu_ToolTip", "Show the IK Rig Output Log Tab");
}

TSharedPtr<SToolTip> FIKRigOutputLogTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRigOutputLogTooltip", "View warnings and errors from this rig."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRigOutputLog_Window"));
}

TSharedRef<SWidget> FIKRigOutputLogTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	// at this point the processor's log may not be bound to the asset, so forcibly do it now
	const TSharedRef<FIKRigEditorController>& Controller = IKRigEditor.Pin()->GetController();
	FIKRigProcessor* Processor = Controller->GetIKRigProcessor();
	Processor->Log.SetLogTarget(Controller->AssetController->GetAsset());
	// now bind the log UI to the processor's log target
	const FName LogName = Processor->Log.GetLogTarget();
	TSharedRef<SIKRigOutputLog> LogView = SNew(SIKRigOutputLog, LogName);
	Controller->SetOutputLogView(LogView);
	return LogView;
}

#undef LOCTEXT_NAMESPACE 
