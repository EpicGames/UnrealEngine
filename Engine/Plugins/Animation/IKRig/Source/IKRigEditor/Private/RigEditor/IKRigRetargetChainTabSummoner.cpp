// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigRetargetChainTabSummoner.h"

#include "IDocumentation.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/SIKRigSolverStack.h"

#define LOCTEXT_NAMESPACE "RetargetChainTabSummoner"

const FName FIKRigRetargetChainTabSummoner::TabID(TEXT("RetargetingChains"));

FIKRigRetargetChainTabSummoner::FIKRigRetargetChainTabSummoner(const TSharedRef<FIKRigEditorToolkit>& InIKRigEditor)
	: FWorkflowTabFactory(TabID, InIKRigEditor)
	, IKRigEditor(InIKRigEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRigRetargetChainTabLabel", "IK Retargeting");
	TabIcon = FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(), "IKRig.IKRetargeting");

	ViewMenuDescription = LOCTEXT("IKRigRetargetChain_ViewMenu_Desc", "IK Rig Retarget Chains");
	ViewMenuTooltip = LOCTEXT("IKRigRetargetChain_ViewMenu_ToolTip", "Show the IK Rig Retarget Chains Tab");
}

TSharedPtr<SToolTip> FIKRigRetargetChainTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRigRetargetChainTooltip", "A list of labeled bone chains for retargeting animation between different skeletons. Used by the IK Retargeter asset."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRigRetargetChain_Window"));
}

TSharedRef<SWidget> FIKRigRetargetChainTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SIKRigRetargetChainList, IKRigEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
