// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigSkeletonCommands.h"

#define LOCTEXT_NAMESPACE "IKRigHierarchyCommands"

void FIKRigSkeletonCommands::RegisterCommands()
{
	UI_COMMAND(NewGoal, "New IK Goal", "Add new IK goal at the selected bone.", EUserInterfaceActionType::Button, FInputChord(EKeys::N, EModifierKey::Control));
	UI_COMMAND(DeleteElement, "Delete Goal or Effector", "Delete the selected IK Goal, Effector or Bone Settings.", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete), FInputChord(EKeys::BackSpace));
	UI_COMMAND(ConnectGoalToSolver, "Connect Goal to Selected Solver", "Connect this goal to the selected solver.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DisconnectGoalFromSolver, "Disconnect Goal from Selected Solver", "Disconnect this goal from the selected solver.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetStartBoneOnSolver, "Set Start Bone on Selected Solver", "Set the Start Bone on the selected solver.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetEndBoneOnSolver, "Set End Bone on Selected Solver", "Set End Bone on selected solver.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AddBoneSettings, "Add Settings to Selected Bone", "Apply settings to the selected bone for the selected solver (defined per-solver for limits, stiffness etc).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveBoneSettings, "Remove Settings on Selected Bone", "Remove all settings on the selected bone for the selected solver.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExcludeBone, "Exclude Selected Bone From Solve", "Ignore bone in all solvers.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(IncludeBone, "Include Selected Bone In Solve", "Include bone in all solvers.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(NewRetargetChain, "New Retarget Chain...", "Create a new retarget bone chain from the selected bones.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SetPelvis, "Set Pelvis", "Set the Pelvis bone used for retargeting.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ClearPelvis, "Clear Pelvis", "Clear the Pelvis bone used for retargeting.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RenameGoal, "Rename Goal", "Rename the selected goal.", EUserInterfaceActionType::Button, FInputChord(EKeys::F2));
	UI_COMMAND(RefreshHierarchy, "Refresh Hierarchy", "Reload bones from current skeletal mesh.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
