// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigSkeletonTabSummoner.h"

#include "IDocumentation.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/SIKRigHierarchy.h"

#define LOCTEXT_NAMESPACE "IKRigSkeletonTabSummoner"

const FName FIKRigSkeletonTabSummoner::TabID(TEXT("IKRigSkeleton"));

FIKRigSkeletonTabSummoner::FIKRigSkeletonTabSummoner(const TSharedRef<FIKRigEditorToolkit>& InIKRigEditor)
	: FWorkflowTabFactory(TabID, InIKRigEditor)
	, IKRigEditor(InIKRigEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRigSkeletonTabLabel", "Hierarchy");
	TabIcon = FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(), "IKRig.Hierarchy");

	ViewMenuDescription = LOCTEXT("IKRigSkeleton_ViewMenu_Desc", "Hierarchy");
	ViewMenuTooltip = LOCTEXT("IKRigSkeleton_ViewMenu_ToolTip", "Show the IK Rig Hierarchy Tab");
}

TSharedPtr<SToolTip> FIKRigSkeletonTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRigSkeletonTooltip", "The IK Rig Hierarchy tab lets you view the rig hierarchy."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRigSkeleton_Window"));
}

TSharedRef<SWidget> FIKRigSkeletonTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SIKRigHierarchy, IKRigEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
