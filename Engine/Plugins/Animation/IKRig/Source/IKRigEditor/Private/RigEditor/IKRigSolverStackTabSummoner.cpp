// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigSolverStackTabSummoner.h"

#include "IDocumentation.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/SIKRigSolverStack.h"

#define LOCTEXT_NAMESPACE "SolverStackTabSummoner"

const FName FIKRigSolverStackTabSummoner::TabID(TEXT("IKRigSolverStack"));

FIKRigSolverStackTabSummoner::FIKRigSolverStackTabSummoner(const TSharedRef<FIKRigEditorToolkit>& InIKRigEditor)
	: FWorkflowTabFactory(TabID, InIKRigEditor)
	, IKRigEditor(InIKRigEditor)
{
	bIsSingleton = true; // only allow a single instance of this tab
	
	TabLabel = LOCTEXT("IKRigSolverStackTabLabel", "Solver Stack");
	TabIcon = FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(), "IKRig.SolverStack");

	ViewMenuDescription = LOCTEXT("IKRigSolverStack_ViewMenu_Desc", "IK Rig Solver Stack");
	ViewMenuTooltip = LOCTEXT("IKRigSolverStack_ViewMenu_ToolTip", "Show the IK Rig Solver Stack Tab");
}

TSharedPtr<SToolTip> FIKRigSolverStackTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
{
	return  IDocumentation::Get()->CreateToolTip(LOCTEXT("IKRigSolverStackTooltip", "A stack of IK solvers executed from top to bottom. These are associated with goals to affect the skeleton."), NULL, TEXT("Shared/Editors/Persona"), TEXT("IKRigSolverStack_Window"));
}

TSharedRef<SWidget> FIKRigSolverStackTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	return SNew(SIKRigSolverStack, IKRigEditor.Pin()->GetController());
}

#undef LOCTEXT_NAMESPACE 
