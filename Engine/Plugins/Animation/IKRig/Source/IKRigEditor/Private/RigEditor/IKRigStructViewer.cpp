// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigEditor/IKRigStructViewer.h"

#include "Editor.h"
#include "IKRig.h"
#include "RetargetEditor/IKRetargeterController.h"
#include "Retargeter/IKRetargetOps.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigStructViewer)

#define LOCTEXT_NAMESPACE "IKRigStructViewer"

void UIKRigStructViewer::TriggerReinitializationIfNeeded(const FPropertyChangedEvent& InEvent)
{
	if (!InEvent.Property)
	{
		return;
	}

	const bool bIsInteractive = InEvent.ChangeType == EPropertyChangeType::Interactive;
	const bool bNeedsReinit = InEvent.Property->HasMetaData(IKRigReinitOnEditMetaLabel);

	// Allow reinitialization during interactive slider drag only for properties explicitly
	// marked ReinitializeOnEdit — this enables live viewport updates (e.g. foot dimension
	// debug drawing) while still avoiding unnecessary reinits for non-marked properties.
	if (bIsInteractive && !bNeedsReinit)
	{
		return;
	}

	// always reinitialize if property is marked for it
	if (bNeedsReinit)
	{
		OnNeedsReinitialized().Broadcast(StructToView.UniqueName, InEvent);
		return;
	}

	// if it is a name property, we want to reinitialize to validate it
	if (CastField<FNameProperty>(InEvent.Property) != nullptr)
	{
		OnNeedsReinitialized().Broadcast(StructToView.UniqueName, InEvent);
		return;
	}

	// always reinitialize if the property is a bone name
	// NOTE: this is needed because FBoneReference's child BoneName property
	// cannot be marked with the meta-tag (IKRig plugin doesn't own it and it's only relevant here)
	if (InEvent.Property->GetFName() == FName(TEXT("BoneName")))
	{
		OnNeedsReinitialized().Broadcast(StructToView.UniqueName, InEvent);
		return;
	}
}

void UIKRigStructViewer::SetupPropertyEditingCallbacks(const TSharedPtr<IPropertyHandle>& InProperty)
{
	UObject* ObjectToTransact = GetStructOwner();

	InProperty->SetOnPropertyValuePreChange(FSimpleDelegate::CreateLambda([ObjectToTransact]()
	{
		GEditor->BeginTransaction(LOCTEXT("IKRigStructViewer", "Edited property."));
		ObjectToTransact->Modify();
	}));
		
	InProperty->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda([this](const FPropertyChangedEvent& InEvent)
	{
		GEditor->EndTransaction();
		// Push wrapper values to the underlying struct before triggering reinitialization,
		// so the retargeter reads the latest values when it reinitializes.
		if (UIKRigStructWrapperBase* Wrapper = Cast<UIKRigStructWrapperBase>(this))
		{
			Wrapper->UpdateWrappedStructWithLatestValues();
		}
		TriggerReinitializationIfNeeded(InEvent);
	}));

	InProperty->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateLambda([ObjectToTransact]()
	{
		GEditor->BeginTransaction(LOCTEXT("IKRigStructViewerChild", "Edited child property."));
		ObjectToTransact->Modify();
	}));

	InProperty->SetOnChildPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateLambda([this](const FPropertyChangedEvent& InEvent)
	{
		GEditor->EndTransaction();
		// Push wrapper values to the underlying struct before triggering reinitialization,
		// so the retargeter reads the latest values when it reinitializes.
		if (UIKRigStructWrapperBase* Wrapper = Cast<UIKRigStructWrapperBase>(this))
		{
			Wrapper->UpdateWrappedStructWithLatestValues();
		}
		TriggerReinitializationIfNeeded(InEvent);
	}));
}

USkeleton* UIKRigStructViewer::GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle)
{
	if (!StructToView.IsValid())
	{
		return nullptr;
	}
	
	// NOTE: it's not ideal that we are hardcoding supported types here, but because UStruct's do not support multiple
	// inheritance we cannot use an interface to identify skeleton providers as we normally would.
	if (StructToView.Type->IsChildOf(FIKRetargetOpSkeletonProvider::StaticStruct()))
	{
		FName PropertyName = PropertyHandle->GetProperty()->GetFName();
		uint8* StructMemory = StructToView.MemoryProvider();
		FIKRetargetOpSkeletonProvider* SkeletonProvider = reinterpret_cast<FIKRetargetOpSkeletonProvider*>(StructMemory);
		USkeleton* Skeleton = SkeletonProvider->GetSkeleton(PropertyName);
		if (!Skeleton)
		{
			// if GetSkeleton(FName) returns null, try the alternative function that takes a property handle
			Skeleton = SkeletonProvider->GetSkeleton(PropertyHandle);
		}
		return Skeleton;
	}
	
	return nullptr;
}

TSharedRef<IDetailCustomization> FIKRigStructViewerCustomization::MakeInstance()
{
	return MakeShareable(new FIKRigStructViewerCustomization);
}

void FIKRigStructViewerCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	UIKRigStructViewer* StructViewer = Cast<UIKRigStructViewer>(ObjectsBeingCustomized[0]);
	if (!ensure(StructViewer))
	{
		return;
	}
	if (!ensure(StructViewer->IsValid()))
	{
		return;
	}

	if (StructViewer->GetClass() != UIKRigStructViewer::StaticClass())
	{
		return; // skip if it’s a derived class
	}

	// show the struct in the details panel
	const TSharedPtr<FStructOnScope>& StructOnScope = StructViewer->GetStructOnScope();
	FName StructTitle = StructViewer->GetTypeName();
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(StructTitle);

	// determine if this struct is customized
	// if it is, then we add the stuct itself and let the customization do its thing, otherwise we add all the struct properties
	const UStruct* Struct = StructViewer->GetStructOnScope().Get()->GetStruct();
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	const bool bIsStructCustomized =  PropertyEditorModule.IsCustomizedStruct(Struct, FCustomPropertyTypeLayoutMap());

	// the property rows to add callbacks to
	TArray<IDetailPropertyRow*> OutRows;

	if (bIsStructCustomized)
	{
		// add struct itself, triggering callbacks to customization
		IDetailPropertyRow* StructRow = CategoryBuilder.AddExternalStructure(StructOnScope.ToSharedRef(), EPropertyLocation::Default);
		StructRow->ShouldAutoExpand(true);
		OutRows.Add(StructRow);
	}
	else
	{
		// adds all the properties in the struct with categories intact 
		OutRows = AddAllPropertiesToCategoryGroups(StructOnScope, DetailBuilder);
	}
	
	// setup callbacks to begin/end an Undo transaction when property is edited
	for (IDetailPropertyRow* Row : OutRows)
	{
		StructViewer->SetupPropertyEditingCallbacks(Row->GetPropertyHandle());
	}
}

TArray<IDetailPropertyRow*> FIKRigStructViewerCustomization::AddAllPropertiesToCategoryGroups(
	const TSharedPtr<FStructOnScope>& InStructData,
	IDetailLayoutBuilder& InDetailBuilder)
{
	TArray<IDetailPropertyRow*> OutRows;

	const UStruct* Struct = InStructData->GetStruct();
	for (FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		FName PropertyName = Property->GetFName();
		TSharedPtr<IPropertyHandle> PropertyHandle = InDetailBuilder.AddStructurePropertyData(InStructData, PropertyName);
		if (!PropertyHandle.IsValid())
		{
			continue; // can happen with deprecated properties
		}
		IDetailPropertyRow& Row = InDetailBuilder.AddPropertyToCategory(PropertyHandle.ToSharedRef());
		OutRows.Add(&Row);
	}
	
	return MoveTemp(OutRows);
}

void UIKRigStructWrapperBase::Initialize(FIKRigStructToView& InStructToWrap, const FName& InWrapperPropertyName)
{
	StructToView = InStructToWrap;
	if (!ensureMsgf(StructToView.IsValid(), TEXT("Must have valid struct to wrap")))
	{
		return;
	}
	
	WrapperProperty = GetClass()->FindPropertyByName(InWrapperPropertyName);
	if (!ensureMsgf(WrapperProperty, TEXT("Wrapper class member variable not found.")))
	{
		return;
	}

	WrapperPropertyName = InWrapperPropertyName;

	// update wrapper to reflect current values
	UpdateWrapperStructWithLatestValues();
}

void UIKRigStructWrapperBase::InitializeWithRetargeter(
	FIKRigStructToView& InStructToWrap,
	const FName& InWrapperPropertyName,
	UIKRetargeterController* InRetargeterController)
{
	Initialize(InStructToWrap, InWrapperPropertyName);
	OnNeedsReinitialized().AddUObject(InRetargeterController, &UIKRetargeterController::OnOpPropertyChanged);
}

bool UIKRigStructWrapperBase::IsValid() const
{
	return StructToView.IsValid() && WrapperProperty != nullptr;
}

void UIKRigStructWrapperBase::SetPropertyHidden(const FName& InPropertyName, bool bHidden)
{
	if (bHidden)
	{
		if (PropertiesToHide.Contains(InPropertyName))
		{
			return;
		}
		PropertiesToHide.Add(InPropertyName);
	}
	else
	{
		PropertiesToHide.Remove(InPropertyName);
	}
}

void UIKRigStructWrapperBase::UpdateWrappedStructWithLatestValues()
{
	if (!(StructToView.IsValid() && WrapperProperty))
	{
		return;
	}

	// push wrapper values to the wrapped struct
	void* WrapperMemory = WrapperProperty->ContainerPtrToValuePtr<void>(this);
	void* WrappedMemory = StructToView.MemoryProvider();
	StructToView.Type->CopyScriptStruct(WrappedMemory, WrapperMemory);
}

void UIKRigStructWrapperBase::UpdateWrapperStructWithLatestValues()
{
	if (!(StructToView.IsValid() && WrapperProperty))
	{
		return;
	}
	
	void* WrapperMemory = WrapperProperty->ContainerPtrToValuePtr<void>(this);
	void* WrappedMemory = StructToView.MemoryProvider();
	StructToView.Type->CopyScriptStruct(WrapperMemory, WrappedMemory);
}

void UIKRigStructWrapperBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	UpdateWrappedStructWithLatestValues();

	// Paste operations (and other struct-level changes) come through PostEditChangeProperty rather than the
	// IPropertyHandle callbacks set up in SetupPropertyEditingCallbacks. This means that pasting property
	// values (e.g. right-click -> Paste on the Foot Settings section) correctly updates the underlying struct
	// data but does NOT trigger the reinitialization broadcast needed to refresh the viewport.
	// Calling TriggerReinitializationIfNeeded here ensures the viewport updates after a paste.
	TriggerReinitializationIfNeeded(PropertyChangedEvent);
}

TSharedRef<IDetailCustomization> FIKRigStructWrapperCustomization::MakeInstance()
{
	return MakeShareable(new FIKRigStructWrapperCustomization);
}

void FIKRigStructWrapperCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	InDetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	UIKRigStructWrapperBase* StructWrapper = Cast<UIKRigStructWrapperBase>(ObjectsBeingCustomized[0]);
	if (!ensure(StructWrapper))
	{
		return;
	}
	if (!StructWrapper->IsValid())
	{
		return;
	}

	// get a handle to the property representing the struct we are wrapping
	TSharedRef<IPropertyHandle> WrappedProperty = InDetailBuilder.GetProperty(StructWrapper->GetWrapperPropertyName(), StructWrapper->GetClass());
	// hide it so we can add the children properties manually
	InDetailBuilder.HideProperty(WrappedProperty);
	// setup undo/redo/reinit callbacks
	StructWrapper->SetupPropertyEditingCallbacks(WrappedProperty);
	
	// add all the immediate properties under the wrapped struct
	uint32 NumChildren;
	WrappedProperty->GetNumChildren(NumChildren);
	for (uint32 i = 0; i < NumChildren; ++i)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = WrappedProperty->GetChildHandle(i);
		if (!ChildHandle.IsValid())
		{
			continue;
		}
		const FName PropertyName = ChildHandle->GetProperty()->GetFName();
		if (StructWrapper->IsPropertyHidden(PropertyName))
		{
			continue;
		}
		
		InDetailBuilder.AddPropertyToCategory(ChildHandle);
	}
}

#undef LOCTEXT_NAMESPACE
