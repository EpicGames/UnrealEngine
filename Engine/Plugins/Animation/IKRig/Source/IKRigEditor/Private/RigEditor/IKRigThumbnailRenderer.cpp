// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigThumbnailRenderer.h"

#include "Engine/SkeletalMesh.h"
#include "Rig/IKRigDefinition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IKRigThumbnailRenderer)

bool UIKRigThumbnailRenderer::CanVisualizeAsset(UObject* Object)
{
	return GetPreviewMeshFromRig(Object) != nullptr;
}

void UIKRigThumbnailRenderer::Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget* RenderTarget, FCanvas* Canvas, bool bAdditionalViewFamily)
{
	if (USkeletalMesh* MeshToDraw = GetPreviewMeshFromRig(Object))
	{
		Super::Draw(MeshToDraw, X, Y, Width, Height, RenderTarget, Canvas, bAdditionalViewFamily);	
	}
}

EThumbnailRenderFrequency UIKRigThumbnailRenderer::GetThumbnailRenderFrequency(UObject* Object) const
{
	return GetPreviewMeshFromRig(Object) ? EThumbnailRenderFrequency::Realtime : EThumbnailRenderFrequency::OnPropertyChange;
}

USkeletalMesh* UIKRigThumbnailRenderer::GetPreviewMeshFromRig(UObject* Object) const
{
	const UIKRigDefinition* InRig = Cast<UIKRigDefinition>(Object);
	if (!InRig)
	{
		return nullptr;
	}

	return InRig->GetPreviewMesh();
}
