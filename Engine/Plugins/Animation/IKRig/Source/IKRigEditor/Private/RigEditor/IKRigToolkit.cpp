// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/IKRigToolkit.h"

#include "AnimationEditorViewportClient.h"
#include "AnimationEditorPreviewActor.h"
#include "EditorModeManager.h"
#include "GameFramework/WorldSettings.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "IPersonaToolkit.h"
#include "IAssetFamily.h"
#include "ISkeletonEditorModule.h"
#include "Preferences/PersonaOptions.h"
#include "AnimCustomInstanceHelper.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Rig/IKRigDefinition.h"
#include "IPersonaViewport.h"
#include "RigEditor/IKRigAnimInstance.h"
#include "RigEditor/IKRigCommands.h"
#include "RigEditor/IKRigEditMode.h"
#include "RigEditor/IKRigEditorController.h"
#include "RigEditor/IKRigMode.h"

#include "Misc/ITransaction.h"

#define LOCTEXT_NAMESPACE "IKRigEditorToolkit"

const FName IKRigEditorModes::IKRigEditorMode("IKRigEditorMode");
const FName IKRigEditorAppName = FName(TEXT("IKRigEditorApp"));

FIKRigEditorToolkit::FIKRigEditorToolkit()
	: EditorController(MakeShared<FIKRigEditorController>())
{
}

FIKRigEditorToolkit::~FIKRigEditorToolkit()
{
	if (PersonaToolkit.IsValid())
	{
		static constexpr bool bSetPreviewMeshInAsset = false;
		PersonaToolkit->SetPreviewMesh(nullptr, bSetPreviewMeshInAsset);
	}
}

void FIKRigEditorToolkit::InitAssetEditor(
	const EToolkitMode::Type Mode,
    const TSharedPtr<IToolkitHost>& InitToolkitHost, 
    UIKRigDefinition* IKRigAsset)
{
	EditorController->Initialize(SharedThis(this), IKRigAsset);

	BindCommands();
	
	FPersonaToolkitArgs PersonaToolkitArgs;
	PersonaToolkitArgs.OnPreviewSceneCreated = FOnPreviewSceneCreated::FDelegate::CreateSP(this, &FIKRigEditorToolkit::HandlePreviewSceneCreated);
	PersonaToolkitArgs.OnPreviewSceneSettingsCustomized = FOnPreviewSceneSettingsCustomized::FDelegate::CreateSP(this, &FIKRigEditorToolkit::HandleOnPreviewSceneSettingsCustomized);
	
	const FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(IKRigAsset, PersonaToolkitArgs);

	static constexpr bool bCreateDefaultStandaloneMenu = true;
	static constexpr bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(
		Mode, 
		InitToolkitHost, 
		IKRigEditorAppName, 
		FTabManager::FLayout::NullLayout, 
		bCreateDefaultStandaloneMenu, 
		bCreateDefaultToolbar, 
		IKRigAsset);

	TSharedRef<FIKRigMode> IKRigEditMode = MakeShareable(new FIKRigMode(SharedThis(this), PersonaToolkit->GetPreviewScene()));
	AddApplicationMode(IKRigEditorModes::IKRigEditorMode, IKRigEditMode);

	SetCurrentMode(IKRigEditorModes::IKRigEditorMode);

	GetEditorModeManager().SetDefaultMode(FIKRigEditMode::ModeName);
	GetEditorModeManager().ActivateDefaultMode();
	FIKRigEditMode* EditMode = GetEditorModeManager().GetActiveModeTyped<FIKRigEditMode>(FIKRigEditMode::ModeName);
	EditMode->SetEditorController(EditorController);

	ExtendToolbar();
	RegenerateMenusAndToolbars();
}

void FIKRigEditorToolkit::OnClose()
{
	FPersonaAssetEditorToolkit::OnClose();
	EditorController->Close();
}

void FIKRigEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_IKRigEditor", "IK Rig Editor"));
	TSharedRef<FWorkspaceItem> WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);
}

void FIKRigEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
}

void FIKRigEditorToolkit::BindCommands()
{
	const FIKRigCommands& Commands = FIKRigCommands::Get();

	ToolkitCommands->MapAction(
        Commands.Reset,
        FExecuteAction::CreateSP(EditorController, &FIKRigEditorController::Reset),
		EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.AutoRetargetChains,
		FExecuteAction::CreateSP(EditorController, &FIKRigEditorController::AutoGenerateRetargetChains),
		EUIActionRepeatMode::RepeatDisabled);
	
	ToolkitCommands->MapAction(
			Commands.AutoSetupFBIK,
			FExecuteAction::CreateSP(EditorController, &FIKRigEditorController::AutoGenerateFBIK),
			EUIActionRepeatMode::RepeatDisabled);

	ToolkitCommands->MapAction(
		Commands.ShowAssetSettings,
		FExecuteAction::CreateLambda([this]()
		{
			return EditorController->ShowAssetDetails();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]() ->bool
		{
			const UIKRigDefinition* Asset = EditorController->AssetController->GetAsset();
			return EditorController->IsObjectInDetailsView(Asset);	
		}));
}

void FIKRigEditorToolkit::ExtendToolbar()
{
	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	ToolbarExtender->AddToolBarExtension(
        "Asset",
        EExtensionHook::After,
        GetToolkitCommands(),
        FToolBarExtensionDelegate::CreateSP(this, &FIKRigEditorToolkit::FillToolbar)
    );
}

void FIKRigEditorToolkit::FillToolbar(FToolBarBuilder& ToolbarBuilder)
{
	ToolbarBuilder.BeginSection("Reset");
	{
		ToolbarBuilder.AddToolBarButton(
			FIKRigCommands::Get().Reset,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(),"Icons.Refresh"));

		ToolbarBuilder.AddSeparator();

		ToolbarBuilder.AddToolBarButton(
			FIKRigCommands::Get().AutoRetargetChains,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(),"IKRig.AutoRetarget"));

		ToolbarBuilder.AddToolBarButton(
			FIKRigCommands::Get().AutoSetupFBIK,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>(),
			FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(),"IKRig.AutoIK"));
	}
	ToolbarBuilder.EndSection();

	ToolbarBuilder.AddSeparator();
	ToolbarBuilder.AddWidget(SNew(SSpacer), NAME_None, true, HAlign_Right);

	ToolbarBuilder.BeginSection("Show Settings");
	{
		ToolbarBuilder.AddToolBarButton(
		FIKRigCommands::Get().ShowAssetSettings,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FIKRigEditorStyle::Get().GetStyleSetName(),"IKRig.AssetSettings"));
	}
	ToolbarBuilder.EndSection();
}

FName FIKRigEditorToolkit::GetToolkitFName() const
{
	return FName("IKRigEditor");
}

FText FIKRigEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("IKRigEditorAppLabel", "IK Rig Editor");
}

FText FIKRigEditorToolkit::GetToolkitName() const
{
	return FText::FromString(EditorController->AssetController->GetAsset()->GetName());
}

FLinearColor FIKRigEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor::White;
}

FString FIKRigEditorToolkit::GetWorldCentricTabPrefix() const
{
	return TEXT("IKRigEditor");
}

void FIKRigEditorToolkit::Tick(float DeltaTime)
{
	// forces viewport to always update, even when mouse pressed down in other tabs
	GetPersonaToolkit()->GetPreviewScene()->InvalidateViews();
}

TStatId FIKRigEditorToolkit::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FIKRigEditorToolkit, STATGROUP_Tickables);
}

void FIKRigEditorToolkit::HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder) const
{
	DetailBuilder.HideCategory("Additional Meshes");
	DetailBuilder.HideCategory("Physics");
	DetailBuilder.HideCategory("Mesh");
	DetailBuilder.HideCategory("Animation Blueprint");
}

bool FIKRigEditorToolkit::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	if (InContext.PrimaryObject == EditorController->AssetController)
	{
		return true;
	}
	for (const TPair<UObject*, FTransactionObjectEvent>& ObjectTransactionPair : TransactionObjectContexts)
	{
		if (ObjectTransactionPair.Get<0>() == EditorController->AssetController)
		{
			return true;
		}
	}	
	return false;
}

void FIKRigEditorToolkit::PostUndo(bool bSuccess)
{
	FScopedReinitializeIKRig Reinitialize(EditorController->AssetController);
}

void FIKRigEditorToolkit::PostRedo(bool bSuccess)
{
	FScopedReinitializeIKRig Reinitialize(EditorController->AssetController);
}

void FIKRigEditorToolkit::HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene)
{
	AAnimationEditorPreviewActor* Actor = InPersonaPreviewScene->GetWorld()->SpawnActor<AAnimationEditorPreviewActor>(AAnimationEditorPreviewActor::StaticClass(), FTransform::Identity);
	Actor->SetFlags(RF_Transient);
	InPersonaPreviewScene->SetActor(Actor);
	
	// create the preview skeletal mesh component
	EditorController->SkelMeshComponent = NewObject<UDebugSkelMeshComponent>(Actor);
	// turn off default bone rendering (we do our own in the IK Rig editor)
	EditorController->SkelMeshComponent->SkeletonDrawMode = ESkeletonDrawMode::Hidden;

	// setup and apply an anim instance to the skeletal mesh component
	UIKRigAnimInstance* AnimInstance = NewObject<UIKRigAnimInstance>(EditorController->SkelMeshComponent, TEXT("IKRigAnimScriptInstance"));
	EditorController->AnimInstance = AnimInstance;
	AnimInstance->SetIKRigAsset(EditorController->AssetController->GetAsset());

	// must set the animation mode to "AnimationCustomMode" to prevent USkeletalMeshComponent::InitAnim() from
	// replacing the custom ik rig anim instance with a generic preview anim instance.
	EditorController->SkelMeshComponent->PreviewInstance = AnimInstance;
    EditorController->SkelMeshComponent->SetAnimationMode(EAnimationMode::AnimationCustomMode);
    
    // must call AddComponent() BEFORE assigning the mesh to prevent auto-assignment of a default anim instance
    InPersonaPreviewScene->AddComponent(EditorController->SkelMeshComponent, FTransform::Identity);

	// apply mesh component to the preview scene
	InPersonaPreviewScene->SetPreviewMeshComponent(EditorController->SkelMeshComponent);
	InPersonaPreviewScene->SetAllowMeshHitProxies(false);
	InPersonaPreviewScene->SetAdditionalMeshesSelectable(false);
	EditorController->SkelMeshComponent->bSelectable = false;
	
	// set the skeletal mesh on the component
    // NOTE: this must be done AFTER setting the PreviewInstance so that it assigns it as the main anim instance
    USkeletalMesh* Mesh = EditorController->AssetController->GetSkeletalMesh();
    InPersonaPreviewScene->SetPreviewMesh(Mesh);
}

void FIKRigEditorToolkit::HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView) const
{
	EditorController->SetDetailsView(InDetailsView);
}

void FIKRigEditorToolkit::HandleViewportCreated(const TSharedRef<IPersonaViewport>& InViewport)
{
	// register callbacks to allow the asset to store the Bone Size viewport setting
	FEditorViewportClient& ViewportClient = InViewport->GetViewportClient();
	if (FAnimationViewportClient* AnimViewportClient = static_cast<FAnimationViewportClient*>(&ViewportClient))
	{
		AnimViewportClient->OnSetBoneSize.BindLambda([this](float InBoneSize)
		{
			if (UIKRigDefinition* Asset = EditorController->AssetController->GetAsset())
			{
				Asset->Modify();
				Asset->BoneSize = InBoneSize;
			}
		});
		
		AnimViewportClient->OnGetBoneSize.BindLambda([this]() -> float
		{
			if (const UIKRigDefinition* Asset = EditorController->AssetController->GetAsset())
			{
				return Asset->BoneSize;
			}

			return 1.0f;
		});
	}

	// highlight viewport when processor disabled
	auto GetBorderColorAndOpacity = [this]()
	{
		// no processor or processor not initialized
		const FIKRigProcessor* Processor = EditorController.Get().GetIKRigProcessor();
		if (!Processor || !Processor->IsInitialized() )
		{
			return FLinearColor::Red;
		}

		// highlight viewport if warnings
		const TArray<FText>& Warnings = Processor->Log.GetWarnings();
		if (!Warnings.IsEmpty())
		{
			return FLinearColor::Yellow;
		}

		return FLinearColor::Transparent;
	};

	InViewport->AddOverlayWidget(
		SNew(SBorder)
		.BorderImage(FIKRigEditorStyle::Get().GetBrush("IKRig.Viewport.Border"))
		.BorderBackgroundColor_Lambda(GetBorderColorAndOpacity)
		.Visibility(EVisibility::HitTestInvisible)
		.Padding(0.0f)
		.ShowEffectWhenDisabled(false)
	);
}

#undef LOCTEXT_NAMESPACE
