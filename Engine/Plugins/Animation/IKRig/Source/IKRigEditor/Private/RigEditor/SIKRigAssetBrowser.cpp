// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/SIKRigAssetBrowser.h"

#include "AnimPreviewInstance.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Animation/AnimComposite.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSequence.h"
#include "Animation/PoseAsset.h"
#include "RigEditor/IKRigController.h"
#include "RigEditor/IKRigEditorController.h"
#include "Engine/SkeletalMesh.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UObject/AssetRegistryTagsContext.h"

#define LOCTEXT_NAMESPACE "IKRigAssetBrowser"

void SIKRigAssetBrowser::Construct(
	const FArguments& InArgs,
	TSharedRef<FIKRigEditorController> InEditorController)
{
	EditorController = InEditorController;
	EditorController.Pin()->AssetBrowserView = SharedThis(this);
	CommandList = MakeShared<FUICommandList>();

	const FGlobalEditorCommonCommands& GlobalCommands = FGlobalEditorCommonCommands::Get();
	CommandList->MapAction(
		GlobalCommands.FindInContentBrowser,
		FExecuteAction::CreateSP(this, &SIKRigAssetBrowser::ExecuteFindInContentBrowserAction),
		FCanExecuteAction::CreateSP(this, &SIKRigAssetBrowser::CanExecuteFindInContentBrowserAction)
	);
	
	ChildSlot
    [
        SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SAssignNew(AssetBrowserBox, SBox)
		]
    ];

	RefreshView();
}

FReply SIKRigAssetBrowser::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return SBox::OnKeyDown(InGeometry, InKeyEvent);
}

void SIKRigAssetBrowser::ExecuteFindInContentBrowserAction()
{
	if (!GetCurrentSelectionDelegate.IsBound())
	{
		return;
	}

	const TArray<FAssetData> SelectedAssets = GetCurrentSelectionDelegate.Execute();
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	ContentBrowserModule.Get().SyncBrowserToAssets(SelectedAssets);
}

bool SIKRigAssetBrowser::CanExecuteFindInContentBrowserAction()
{
	return GetCurrentSelectionDelegate.IsBound() && !GetCurrentSelectionDelegate.Execute().IsEmpty();
}

void SIKRigAssetBrowser::RefreshView()
{
	// setup filtering
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimMontage::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UPoseAsset::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UAnimComposite::StaticClass()->GetClassPathName());
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::Column;
	AssetPickerConfig.bAddFilterUI = true;
	AssetPickerConfig.bShowPathInColumnView = true;
	AssetPickerConfig.bShowTypeInColumnView = true;
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::ItemDiskSize.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(ContentBrowserItemAttributes::VirtualizedData.ToString());
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("RevisionControl"));
	AssetPickerConfig.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SIKRigAssetBrowser::OnShouldFilterAsset);
	AssetPickerConfig.DefaultFilterMenuExpansion = EAssetTypeCategories::Animation;
	AssetPickerConfig.OnAssetDoubleClicked = FOnAssetSelected::CreateSP(this, &SIKRigAssetBrowser::OnAssetDoubleClicked);
	AssetPickerConfig.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SIKRigAssetBrowser::OnGetAssetContextMenu);
	AssetPickerConfig.bAllowNullSelection = false;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = false;
	AssetPickerConfig.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);

	// hide all asset registry columns by default (we only really want the name and path)
	UObject* AnimSequenceDefaultObject = UAnimSequence::StaticClass()->GetDefaultObject();
	FAssetRegistryTagsContextData TagsContext(AnimSequenceDefaultObject, EAssetRegistryTagsCaller::Uncategorized);
	AnimSequenceDefaultObject->GetAssetRegistryTags(TagsContext);
	for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
	{
		AssetPickerConfig.HiddenColumnNames.Add(TagPair.Key.ToString());
	}

	// Also hide the type column by default (but allow users to enable it, so don't use bShowTypeInColumnView)
	AssetPickerConfig.HiddenColumnNames.Add(TEXT("Class"));

	const FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
	AssetBrowserBox->SetContent(ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig));
}

TSharedPtr<SWidget> SIKRigAssetBrowser::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets) const
{
	if (SelectedAssets.Num() <= 0)
	{
		return nullptr;
	}

	UObject* SelectedAsset = SelectedAssets[0].GetAsset();
	if (SelectedAsset == nullptr)
	{
		return nullptr;
	}
	
	FMenuBuilder MenuBuilder(true, CommandList);

	MenuBuilder.BeginSection(TEXT("Asset"), LOCTEXT("AssetSectionLabel", "Asset"));
	{
		const TSharedPtr<FUICommandInfo> FindInContentBrowserCommand = FGlobalEditorCommonCommands::Get().FindInContentBrowser;
		checkf(FindInContentBrowserCommand.IsValid(), TEXT("FGlobalEditorCommonCommands::FindInContentBrowser command not found!"));
		MenuBuilder.AddMenuEntry(FindInContentBrowserCommand);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SIKRigAssetBrowser::OnAssetDoubleClicked(const FAssetData& AssetData)
{
	if (!AssetData.GetAsset())
	{
		return;
	}
	
	UAnimationAsset* NewAnimationAsset = Cast<UAnimationAsset>(AssetData.GetAsset());
	if (NewAnimationAsset && EditorController.Pin().IsValid())
	{
		EditorController.Pin()->PlayAnimationAsset(NewAnimationAsset);
	}
}

bool SIKRigAssetBrowser::OnShouldFilterAsset(const struct FAssetData& AssetData)
{
	// is this an animation asset?
	if (!AssetData.IsInstanceOf(UAnimationAsset::StaticClass()))
	{
		return true;
	}
	
	// controller setup
	FIKRigEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return true;
	}

	// get skeletal mesh
	USkeletalMesh* SkeletalMesh = Controller->AssetController->GetSkeletalMesh();
	if (!SkeletalMesh)
	{
		return true;
	}
	
	// get skeleton
	USkeleton* DesiredSkeleton = SkeletalMesh->GetSkeleton();
	if (!DesiredSkeleton)
	{
		return true;
	}

	return !DesiredSkeleton->IsCompatibleForEditor(AssetData);
}

#undef LOCTEXT_NAMESPACE
