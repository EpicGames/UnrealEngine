// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/SIKRigHierarchy.h"

#include "Rig/IKRigProcessor.h"
#include "Rig/Solvers/IKRigSolverBase.h"
#include "Engine/SkeletalMesh.h"
#include "IPersonaToolkit.h"
#include "RenderGraphResources.h"
#include "SKismetInspector.h"
#include "Dialogs/Dialogs.h"
#include "SPositiveActionButton.h"
#include "RigEditor/IKRigEditorController.h"
#include "RigEditor/IKRigEditorStyle.h"
#include "RigEditor/IKRigSkeletonCommands.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/SIKRigSolverStack.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UICommandList.h"
#include "Preferences/PersonaOptions.h"
#include "Widgets/Input/SSearchBox.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SIKRigHierarchy)

#define LOCTEXT_NAMESPACE "SIKRigHierarchy"

static const FName IKRigElementColumnName(TEXT("ElementName"));
static const FName IKRigChainColumnName(TEXT("RetargetChainName"));

FIKRigTreeElement::FIKRigTreeElement(const FText& InKey, EIKRigTreeElementType InType,
									 const TSharedRef<FIKRigEditorController>& InEditorController)
	: Key(InKey)
	, ElementType(InType)
	, EditorController(InEditorController)
	, OptionalBoneDetails(nullptr)
{}

void FIKRigTreeElement::RequestRename()
{
	OnRenameRequested.ExecuteIfBound();
}

void FIKRigTreeElement::GetStruct(const UScriptStruct*& OutType, uint8*& OutMemory) const
{
	const TSharedPtr<FIKRigEditorController>& Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	if (Controller->AssetController == nullptr)
	{
		return;
	}

	/* TODO
	switch(ElementType)
	{
	case IKRigTreeElementType::BONE:
		if (OptionalBoneDetails == nullptr)
		{
			OptionalBoneDetails = Controller->CreateBoneDetails(AsShared());
		}
		return OptionalBoneDetails;
	case IKRigTreeElementType::BONE_SETTINGS:
		return Controller->AssetController->GetBoneSettings(BoneSettingBoneName, BoneSettingsSolverIndex);
	case IKRigTreeElementType::GOAL:
		return Controller->AssetController->GetGoal(GoalName);
	case IKRigTreeElementType::SOLVERGOAL:
		if (const FIKRigSolverBase* SolverWithEffector = Controller->AssetController->GetSolverAtIndex(EffectorIndex))
		{
			return nullptr; // TODO SolverWithEffector->GetGoalSettings(EffectorGoalName);
		}
	default:
		return nullptr;
	}*/
}

FName FIKRigTreeElement::GetChainName() const
{
	const UIKRigController* AssetController = EditorController.Pin()->AssetController;
	const FIKRigSkeleton* Skeleton  = EditorController.Pin()->GetCurrentIKRigSkeleton();
	
	switch(ElementType)
	{
	case EIKRigTreeElementType::BONE:
		return AssetController->GetRetargetChainFromBone(BoneName, Skeleton);
	case EIKRigTreeElementType::BONE_SETTINGS:
		return AssetController->GetRetargetChainFromBone(BoneSettingBoneName, Skeleton);
	case EIKRigTreeElementType::GOAL:
		return AssetController->GetRetargetChainFromGoal(GoalName);
	case EIKRigTreeElementType::GOAL_SETTINGS:
		return AssetController->GetRetargetChainFromGoal(EffectorGoalName);
	default:
		return NAME_None;
	}
}

void SIKRigHierarchyItem::Construct(const FArguments& InArgs)
{
	WeakTreeElement = InArgs._TreeElement;
	EditorController = InArgs._EditorController;
	HierarchyView = InArgs._HierarchyView;

	TSharedPtr<FIKRigTreeElement> TreeElement = WeakTreeElement.Pin();
	
	// is this element affected by the selected solver?
	bool bIsConnectedToSelectedSolver;
	const int32 SelectedSolver = EditorController.Pin()->GetSelectedSolverIndex();
	if (SelectedSolver == INDEX_NONE)
	{
		bIsConnectedToSelectedSolver = HierarchyView.Pin()->IsElementConnectedToAnySolver(WeakTreeElement);
	}
	else
	{
		bIsConnectedToSelectedSolver = HierarchyView.Pin()->IsElementConnectedToSolver(WeakTreeElement, SelectedSolver);
	}

	// determine text style
	FTextBlockStyle NormalText = FIKRigEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("IKRig.Tree.NormalText");
	FTextBlockStyle ItalicText = FIKRigEditorStyle::Get().GetWidgetStyle<FTextBlockStyle>("IKRig.Tree.ItalicText");
	FSlateFontInfo TextFont;
	FSlateColor TextColor;
	if (bIsConnectedToSelectedSolver)
	{
		// elements connected to the selected solver are green
		TextFont = ItalicText.Font;
		TextColor = NormalText.ColorAndOpacity;
	}
	else
	{
		TextFont = NormalText.Font;
		TextColor = FLinearColor(0.2f, 0.2f, 0.2f, 0.5f);
	}

	// determine which icon to use for tree element
	const FSlateBrush* Brush = FAppStyle::Get().GetBrush("SkeletonTree.Bone");
	switch(TreeElement->ElementType)
	{
		case EIKRigTreeElementType::BONE:
			if (!HierarchyView.Pin()->IsElementExcludedBone(WeakTreeElement))
			{
				Brush = FAppStyle::Get().GetBrush("SkeletonTree.Bone");
			}
			else
			{
				Brush = FAppStyle::Get().GetBrush("SkeletonTree.BoneNonWeighted");
			}
			break;
		case EIKRigTreeElementType::BONE_SETTINGS:
			Brush = FIKRigEditorStyle::Get().GetBrush("IKRig.Tree.BoneWithSettings");
			break;
		case EIKRigTreeElementType::GOAL:
			Brush = FIKRigEditorStyle::Get().GetBrush("IKRig.Tree.Goal");
			break;
		case EIKRigTreeElementType::GOAL_SETTINGS:
			Brush = FIKRigEditorStyle::Get().GetBrush("IKRig.Tree.Effector");
			break;
		default:
			checkNoEntry();
	}

	TSharedPtr<SHorizontalBox> RowBox;

	ChildSlot
	[
		SAssignNew(RowBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.MaxWidth(18)
		.FillWidth(1.0)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(Brush)
		]
	];
	
	if (TreeElement->ElementType == EIKRigTreeElementType::BONE)
	{
		RowBox->AddSlot()
        .AutoWidth()
        .VAlign(VAlign_Center)
        [
			SNew(STextBlock)
			.Text(this, &SIKRigHierarchyItem::GetName)
			.Font(TextFont)
			.ColorAndOpacity(TextColor)
        ];
	}
	else
	{
		TSharedPtr< SInlineEditableTextBlock > InlineWidget;
		RowBox->AddSlot()
        .AutoWidth()
        .VAlign(VAlign_Center)
        [
	        SAssignNew(InlineWidget, SInlineEditableTextBlock)
		    .Text(this, &SIKRigHierarchyItem::GetName)
		    .Font(TextFont)
			.ColorAndOpacity(TextColor)
			.OnVerifyTextChanged(this, &SIKRigHierarchyItem::OnVerifyNameChanged)
		    .OnTextCommitted(this, &SIKRigHierarchyItem::OnNameCommitted)
		    .MultiLine(false)
        ];
		TreeElement->OnRenameRequested.BindSP(InlineWidget.Get(), &SInlineEditableTextBlock::EnterEditingMode);
	}
}

void SIKRigHierarchyItem::OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const
{
	check(WeakTreeElement.IsValid());

	if (!(InCommitType == ETextCommit::OnEnter || InCommitType == ETextCommit::OnUserMovedFocus))
	{
		return; // make sure user actually intends to commit a name change
	}

	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	const FText OldText = WeakTreeElement.Pin()->Key;
	const FName OldName = WeakTreeElement.Pin()->GoalName;
	const FName PotentialNewName = FName(InText.ToString());
	const FName NewName = Controller->AssetController->RenameGoal(OldName, PotentialNewName);
	if (NewName != NAME_None)
	{
		WeakTreeElement.Pin()->Key = FText::FromName(NewName);
		WeakTreeElement.Pin()->GoalName = NewName;
	}
	
	HierarchyView.Pin()->ReplaceItemInSelection(OldText, WeakTreeElement.Pin()->Key);
}

bool SIKRigHierarchyItem::OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage) const
{
	// TODO let the user know when/why the goal can't be renamed 
	return true;
}

FText SIKRigHierarchyItem::GetName() const
{
	return WeakTreeElement.Pin()->Key;
}

TSharedRef<SWidget> SIKRigSkeletonRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	// the actual bone/goal/settings element in the hierarchy
	if (ColumnName == IKRigElementColumnName)
	{
		return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew( SExpanderArrow, SharedThis(this) )
			.ShouldDrawWires(true)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SIKRigHierarchyItem)
			.EditorController(EditorController)
			.HierarchyView(HierarchyView)
			.TreeElement(WeakTreeElement)
		];
	}

	// the column of chain names for each hierarchy element
	if (ColumnName == IKRigChainColumnName)
	{
		const FName ChainName = WeakTreeElement.Pin()->GetChainName();
		const FText ChainText = (ChainName == NAME_None) ? FText::FromString("") : FText::FromName(ChainName);
		const FSlateFontInfo Font = ChainName != "(Pelvis)" ? FAppStyle::Get().GetFontStyle("NormalFont") : FAppStyle::Get().GetFontStyle("BoldFont");
		
		TSharedPtr< SHorizontalBox > RowBox;
		SAssignNew(RowBox, SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock).Text(ChainText).Font(Font)
		];

		return RowBox.ToSharedRef();
	}
	
	return SNullWidget::NullWidget;
	
}

TSharedRef<FIKRigSkeletonDragDropOp> FIKRigSkeletonDragDropOp::New(TWeakPtr<FIKRigTreeElement> InElement)
{
	TSharedRef<FIKRigSkeletonDragDropOp> Operation = MakeShared<FIKRigSkeletonDragDropOp>();
	Operation->Element = InElement;
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FIKRigSkeletonDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
        .Visibility(EVisibility::Visible)
        .BorderImage(FAppStyle::GetBrush("Menu.Background"))
        [
            SNew(STextBlock)
            .Text(FText::FromString(Element.Pin()->Key.ToString()))
        ];
}

FReply SIKRigSkeletonRow::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = HierarchyView.Pin()->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<FIKRigTreeElement> DraggedElement = SelectedItems[0];
	if (DraggedElement->ElementType != EIKRigTreeElementType::GOAL)
	{
		return FReply::Unhandled();
	}
	
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TSharedRef<FIKRigSkeletonDragDropOp> DragDropOp = FIKRigSkeletonDragDropOp::New(DraggedElement);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SIKRigSkeletonRow::HandleCanAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
    TSharedPtr<FIKRigTreeElement> TargetItem)
{
	TOptional<EItemDropZone> ReturnedDropZone;
	
	const TSharedPtr<FIKRigSkeletonDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FIKRigSkeletonDragDropOp>();
	if (DragDropOp.IsValid())
	{
		if (TargetItem.Get()->ElementType == EIKRigTreeElementType::BONE)
        {
        	ReturnedDropZone = EItemDropZone::BelowItem;	
        }
	}
	
	return ReturnedDropZone;
}

FReply SIKRigSkeletonRow::HandleAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
    TSharedPtr<FIKRigTreeElement> TargetItem)
{
	const TSharedPtr<FIKRigSkeletonDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FIKRigSkeletonDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}
	
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return FReply::Handled();
	}

	if (DragDropOp->Element.IsValid())
	{
		return FReply::Handled();
	}

	const FIKRigTreeElement& DraggedElement = *DragDropOp.Get()->Element.Pin().Get();
	ensure(DraggedElement.ElementType == EIKRigTreeElementType::GOAL);		// drag only supported for goals
	ensure(TargetItem.Get()->ElementType == EIKRigTreeElementType::BONE);	// drop only supported for bones

	// re-parent the goal to a different bone
	const UIKRigController* AssetController = Controller->AssetController;
	AssetController->SetGoalBone(DraggedElement.GoalName, TargetItem.Get()->BoneName);
	
	return FReply::Handled();
}

void SIKRigHierarchy::Construct(const FArguments& InArgs, TSharedRef<FIKRigEditorController> InEditorController)
{
	EditorController = InEditorController;
	EditorController.Pin()->SetHierarchyView(SharedThis(this));
	TextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));
	CommandList = MakeShared<FUICommandList>();
	BindCommands();
	
	ChildSlot
    [
        SNew(SVerticalBox)

        +SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility_Lambda([this](){return IsSkeletonLoaded() ? EVisibility::Visible : EVisibility::Collapsed; })
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.f, 0.0))
			[
				SNew(SPositiveActionButton)
				.OnGetMenuContent(this, &SIKRigHierarchy::CreateAddNewMenu)
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSearchBox)
				.SelectAllTextWhenFocused(true)
				.OnTextChanged( this, &SIKRigHierarchy::OnFilterTextChanged )
				.HintText(LOCTEXT( "SearchBoxHint", "Filter Hierarchy Tree..."))
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.f, 0.0))
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.ForegroundColor(FSlateColor::UseStyle())
				.ContentPadding(2.0f)
				.OnGetMenuContent( this, &SIKRigHierarchy::CreateFilterMenuWidget )
				.ToolTipText(LOCTEXT( "FilterMenuLabel", "Filter hierarchy tree options."))
				.HasDownArrow(true)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]

    	+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		[
			SNew(SHorizontalBox)
			.Visibility_Lambda([this](){return IsSkeletonLoaded() ? EVisibility::Collapsed : EVisibility::Visible; })

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.MaxWidth(180.0f)
			.Padding(3.0f, 1.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
				.ForegroundColor(FLinearColor::White)
				.OnClicked(FOnClicked::CreateSP(this, &SIKRigHierarchy::OnImportSkeletonClicked))
				.Text(LOCTEXT("ImportSkeletonLabel", "Import Hierarchy"))
				.IsEnabled_Lambda([this](){ return !IsSkeletonLoaded(); })
			]
		]
		
        +SVerticalBox::Slot()
        .Padding(0.0f, 0.0f)
        [
            SNew(SBorder)
        	.Visibility_Lambda([this](){return IsSkeletonLoaded() ? EVisibility::Visible : EVisibility::Collapsed; })
            .Padding(2.0f)
            .BorderImage(FAppStyle::GetBrush("SCSEditor.TreePanel"))
            [
                SAssignNew(TreeView, SIKRigSkeletonTreeView)
                .TreeItemsSource(&RootElements)
                .SelectionMode(ESelectionMode::Multi)
                .OnGenerateRow_Lambda( [this](
					TSharedPtr<FIKRigTreeElement> InItem,
					const TSharedRef<STableViewBase>& OwnerTable) -> TSharedRef<ITableRow>
                {
                	return SNew(SIKRigSkeletonRow, OwnerTable)
                	.EditorController(EditorController.Pin().ToSharedRef())
                	.TreeElement(InItem)
                	.HierarchyView(SharedThis(this));
                })
                .OnGetChildren(this, &SIKRigHierarchy::HandleGetChildrenForTree)
                .OnSelectionChanged(this, &SIKRigHierarchy::OnSelectionChanged)
                .OnContextMenuOpening(this, &SIKRigHierarchy::CreateContextMenu)
                .OnMouseButtonClick(this, &SIKRigHierarchy::OnItemClicked)
                .OnMouseButtonDoubleClick(this, &SIKRigHierarchy::OnItemDoubleClicked)
                .OnSetExpansionRecursive(this, &SIKRigHierarchy::OnSetExpansionRecursive)
                .HighlightParentNodesForSelection(false)
                .HeaderRow
				(
					SNew(SHeaderRow)
					
					+ SHeaderRow::Column(IKRigElementColumnName)
					.DefaultLabel(LOCTEXT("IKRigElementColumnLabel", "Rig Element"))
					.FillWidth(0.7f)
						
					+ SHeaderRow::Column(IKRigChainColumnName)
					.DefaultLabel(LOCTEXT("IKRigChainColumnLabel", "Retarget Chain"))
					.FillWidth(0.3f)
				)
            ]
        ]
    ];

	constexpr bool IsInitialSetup = true;
	RefreshTreeView(IsInitialSetup);
}

void SIKRigHierarchy::AddSelectedItemFromViewport(
	const FName& ItemName,
	EIKRigTreeElementType ItemType,
	const bool bReplace)
{	
	// nothing to add
	if (ItemName == NAME_None)
	{
		return;
	}

	// record what was already selected
	TArray<TSharedPtr<FIKRigTreeElement>> PreviouslySelectedItems = TreeView->GetSelectedItems();
	// add/remove items as needed
	for (const TSharedPtr<FIKRigTreeElement>& Item : AllElements)
	{
		bool bIsBeingAdded = false;
		switch (ItemType)
		{
		case EIKRigTreeElementType::GOAL:
			if (ItemName == Item->GoalName)
			{
				bIsBeingAdded = true;
			}
			break;
		case EIKRigTreeElementType::BONE:
			if (ItemName == Item->BoneName)
			{
				bIsBeingAdded = true;
			}
			break;
		default:
			ensureMsgf(false, TEXT("IKRig cannot select anything but bones and goals in viewport."));
			return;
		}

		if (bReplace)
		{
			if (bIsBeingAdded)
			{
				TreeView->ClearSelection();
				AddItemToSelection(Item);
				return;
			}
			
			continue;
		}

		// remove if already selected (invert)
		if (bIsBeingAdded && PreviouslySelectedItems.Contains(Item))
		{
			RemoveItemFromSelection(Item);
			continue;
		}

		// add if being added
		if (bIsBeingAdded)
		{
			AddItemToSelection(Item);
			continue;
		}
	}
}

void SIKRigHierarchy::AddItemToSelection(const TSharedPtr<FIKRigTreeElement>& InItem)
{
	TreeView->SetItemSelection(InItem, true, ESelectInfo::Direct);
    
	if(GetDefault<UPersonaOptions>()->bExpandTreeOnSelection)
	{
		TSharedPtr<FIKRigTreeElement> ItemToExpand = InItem->Parent;
		while(ItemToExpand.IsValid())
		{
			TreeView->SetItemExpansion(ItemToExpand, true);
			ItemToExpand = ItemToExpand->Parent;
		}
	}
    
	TreeView->RequestScrollIntoView(InItem);
}

void SIKRigHierarchy::RemoveItemFromSelection(const TSharedPtr<FIKRigTreeElement>& InItem)
{
	TreeView->SetItemSelection(InItem, false, ESelectInfo::Direct);
}

void SIKRigHierarchy::ReplaceItemInSelection(const FText& OldName, const FText& NewName)
{
	for (const TSharedPtr<FIKRigTreeElement>& Item : AllElements)
	{
		// remove old selection
		if (Item->Key.EqualTo(OldName))
		{
			TreeView->SetItemSelection(Item, false, ESelectInfo::Direct);
		}
		// add new selection
		if (Item->Key.EqualTo(NewName))
		{
			TreeView->SetItemSelection(Item, true, ESelectInfo::Direct);
		}
	}
}

void SIKRigHierarchy::GetSelectedBoneChains(TArray<FBoneChain>& OutChains) const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	const FIKRigSkeleton& Skeleton = Controller->AssetController->GetIKRigSkeleton();

	// get selected bones
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBoneItems;
	GetSelectedBones(SelectedBoneItems);

	// get selected bone indices
	TArray<int32> SelectedBones;
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBoneItems)
	{
		const FName BoneName = BoneItem.Get()->BoneName;
		const int32 BoneIndex = Skeleton.GetBoneIndexFromName(BoneName);
		SelectedBones.Add(BoneIndex);
	}

	// get bones in chains
	Skeleton.GetChainsInList(SelectedBones, OutChains);
	
	// add goals (if there are any on the end bone of the chain)
	for (FBoneChain& Chain : OutChains)
	{
		const FName GoalName = Controller->AssetController->GetGoalNameForBone(Chain.EndBone.BoneName);
		if (GoalName != NAME_None)
		{
			Chain.IKGoalName = GoalName;
		}
	}
}

TArray<TSharedPtr<FIKRigTreeElement>> SIKRigHierarchy::GetSelectedItems() const
{
	return TreeView->GetSelectedItems();
}

bool SIKRigHierarchy::HasSelectedItems() const
{
	return TreeView->GetNumItemsSelected() > 0;
}

bool SIKRigHierarchy::IsElementConnectedToSolver(TWeakPtr<FIKRigTreeElement> InTreeElement, int32 SolverIndex)
{
	const UIKRigController* AssetController = EditorController.Pin()->AssetController;
	const TSharedPtr<FIKRigTreeElement> TreeElement = InTreeElement.Pin();
	const FIKRigSolverBase* Solver = AssetController->GetSolverAtIndex(SolverIndex);
	
	if (!Solver)
	{
		return false; // not a valid solver index
	}
	
	if (TreeElement->ElementType == EIKRigTreeElementType::BONE)
	{
		// is this bone affected by this solver?
		return Solver->IsBoneAffectedBySolver(TreeElement->BoneName, AssetController->GetIKRigSkeleton());
	}

	if (TreeElement->ElementType == EIKRigTreeElementType::BONE_SETTINGS)
	{
		// is this bone setting belonging to the solver?
		return Solver->HasSettingsOnBone(TreeElement->BoneSettingBoneName);
	}

	if (TreeElement->ElementType == EIKRigTreeElementType::GOAL)
	{
		// is goal connected to the solver?
		return AssetController->IsGoalConnectedToSolver(TreeElement->GoalName, SolverIndex);
	}

	if (TreeElement->ElementType == EIKRigTreeElementType::GOAL_SETTINGS)
	{
		// is this an effector for this solver?
		return TreeElement->EffectorIndex == SolverIndex;
	}

	checkNoEntry();
	return false;
}

bool SIKRigHierarchy::IsElementConnectedToAnySolver(TWeakPtr<FIKRigTreeElement> InTreeElement)
{
	const UIKRigController* AssetController = EditorController.Pin()->AssetController;
	const TSharedPtr<FIKRigTreeElement> TreeElement = InTreeElement.Pin();
	
	const int32 NumSolvers = AssetController->GetNumSolvers();
	for (int32 SolverIndex=0; SolverIndex<NumSolvers; ++SolverIndex)
	{
		if (IsElementConnectedToSolver(TreeElement, SolverIndex))
		{
			return true;
		}
	}

	return false;
}

bool SIKRigHierarchy::IsElementExcludedBone(TWeakPtr<FIKRigTreeElement> TreeElement)
{
	if (TreeElement.Pin()->ElementType != EIKRigTreeElementType::BONE)
	{
		return false;
	}
	
	// is this bone excluded?
	const UIKRigController* AssetController = EditorController.Pin()->AssetController;
	return AssetController->GetBoneExcluded(TreeElement.Pin()->BoneName);
}

bool SIKRigHierarchy::IsSkeletonLoaded() const
{
	const UIKRigController* AssetController = EditorController.Pin()->AssetController;
	return AssetController->GetAsset()->GetPreviewMesh() != nullptr;
}

FReply SIKRigHierarchy::OnImportSkeletonClicked() const
{
	TSharedPtr<FStructOnScope> StructToDisplay = MakeShareable(new FStructOnScope(FIKRigHierarchyImportSettings::StaticStruct(), (uint8*)&ImportSettings));

	TSharedRef<SKismetInspector> KismetInspector = SNew(SKismetInspector);
	KismetInspector->ShowSingleStruct(StructToDisplay);

	SGenericDialogWidget::FArguments DialogArguments;
	DialogArguments.OnOkPressed_Lambda([this] ()
	{
		if (ImportSettings.Mesh == nullptr)
		{
			return;
		}

		FIKRigEditorController* Controller = EditorController.Pin().Get();
		if (!Controller)
		{
			return;
		}

		Controller->AssetController->SetSkeletalMesh(ImportSettings.Mesh);
	});

	constexpr bool bAsModalDialog = false;
	SGenericDialogWidget::OpenDialog(LOCTEXT("IKRigHierarchyImport", "Import Hierarchy"), KismetInspector, DialogArguments, bAsModalDialog);

	return FReply::Handled();
}

void SIKRigHierarchy::BindCommands()
{
	const FIKRigSkeletonCommands& Commands = FIKRigSkeletonCommands::Get();

	CommandList->MapAction(Commands.NewGoal,
        FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleNewGoal),
        FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanAddNewGoal));
	
	CommandList->MapAction(Commands.DeleteElement,
        FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleDeleteElements),
        FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanDeleteElement));

	CommandList->MapAction(Commands.ConnectGoalToSolver,
        FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleConnectGoalToSolver),
        FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanConnectGoalToSolvers));

	CommandList->MapAction(Commands.DisconnectGoalFromSolver,
        FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleDisconnectGoalFromSolver),
        FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanDisconnectGoalFromSolvers));

	CommandList->MapAction(Commands.SetStartBoneOnSolver,
        FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleSetRootBoneOnSolvers),
        FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanSetRootBoneOnSolvers));

	CommandList->MapAction(Commands.SetEndBoneOnSolver,
	FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleSetEndBoneOnSolvers),
	FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanSetEndBoneOnSolvers),
	FIsActionChecked(),
	FIsActionButtonVisible::CreateSP(this, &SIKRigHierarchy::HasEndBoneCompatibleSolverSelected));

	CommandList->MapAction(Commands.AddBoneSettings,
        FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleAddBoneSettings),
        FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanAddBoneSettings));

	CommandList->MapAction(Commands.RemoveBoneSettings,
        FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleRemoveBoneSettings),
        FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanRemoveBoneSettings));

	CommandList->MapAction(Commands.ExcludeBone,
		FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleExcludeBone),
		FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanExcludeBone));

	CommandList->MapAction(Commands.IncludeBone,
		FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleIncludeBone),
		FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanIncludeBone));

	CommandList->MapAction(Commands.NewRetargetChain,
		FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleNewRetargetChain),
		FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanAddNewRetargetChain));

	CommandList->MapAction(Commands.SetPelvis,
		FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleSetRetargetRoot),
		FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanSetPelvis));

	CommandList->MapAction(Commands.ClearPelvis,
		FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleClearPelvis),
		FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanClearPelvis));

	CommandList->MapAction(Commands.RenameGoal,
	FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleRenameGoal),
		FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanRenameGoal));

	CommandList->MapAction(Commands.RefreshHierarchy,
	FExecuteAction::CreateSP(this, &SIKRigHierarchy::HandleRefreshHierarchy),
		FCanExecuteAction::CreateSP(this, &SIKRigHierarchy::CanRefreshHierarchy));

	
}

void SIKRigHierarchy::FillContextMenu(FMenuBuilder& MenuBuilder)
{
	const FIKRigSkeletonCommands& Actions = FIKRigSkeletonCommands::Get();

	const TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		MenuBuilder.AddWidget(
			SNew(SBox)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock).Text(LOCTEXT("InvalidSelection", "Nothing selected."))
			], FText(), false);
	}

	MenuBuilder.BeginSection("AddRemoveGoals", LOCTEXT("AddRemoveGoalOperations", "Goals"));
	MenuBuilder.AddMenuEntry(Actions.NewGoal);
	MenuBuilder.EndSection();
	
	MenuBuilder.BeginSection("ConnectGoals", LOCTEXT("ConnectGoalOperations", "Connect Goals To Solvers"));
	MenuBuilder.AddMenuEntry(Actions.ConnectGoalToSolver);
	MenuBuilder.AddMenuEntry(Actions.DisconnectGoalFromSolver);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("BoneSettings", LOCTEXT("BoneSettingsOperations", "Bone Settings"));
	MenuBuilder.AddMenuEntry(Actions.SetStartBoneOnSolver);
	MenuBuilder.AddMenuEntry(Actions.SetEndBoneOnSolver);
	MenuBuilder.AddMenuEntry(Actions.AddBoneSettings);
	MenuBuilder.AddMenuEntry(Actions.RemoveBoneSettings);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("IncludeExclude", LOCTEXT("IncludeExcludeOperations", "Exclude Bones"));
	MenuBuilder.AddMenuEntry(Actions.ExcludeBone);
	MenuBuilder.AddMenuEntry(Actions.IncludeBone);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Retargeting", LOCTEXT("RetargetingOperations", "Retargeting"));
	MenuBuilder.AddMenuEntry(Actions.SetPelvis);
	MenuBuilder.AddMenuEntry(Actions.ClearPelvis);
	MenuBuilder.AddMenuEntry(Actions.NewRetargetChain);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Hierarchy", LOCTEXT("HierarchyOperations", "Hierarchy"));
	MenuBuilder.AddMenuEntry(Actions.RefreshHierarchy);
	MenuBuilder.EndSection();
}

void SIKRigHierarchy::HandleNewGoal() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	// get names of selected bones and default goal names for them
	TArray<FName> GoalNames;
	TArray<FName> BoneNames;
	const TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = TreeView.Get()->GetSelectedItems();
	for (const TSharedPtr<FIKRigTreeElement>& Item : SelectedItems)
	{
		if (Item->ElementType != EIKRigTreeElementType::BONE)
		{
			continue; // can only add goals to bones
		}

		// build default name for the new goal
		const FName BoneName = Item->BoneName;
		const FName NewGoalName = FName(BoneName.ToString() + "_Goal");
		
		GoalNames.Add(NewGoalName);
		BoneNames.Add(BoneName);
	}
	
	// add new goals
	Controller->AddNewGoals(GoalNames, BoneNames);
}

bool SIKRigHierarchy::CanAddNewGoal() const
{
	// is anything selected?
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return false;
	}

	// can only add goals to selected bones
	for (const TSharedPtr<FIKRigTreeElement>& Item : SelectedItems)
	{
		if (Item->ElementType != EIKRigTreeElementType::BONE)
		{
			return false;
		}
	}

	return true;
}

void SIKRigHierarchy::HandleDeleteElements()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	Controller->HandleDeleteSelectedElements();
}
bool SIKRigHierarchy::CanDeleteElement() const
{
	// is anything selected?
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return false;
	}

	// are all selected items goals, effectors or bone settings?
	for (const TSharedPtr<FIKRigTreeElement>& Item : SelectedItems)
	{
		const bool bIsBone = Item->ElementType == EIKRigTreeElementType::BONE;
		if (bIsBone)
		{
			return false;
		}
	}

	return true;
}

void SIKRigHierarchy::HandleConnectGoalToSolver()
{
	const bool bConnect = true; //connect
	ConnectSelectedGoalsToSelectedSolvers(bConnect);
}

void SIKRigHierarchy::HandleDisconnectGoalFromSolver()
{
	const bool bConnect = false; //disconnect
	ConnectSelectedGoalsToSelectedSolvers(bConnect);
}

bool SIKRigHierarchy::CanConnectGoalToSolvers() const
{
	const bool bCountOnlyConnected = false;
	const int32 NumDisconnected = GetNumSelectedGoalToSolverConnections(bCountOnlyConnected);
	return NumDisconnected > 0;
}

bool SIKRigHierarchy::CanDisconnectGoalFromSolvers() const
{
	const bool bCountOnlyConnected = true;
	const int32 NumConnected = GetNumSelectedGoalToSolverConnections(bCountOnlyConnected);
	return NumConnected > 0;
}

void SIKRigHierarchy::ConnectSelectedGoalsToSelectedSolvers(bool bConnect)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedGoals;
	GetSelectedGoals(SelectedGoals);
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);

	FScopedTransaction Transaction(LOCTEXT("ConnectGoalsToSolver_Label", "Connect/Disconnect Goal(s) to Solver"));
	FScopedReinitializeIKRig Reinitialize(Controller->AssetController);
	
	for (const TSharedPtr<FIKRigTreeElement>& GoalElement : SelectedGoals)
	{
		const FName GoalName = GoalElement->GoalName;
		const UIKRigEffectorGoal* Goal = Controller->AssetController->GetGoal(GoalName);
		check(Goal);
		for (const TSharedPtr<FSolverStackElement>& SolverElement : SelectedSolvers)
		{
			if (bConnect)
			{
				Controller->AssetController->ConnectGoalToSolver(Goal->GoalName, SolverElement->IndexInStack);	
			}
			else
			{
				Controller->AssetController->DisconnectGoalFromSolver(Goal->GoalName, SolverElement->IndexInStack);	
			}
		}
	}
}

int32 SIKRigHierarchy::GetNumSelectedGoalToSolverConnections(bool bCountOnlyConnected) const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return 0;
	}
	
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedGoals;
	GetSelectedGoals(SelectedGoals);
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);

	int32 NumMatched = 0;
	for (const TSharedPtr<FIKRigTreeElement>& Goal : SelectedGoals)
	{
		for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
		{
			const bool bIsConnected = Controller->AssetController->IsGoalConnectedToSolver(Goal->GoalName, Solver->IndexInStack);
			if (bIsConnected == bCountOnlyConnected)
			{
				++NumMatched;
			}
		}
	}

	return NumMatched;
}

void SIKRigHierarchy::HandleSetRootBoneOnSolvers()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

    // get name of selected root bone
    TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	const FName RootBoneName = SelectedBones[0]->BoneName;

	// apply to all selected solvers (ignored on solvers that don't accept a root bone)
	UIKRigController* AssetController = Controller->AssetController;
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);
	TArray<int32> SolversToShow;
	for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
	{
		if (AssetController->SetStartBone(RootBoneName, Solver->IndexInStack))
		{
			SolversToShow.Add(Solver->IndexInStack);
		}
	}

	// show solver that had it's root bone updated
	Controller->ShowDetailsForSolvers(SolversToShow);
	
	// show new icon when bone has settings applied
	RefreshTreeView();
}

bool SIKRigHierarchy::CanSetRootBoneOnSolvers()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	
	// must have at least 1 bone selected
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	if (SelectedBones.Num() != 1)
	{
		return false;
	}

	// must have at least 1 solver selected that accepts root bones
	UIKRigController* AssetController = Controller->AssetController;
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);
	for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
	{
		if (AssetController->GetSolverAtIndex(Solver->IndexInStack)->UsesStartBone())
		{
			return true;
		}
	}
	
	return false;
}

void SIKRigHierarchy::HandleSetEndBoneOnSolvers()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

    // get name of selected root bone
    TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	const FName RootBoneName = SelectedBones[0]->BoneName;

	// apply to all selected solvers (ignored on solvers that don't accept a root bone)
	UIKRigController* AssetController = Controller->AssetController;
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);
	TArray<int32> SolversToShow;
	for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
	{
		if (AssetController->SetEndBone(RootBoneName, Solver->IndexInStack))
		{
			SolversToShow.Add(Solver->IndexInStack);
		}
	}

	// show solver that had it's root bone updated
	Controller->ShowDetailsForSolvers(SolversToShow);
	
	// show new icon when bone has settings applied
	RefreshTreeView();
}

bool SIKRigHierarchy::CanSetEndBoneOnSolvers() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	
	// must have at least 1 bone selected
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	if (SelectedBones.Num() != 1)
	{
		return false;
	}

	// must have at least 1 solver selected that accepts end bones
	UIKRigController* AssetController = Controller->AssetController;
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);
	for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
	{
		if (AssetController->GetSolverAtIndex(Solver->IndexInStack)->UsesEndBone())
		{
			return true;
		}
	}
	
	return false;
}

bool SIKRigHierarchy::HasEndBoneCompatibleSolverSelected() const
{
	return CanSetEndBoneOnSolvers();
}

void SIKRigHierarchy::HandleAddBoneSettings()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	// get selected bones
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	
	// add settings for bone on all selected solvers (ignored if already present)
	UIKRigController* AssetController = Controller->AssetController;
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBones)
	{
		for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
		{
			AssetController->AddBoneSetting(BoneItem->BoneName, Solver->IndexInStack);
        }
	}

	Controller->ShowDetailsForBone(SelectedBones[0]->BoneName);

	// show new icon when bone has settings applied
	RefreshTreeView();
}

bool SIKRigHierarchy::CanAddBoneSettings()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	
    // must have at least 1 bone selected
    TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	if (SelectedBones.IsEmpty())
	{
		return false;
	}

	// must have at least 1 solver selected that does not already have a bone setting for the selected bones
	UIKRigController* AssetController = Controller->AssetController;
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBones)
	{
		for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
		{
			if (AssetController->CanAddBoneSetting(BoneItem->BoneName, Solver->IndexInStack))
			{
				return true;
			}
        }
	}
	
	return false;
}

void SIKRigHierarchy::HandleRemoveBoneSettings()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	// get selected bones
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);

	FScopedTransaction Transaction(LOCTEXT("RemoveBoneSettings_Label", "Remove Bone Settings"));
	FScopedReinitializeIKRig Reinitialize(Controller->AssetController);
	
	// remove settings for bone on all selected solvers
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);
	FName BoneToShowInDetailsView;
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBones)
	{
		for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
		{
			Controller->AssetController->RemoveBoneSetting(BoneItem->BoneName, Solver->IndexInStack);
			BoneToShowInDetailsView = BoneItem->BoneName;
		}
	}
	
	Controller->ShowDetailsForBone(BoneToShowInDetailsView);
}

bool SIKRigHierarchy::CanRemoveBoneSettings()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	
    // must have at least 1 bone selected
    TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	if (SelectedBones.IsEmpty())
	{
		return false;
	}

	// must have at least 1 solver selected that has a bone setting for 1 of the selected bones
	UIKRigController* AssetController = Controller->AssetController;
	TArray<TSharedPtr<FSolverStackElement>> SelectedSolvers;
	Controller->GetSelectedSolvers(SelectedSolvers);
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBones)
	{
		for (const TSharedPtr<FSolverStackElement>& Solver : SelectedSolvers)
		{
			if (AssetController->CanRemoveBoneSetting(BoneItem->BoneName, Solver->IndexInStack))
			{
				return true;
			}
		}
	}
	
	return false;
}

void SIKRigHierarchy::HandleExcludeBone()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	// exclude selected bones
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBones)
	{
		Controller->AssetController->SetBoneExcluded(BoneItem->BoneName, true);
	}

	// show greyed out bone name after being excluded
	RefreshTreeView();
}

bool SIKRigHierarchy::CanExcludeBone()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	
	// must have at least 1 bone selected that is INCLUDED
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBones)
	{
		if (!Controller->AssetController->GetBoneExcluded(BoneItem->BoneName))
		{
			return true;
		}
	}

	return false;
}

void SIKRigHierarchy::HandleIncludeBone()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	// exclude selected bones
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBones)
	{
		Controller->AssetController->SetBoneExcluded(BoneItem->BoneName, false);
	}

	// show normal bone name after being included
	RefreshTreeView();
}

bool SIKRigHierarchy::CanIncludeBone()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	
	// must have at least 1 bone selected that is EXCLUDED
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	for (const TSharedPtr<FIKRigTreeElement>& BoneItem : SelectedBones)
	{
		if (Controller->AssetController->GetBoneExcluded(BoneItem->BoneName))
		{
			return true;
		}
	}

	return false;
}

void SIKRigHierarchy::HandleNewRetargetChain()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

	Controller->CreateNewRetargetChains();
}

bool SIKRigHierarchy::CanAddNewRetargetChain()
{
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	return !SelectedBones.IsEmpty();
}

void SIKRigHierarchy::HandleSetRetargetRoot()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	// get selected bones
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);

	// must have at least 1 bone selected
	if (SelectedBones.IsEmpty())
	{
		return;
	}

	// set the first selected bone as the pelvis
	Controller->AssetController->SetRetargetRoot(SelectedBones[0]->BoneName);
}

bool SIKRigHierarchy::CanSetPelvis()
{
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedBones;
	GetSelectedBones(SelectedBones);
	return !SelectedBones.IsEmpty();
}

void SIKRigHierarchy::HandleClearPelvis()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	Controller->AssetController->SetRetargetRoot(NAME_None);
}

bool SIKRigHierarchy::CanClearPelvis()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}

	return Controller->AssetController->GetRetargetRoot() != NAME_None;
}

bool SIKRigHierarchy::CanRefreshHierarchy()
{
	FIKRigEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return false;
	}

	return Controller->AssetController->GetSkeletalMesh() != nullptr;
}

void SIKRigHierarchy::HandleRefreshHierarchy()
{
	FIKRigEditorController* Controller = EditorController.Pin().Get();
	if (!Controller)
	{
		return;
	}

	if (USkeletalMesh* SkeletalMesh = Controller->AssetController->GetSkeletalMesh())
	{
		Controller->AssetController->SetSkeletalMesh(SkeletalMesh);
	}
}

bool SIKRigHierarchy::IsBoneInSelection(TArray<TSharedPtr<FIKRigTreeElement>>& SelectedBoneItems, const FName& BoneName)
{
	for (const TSharedPtr<FIKRigTreeElement>& Item : SelectedBoneItems)
	{
		if (Item->BoneName == BoneName)
		{
			return true;
		}
	}
	return false;
}

void SIKRigHierarchy::GetSelectedBones(TArray<TSharedPtr<FIKRigTreeElement>>& OutBoneItems) const
{
	const TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	for (const TSharedPtr<FIKRigTreeElement>& Item : SelectedItems)
	{
		if (Item->ElementType == EIKRigTreeElementType::BONE)
		{
			OutBoneItems.Add(Item);
        }
	}
}

void SIKRigHierarchy::GetSelectedBoneNames(TArray<FName>& OutSelectedBoneNames) const
{
	TArray<TSharedPtr<FIKRigTreeElement>> OutSelectedBones;
	GetSelectedBones(OutSelectedBones);
	OutSelectedBoneNames.Reset();
	for (TSharedPtr<FIKRigTreeElement> SelectedBoneItem : OutSelectedBones)
	{
		OutSelectedBoneNames.Add(SelectedBoneItem->BoneName);
	}
}

void SIKRigHierarchy::GetSelectedGoals(TArray<TSharedPtr<FIKRigTreeElement>>& OutSelectedGoals) const
{
	OutSelectedGoals.Reset();
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	for (const TSharedPtr<FIKRigTreeElement>& Item : SelectedItems)
	{
		if (Item->ElementType == EIKRigTreeElementType::GOAL)
		{
			OutSelectedGoals.Add(Item);
		}
	}
}

int32 SIKRigHierarchy::GetNumSelectedGoals()
{
	TArray<TSharedPtr<FIKRigTreeElement>> OutSelectedGoals;
	GetSelectedGoals(OutSelectedGoals);
	return OutSelectedGoals.Num();
}

void SIKRigHierarchy::GetSelectedGoalNames(TArray<FName>& OutSelectedGoalNames) const
{
	TArray<TSharedPtr<FIKRigTreeElement>> OutSelectedGoals;
	GetSelectedGoals(OutSelectedGoals);
	OutSelectedGoalNames.Reset();
	for (TSharedPtr<FIKRigTreeElement> SelectedGoalItem : OutSelectedGoals)
	{
		OutSelectedGoalNames.Add(SelectedGoalItem->GoalName);
	}
}

bool SIKRigHierarchy::IsGoalSelected(const FName& GoalName)
{
	TArray<TSharedPtr<FIKRigTreeElement>> OutSelectedGoals;
	GetSelectedGoals(OutSelectedGoals);
	for (TSharedPtr<FIKRigTreeElement> SelectedGoalItem : OutSelectedGoals)
	{
		if (SelectedGoalItem->GoalName == GoalName)
		{
			return true;
		}
	}
	return false;
}

void SIKRigHierarchy::HandleRenameGoal() const
{
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedGoals;
	GetSelectedGoals(SelectedGoals);
	if (SelectedGoals.Num() != 1)
	{
		return;
	}
	
	SelectedGoals[0]->RequestRename();
}

bool SIKRigHierarchy::CanRenameGoal() const
{
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedGoals;
	GetSelectedGoals(SelectedGoals);
	return SelectedGoals.Num() == 1;
}

TSharedRef<SWidget> SIKRigHierarchy::CreateFilterMenuWidget()
{
	const FUIAction FilterShowAllAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			FilterOptions.ResetShowOptions();
			FilterOptions.bShowAll = true; 
			RefreshTreeView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return FilterOptions.bShowAll;
		}));
	
	const FUIAction FilterOnlyBonesAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			FilterOptions.ResetShowOptions();
			FilterOptions.bShowOnlyBones = true;
			RefreshTreeView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return FilterOptions.bShowOnlyBones;
		}));

	const FUIAction FilterOnlyGoalsAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			FilterOptions.ResetShowOptions();
			FilterOptions.bShowOnlyGoals = true;
			RefreshTreeView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return FilterOptions.bShowOnlyGoals;
		}));

	const FUIAction FilterOnlyBoneSettingsAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			FilterOptions.ResetShowOptions();
			FilterOptions.bShowOnlyBoneSettings = true;
			RefreshTreeView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return FilterOptions.bShowOnlyBoneSettings;
		}));

	const FUIAction FilterUnaffectedBonesAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			FilterOptions.bHideUnaffectedBones = !FilterOptions.bHideUnaffectedBones;
			RefreshTreeView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return FilterOptions.bHideUnaffectedBones;
		}));
	
	static constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);

	MenuBuilder.BeginSection("Hierarchy Filters", LOCTEXT("HierarchyFiltersSection", "Hierarchy Filters"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ShowAllTypesLabel", "Show All Types"),
		LOCTEXT("ShowAllTypesTooltip", "Show all object types in the hierarchy."),
		FSlateIcon(),
		FilterShowAllAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OnlyBonesLabel", "Show Only Bones"),
		LOCTEXT("OnlyBonesTooltip", "Hide everything except bones."),
		FSlateIcon(),
		FilterOnlyBonesAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OnlyGoalsLabel", "Show Only Goals"),
		LOCTEXT("OnlyGoalsTooltip", "Hide everything except IK goals."),
		FSlateIcon(),
		FilterOnlyGoalsAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("OnlyBoneSettingsLabel", "Show Only Bone Settings"),
		LOCTEXT("OnlyBoneSettingsTooltip", "Hide everything except bone settings."),
		FSlateIcon(),
		FilterOnlyBoneSettingsAction,
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("UnaffectedBonesLabel", "Hide Unaffected Bones"),
		LOCTEXT("UnaffectedBonesTooltip", "Hide all bones that are unaffected by any solvers."),
		FSlateIcon(),
		FilterUnaffectedBonesAction,
		NAME_None,
		EUserInterfaceActionType::Check);
	
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Clear", LOCTEXT("ClearMapFiltersSection", "Clear"));
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearSkeletonFilterLabel", "Clear Filters"),
		LOCTEXT("ClearSkeletonFilterTooltip", "Clear all filters to show all hierarchy elements."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]
		{
			FilterOptions = FIKRigHierarchyFilterOptions();
			RefreshTreeView();
		})));

	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

void SIKRigHierarchy::OnFilterTextChanged(const FText& SearchText)
{
	TextFilter->SetFilterText(SearchText);
	RefreshTreeView();
}

void SIKRigHierarchy::RefreshTreeView(bool IsInitialSetup /*=false*/)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

	const TSharedRef<FIKRigEditorController> ControllerRef = Controller.ToSharedRef();
	
	// save expansion and selection state
	TreeView->SaveAndClearState();

	// reset all tree items
	RootElements.Reset();
	AllElements.Reset();

	// validate we have a skeleton to load
	UIKRigController* AssetController = Controller->AssetController;
	const FIKRigSkeleton& Skeleton = AssetController->GetIKRigSkeleton();
	if (Skeleton.BoneNames.IsEmpty())
	{
		TreeView->RequestTreeRefresh();
		return;
	}

	// get all goals
	const TArray<UIKRigEffectorGoal*>& Goals = AssetController->GetAllGoals();
	
	// get all solvers
	const TArray<FIKRigSolverBase*> Solvers = AssetController->GetSolverArray();
	// record bone element indices
	TMap<FName, int32> BoneTreeElementIndices;

	auto FilterString = [this](const FString& StringToTest) ->bool
	{
		return TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(StringToTest));
	};

	// create all bone elements
	for (int32 BoneIndex=0; BoneIndex<Skeleton.BoneNames.Num(); ++BoneIndex)
	{
		// create "Bone" tree element for this bone
		const FName BoneName = Skeleton.BoneNames[BoneIndex];
		const FText BoneDisplayName = FText::FromName(BoneName);
		TSharedPtr<FIKRigTreeElement> BoneElement = MakeShared<FIKRigTreeElement>(BoneDisplayName, EIKRigTreeElementType::BONE, ControllerRef);
		BoneElement.Get()->BoneName = BoneName;
		const int32 BoneElementIndex = AllElements.Add(BoneElement);
		BoneTreeElementIndices.Add(BoneName, BoneElementIndex);

		// store pointer to parent (if there is one)
		const int32 ParentIndex = Skeleton.ParentIndices[BoneIndex];
		if (ParentIndex != INDEX_NONE)
		{
			// get parent tree element
			const FName ParentBoneName = Skeleton.BoneNames[ParentIndex];
			const TSharedPtr<FIKRigTreeElement> ParentBoneTreeElement = AllElements[BoneTreeElementIndices[ParentBoneName]];
			BoneElement->UnFilteredParent = ParentBoneTreeElement;
		}

		// apply type filter to bones
		if (FilterOptions.bShowOnlyBoneSettings || FilterOptions.bShowOnlyGoals)
		{
			BoneElement->bIsHidden = true;
		}

		// apply filter to hide unaffected bones
		if (FilterOptions.bHideUnaffectedBones && !IsElementConnectedToAnySolver(BoneElement))
		{
			BoneElement->bIsHidden = true;	
		}
		
		// apply text filter to bones
		if (!(TextFilter->GetFilterText().IsEmpty() || FilterString(BoneName.ToString())))
		{
			BoneElement->bIsHidden = true;
		}

		// create all "Bone Setting" tree elements for this bone
		for (int32 SolverIndex=0; SolverIndex<Solvers.Num(); ++SolverIndex)
		{
			if (!Solvers[SolverIndex]->HasSettingsOnBone(BoneName))
			{
				continue;
			}
			
			const FText SolverDisplayName = FText::FromString(AssetController->GetSolverUniqueName(SolverIndex));
			const FText BoneSettingDisplayName = FText::Format(LOCTEXT("BoneSettings", "{0} bone settings."), SolverDisplayName);
			TSharedPtr<FIKRigTreeElement> SettingsItem = MakeShared<FIKRigTreeElement>(BoneSettingDisplayName, EIKRigTreeElementType::BONE_SETTINGS, ControllerRef);
			SettingsItem->BoneSettingBoneName = BoneName;
			SettingsItem->BoneSettingsSolverIndex = SolverIndex;
			AllElements.Add(SettingsItem);
			
			// store hierarchy pointer for item
			SettingsItem->UnFilteredParent = BoneElement;

			// apply type filter to bone settings
			if (FilterOptions.bShowOnlyBones || FilterOptions.bShowOnlyGoals)
			{
				SettingsItem->bIsHidden = true;
			}

			// apply text filter to bone settings
			if (!(TextFilter->GetFilterText().IsEmpty() || FilterString(BoneName.ToString())))
			{
				SettingsItem->bIsHidden = true;
			}
		}

		// create all "Goal" and "Effector" tree elements for this bone
		for (const UIKRigEffectorGoal* Goal : Goals)
		{
			if (Goal->BoneName != BoneName)
			{
				continue;
			}
			
			// make new element for goal
			const FText GoalDisplayName = FText::FromName(Goal->GoalName);
			TSharedPtr<FIKRigTreeElement> GoalItem = MakeShared<FIKRigTreeElement>(GoalDisplayName, EIKRigTreeElementType::GOAL, ControllerRef);
			GoalItem->GoalName = Goal->GoalName;
			AllElements.Add(GoalItem);

			// apply type filter to goals
			if (FilterOptions.bShowOnlyBones || FilterOptions.bShowOnlyBoneSettings)
			{
				GoalItem->bIsHidden = true;
			}

			// apply text filter to goals
			if (!(TextFilter->GetFilterText().IsEmpty() ||
				FilterString(Goal->GoalName.ToString()) ||
				FilterString(Goal->BoneName.ToString())))
			{
				GoalItem->bIsHidden = true;
			}

			// store hierarchy pointer for goal
			GoalItem->UnFilteredParent = BoneElement;

			// add all solver settings connected to this goal
			for (int32 SolverIndex=0; SolverIndex<Solvers.Num(); ++SolverIndex)
			{
				if (!AssetController->DoesSolverHaveCustomGoalSettings(Goal->GoalName, SolverIndex))
				{
					continue;
				}
				
				// make new element for effector
				const FText SolverDisplayName = FText::FromString(AssetController->GetSolverUniqueName(SolverIndex));
				const FText EffectorDisplayName = FText::Format(LOCTEXT("GoalSettingsForSolver", "{0} goal settings."), SolverDisplayName);
				TSharedPtr<FIKRigTreeElement> EffectorItem = MakeShared<FIKRigTreeElement>(EffectorDisplayName, EIKRigTreeElementType::GOAL_SETTINGS, ControllerRef);
				EffectorItem->EffectorIndex = SolverIndex;
				EffectorItem->EffectorGoalName = Goal->GoalName;
				AllElements.Add(EffectorItem);

				// apply filter to effectors (treated as goals for filtering purposes)
				EffectorItem->bIsHidden = GoalItem->bIsHidden;
				
				// store hierarchy pointer for effectors
				EffectorItem->UnFilteredParent = GoalItem;
			}
		}
	}

	// resolve parent/children pointers on all tree elements, taking into consideration the filter options
	// (elements are parented to their nearest non-hidden/filtered parent element)
	for (TSharedPtr<FIKRigTreeElement>& Element : AllElements)
	{
		if (Element->bIsHidden)
		{
			continue;
		}
		
		// find first parent that is not filtered
		TSharedPtr<FIKRigTreeElement> ParentElement = Element->UnFilteredParent;
		while (true)
		{
			if (!ParentElement.IsValid())
			{
				break;
			}

			if (!ParentElement->bIsHidden)
			{
				break;
			}

			ParentElement = ParentElement->UnFilteredParent;
		}

		if (ParentElement.IsValid())
		{
			// store pointer to child on parent
			ParentElement->Children.Add(Element);
			// store pointer to parent on child
			Element->Parent = ParentElement;
		}
		else
		{
			// has no parent, store a root element
			RootElements.Add(Element);
		}
	}

	// expand all elements upon the initial construction of the tree
	if (IsInitialSetup)
	{
		for (TSharedPtr<FIKRigTreeElement> RootElement : RootElements)
		{
			SetExpansionRecursive(RootElement, false, true);
		}
	}
	else
	{
		// restore expansion and selection state
		for (const TSharedPtr<FIKRigTreeElement>& Element : AllElements)
		{
			TreeView->RestoreState(Element);
		}
	}
	
	TreeView->RequestTreeRefresh();
}

void SIKRigHierarchy::HandleGetChildrenForTree(
	TSharedPtr<FIKRigTreeElement> InItem,
	TArray<TSharedPtr<FIKRigTreeElement>>& OutChildren)
{
	OutChildren = InItem.Get()->Children;
}

void SIKRigHierarchy::OnSelectionChanged(TSharedPtr<FIKRigTreeElement> Selection, ESelectInfo::Type SelectInfo)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

	// update details view — reorder so the last clicked item is first, ensuring
	// ShowDetailsForElements pivots on the correct type when a mixed selection is made
	TArray<TSharedPtr<FIKRigTreeElement>> SelectedItems = TreeView->GetSelectedItems();
	if (const TSharedPtr<FIKRigTreeElement> LastSelected = TreeView->LastSelected.Pin())
	{
		const int32 LastSelectedIndex = SelectedItems.IndexOfByPredicate(
			[&LastSelected](const TSharedPtr<FIKRigTreeElement>& Item) { return Item == LastSelected; });
		if (LastSelectedIndex > 0)
		{
			SelectedItems.Swap(0, LastSelectedIndex);
		}
	}
	Controller->ShowDetailsForElements(SelectedItems);
}

TSharedRef<SWidget> SIKRigHierarchy::CreateAddNewMenu()
{
	constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);
	FillContextMenu(MenuBuilder);
	return MenuBuilder.MakeWidget();
}

TSharedPtr<SWidget> SIKRigHierarchy::CreateContextMenu()
{
	constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);
	FillContextMenu(MenuBuilder);
	return MenuBuilder.MakeWidget();
}

void SIKRigHierarchy::OnItemClicked(TSharedPtr<FIKRigTreeElement> InItem)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	// to rename an item, you have to select it first, then click on it again within a time limit (slow double click)
	const bool ClickedOnSameItem = TreeView->LastSelected.Pin().Get() == InItem.Get();
	const uint32 CurrentCycles = FPlatformTime::Cycles();
	const double SecondsPassed = static_cast<double>(CurrentCycles - TreeView->LastClickCycles) * FPlatformTime::GetSecondsPerCycle();
	if (ClickedOnSameItem && SecondsPassed > 0.25f)
	{
		RegisterActiveTimer(0.f,
            FWidgetActiveTimerDelegate::CreateLambda([this](double, float) {
            HandleRenameGoal();
            return EActiveTimerReturnType::Stop;
        }));
	}

	TreeView->LastClickCycles = CurrentCycles;
	TreeView->LastSelected = InItem;
	Controller->SetLastSelectedType(EIKRigSelectionType::Hierarchy);
}

void SIKRigHierarchy::OnItemDoubleClicked(TSharedPtr<FIKRigTreeElement> InItem)
{
	if (TreeView->IsItemExpanded(InItem))
	{
		SetExpansionRecursive(InItem, false, false);
	}
	else
	{
		SetExpansionRecursive(InItem, false, true);
	}
}

void SIKRigHierarchy::OnSetExpansionRecursive(TSharedPtr<FIKRigTreeElement> InItem, bool bShouldBeExpanded)
{
	SetExpansionRecursive(InItem, false, bShouldBeExpanded);
}

void SIKRigHierarchy::SetExpansionRecursive(
	TSharedPtr<FIKRigTreeElement> InElement,
	bool bTowardsParent,
    bool bShouldBeExpanded)
{
	TreeView->SetItemExpansion(InElement, bShouldBeExpanded);
    
    if (bTowardsParent)
    {
    	if (InElement->Parent.Get())
    	{
    		SetExpansionRecursive(InElement->Parent, bTowardsParent, bShouldBeExpanded);
    	}
    }
    else
    {
    	for (int32 ChildIndex = 0; ChildIndex < InElement->Children.Num(); ++ChildIndex)
    	{
    		SetExpansionRecursive(InElement->Children[ChildIndex], bTowardsParent, bShouldBeExpanded);
    	}
    }
}

FReply SIKRigHierarchy::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FKey Key = InKeyEvent.GetKey();

	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return FReply::Handled();
	}

	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
