// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/SIKRigOutputLog.h"

#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#include "RetargetEditor/IKRetargetEditorController.h"

void SIKRigOutputLog::Construct(
	const FArguments& InArgs,
	const FName& InLogName)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	// we have to ensure that the log is already registered at this point because otherwise GetLogListing will register
	// it without the proper initialization settings required to filter these logs out of the main Message Log UI
	ensure(MessageLogModule.IsRegisteredLogListing(InLogName));
	
	MessageLogListing = MessageLogModule.GetLogListing(InLogName);
	OutputLogWidget = MessageLogModule.CreateLogListingWidget( MessageLogListing.ToSharedRef() );
	
	ChildSlot
	[
		SNew(SBox)
		[
			OutputLogWidget.ToSharedRef()
		]
	];
}

void SIKRigOutputLog::ClearLog() const
{
	MessageLogListing.Get()->ClearMessages();
}