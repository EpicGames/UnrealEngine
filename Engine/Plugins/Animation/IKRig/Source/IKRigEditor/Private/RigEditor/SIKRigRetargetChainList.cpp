// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/SIKRigRetargetChainList.h"

#include "RigEditor/IKRigEditorStyle.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/IKRigEditorController.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IPersonaToolkit.h"
#include "SKismetInspector.h"
#include "Dialogs/Dialogs.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SPositiveActionButton.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "BoneSelectionWidget.h"
#include "Animation/MirrorDataTable.h"
#include "Engine/SkeletalMesh.h"
#include "Widgets/Input/SSearchBox.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SIKRigRetargetChainList)

#define LOCTEXT_NAMESPACE "SIKRigRetargetChains"

static const FName ColumnId_ChainNameLabel( "Chain Name" );
static const FName ColumnId_ChainStartLabel( "Start Bone" );
static const FName ColumnId_ChainEndLabel( "End Bone" );
static const FName ColumnId_IKGoalLabel( "IK Goal" );
static const FName ColumnId_DeleteChainLabel( "Delete Chain" );

TSharedRef<ITableRow> FRetargetChainElement::MakeListRowWidget(
	const TSharedRef<STableViewBase>& InOwnerTable,
	TSharedRef<FRetargetChainElement> InChainElement,
	TSharedPtr<SIKRigRetargetChainList> InChainList)
{
	return SNew(SIKRigRetargetChainRow, InOwnerTable, InChainElement, InChainList);
}

void SIKRigRetargetChainRow::Construct(
	const FArguments& InArgs,
	const TSharedRef<STableViewBase>& InOwnerTableView,
	TSharedRef<FRetargetChainElement> InChainElement,
	TSharedPtr<SIKRigRetargetChainList> InChainList)
{
	ChainElement = InChainElement;
	ChainList = InChainList;

	// generate list of goals
	// NOTE: cannot just use literal "None" because Combobox considers that a "null" entry and will discard it from the list.
	GoalOptions.Add(MakeShareable(new FString("None")));
	const UIKRigDefinition* IKRigAsset = InChainList->EditorController.Pin()->AssetController->GetAsset();
	const TArray<UIKRigEffectorGoal*>& AssetGoals =  IKRigAsset->GetGoalArray();
	for (const UIKRigEffectorGoal* Goal : AssetGoals)
	{
		GoalOptions.Add(MakeShareable(new FString(Goal->GoalName.ToString())));
	}

	SMultiColumnTableRow< TSharedPtr<FRetargetChainElement> >::Construct( FSuperRowType::FArguments(), InOwnerTableView );
}

TSharedRef<SWidget> SIKRigRetargetChainRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (ColumnName == ColumnId_ChainNameLabel)
	{
		TSharedRef<SWidget> ChainWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(3.0f, 1.0f)
		[
			SNew(SEditableTextBox)
			.Text(FText::FromName(ChainElement.Pin()->ChainName))
			.Font(FAppStyle::GetFontStyle(TEXT("BoldFont")))
			.OnTextCommitted(this, &SIKRigRetargetChainRow::OnRenameChain)
		];
		return ChainWidget;
	}

	if (ColumnName == ColumnId_ChainStartLabel)
	{
		TSharedRef<SWidget> StartWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(3.0f, 1.0f)
		[
			SNew(SBoneSelectionWidget)
			.OnBoneSelectionChanged(this, &SIKRigRetargetChainRow::OnStartBoneComboSelectionChanged)
			.OnGetSelectedBone(this, &SIKRigRetargetChainRow::GetStartBoneName)
			.OnGetReferenceSkeleton(this, &SIKRigRetargetChainRow::GetReferenceSkeleton)
		];
		return StartWidget;
	}

	if (ColumnName == ColumnId_ChainEndLabel)
	{
		TSharedRef<SWidget> EndWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(3.0f, 1.0f)
		[
			SNew(SBoneSelectionWidget)
			.OnBoneSelectionChanged(this, &SIKRigRetargetChainRow::OnEndBoneComboSelectionChanged)
			.OnGetSelectedBone(this, &SIKRigRetargetChainRow::GetEndBoneName)
			.OnGetReferenceSkeleton(this, &SIKRigRetargetChainRow::GetReferenceSkeleton)
		];
		return EndWidget;
	}

	if (ColumnName == ColumnId_IKGoalLabel)
	{
		TSharedRef<SWidget> GoalWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(3.0f, 1.0f)
		[
			SNew(SSearchableComboBox)
			.OptionsSource(&GoalOptions)
			.OnGenerateWidget(this, &SIKRigRetargetChainRow::MakeGoalComboEntryWidget)
			.OnSelectionChanged(this, &SIKRigRetargetChainRow::OnGoalComboSelectionChanged)
			[
				SNew(STextBlock)
				.Text(this, &SIKRigRetargetChainRow::GetGoalName)
			]
		];
		return GoalWidget;
	}

	// ColumnName == ColumnId_DeleteChainLabel
	{
		TSharedRef<SWidget> DeleteWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(3)
		[
			SNew(SButton)
			.ToolTipText(LOCTEXT("DeleteChain", "Remove retarget bone chain from list."))
			.OnClicked_Lambda([this]() -> FReply
			{
				const TSharedPtr<FIKRigEditorController> Controller = ChainList.Pin()->EditorController.Pin();
				if (!Controller.IsValid())
				{
					return FReply::Unhandled();
				}

				UIKRigController* AssetController = Controller->AssetController;
				AssetController->RemoveRetargetChain(ChainElement.Pin()->ChainName);

				ChainList.Pin()->RefreshView();
				return FReply::Handled();
			})
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		];
		return DeleteWidget;
	}
}

TSharedRef<SWidget> SIKRigRetargetChainRow::MakeGoalComboEntryWidget(TSharedPtr<FString> InItem) const
{
	return SNew(STextBlock).Text(FText::FromString(*InItem.Get()));
}

void SIKRigRetargetChainRow::OnStartBoneComboSelectionChanged(FName InName) const
{
	const TSharedPtr<FIKRigEditorController> Controller = ChainList.Pin()->EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	Controller->AssetController->SetRetargetChainStartBone(ChainElement.Pin()->ChainName, InName);
	ChainList.Pin()->RefreshView();
}

FName SIKRigRetargetChainRow::GetStartBoneName(bool& bMultipleValues) const
{
	const TSharedPtr<FIKRigEditorController> Controller = ChainList.Pin()->EditorController.Pin();
	if (!Controller.IsValid())
	{
		return NAME_None;
	}

	bMultipleValues = false;
	return Controller->AssetController->GetRetargetChainStartBone(ChainElement.Pin()->ChainName);
}

FName SIKRigRetargetChainRow::GetEndBoneName(bool& bMultipleValues) const
{	
	const TSharedPtr<FIKRigEditorController> Controller = ChainList.Pin()->EditorController.Pin();
	if (!Controller.IsValid())
	{
		return NAME_None;
	}

	bMultipleValues = false;
	return Controller->AssetController->GetRetargetChainEndBone(ChainElement.Pin()->ChainName);
}

FText SIKRigRetargetChainRow::GetGoalName() const
{
	const TSharedPtr<FIKRigEditorController> Controller = ChainList.Pin()->EditorController.Pin();
	if (!Controller.IsValid())
	{
		return FText::GetEmpty();
	}
	
	const FName GoalName = Controller->AssetController->GetRetargetChainGoal(ChainElement.Pin()->ChainName);
	return FText::FromName(GoalName);
}

void SIKRigRetargetChainRow::OnEndBoneComboSelectionChanged(FName InName) const
{
	const TSharedPtr<FIKRigEditorController> Controller = ChainList.Pin()->EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	Controller->AssetController->SetRetargetChainEndBone(ChainElement.Pin()->ChainName, InName);
	ChainList.Pin()->RefreshView();
}

void SIKRigRetargetChainRow::OnGoalComboSelectionChanged(TSharedPtr<FString> InGoalName, ESelectInfo::Type SelectInfo)
{
	const TSharedPtr<FIKRigEditorController> Controller = ChainList.Pin()->EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	Controller->AssetController->SetRetargetChainGoal(ChainElement.Pin()->ChainName, FName(*InGoalName.Get()));
	ChainList.Pin()->RefreshView();
}

void SIKRigRetargetChainRow::OnRenameChain(const FText& InText, ETextCommit::Type CommitType) const
{
	const TSharedPtr<FIKRigEditorController> Controller = ChainList.Pin()->EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	// prevent setting name to the same name
	const FName OldName = ChainElement.Pin()->ChainName;
	const FName NewName = FName(*InText.ToString());
	if (OldName == NewName)
	{
		return;
	}

	// prevent multiple commits when pressing enter
	if (CommitType == ETextCommit::OnEnter)
	{
		return;
	}
	
	ChainElement.Pin()->ChainName = Controller->AssetController->RenameRetargetChain(OldName, NewName);
	ChainList.Pin()->RefreshView();
}

const FReferenceSkeleton& SIKRigRetargetChainRow::GetReferenceSkeleton() const
{
	if (ChainList.IsValid())
	{
		return ChainList.Pin()->GetReferenceSkeleton();	
	}
	
	static const FReferenceSkeleton DummySkeleton;
	return DummySkeleton;
}

void SIKRigRetargetChainList::Construct(const FArguments& InArgs, TSharedRef<FIKRigEditorController> InEditorController)
{
	EditorController = InEditorController;
	EditorController.Pin()->SetRetargetingView(SharedThis(this));

	TextFilter = MakeShareable(new FTextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::BasicString));
	
	CommandList = MakeShared<FUICommandList>();

	ChildSlot
    [
        SNew(SVerticalBox)
        +SVerticalBox::Slot()
        .AutoHeight()
        .VAlign(VAlign_Top)
        [
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 3.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PelvisLabel", "Pelvis:"))
				.TextStyle(FAppStyle::Get(), "NormalText")
			]
        	
        	+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 1.0f)
			[
				SNew(SBoneSelectionWidget)
				.OnBoneSelectionChanged(this, &SIKRigRetargetChainList::OnPelvisBoneComboSelectionChanged)
				.OnGetSelectedBone(this, &SIKRigRetargetChainList::GetPelvisBoneName)
				.OnGetReferenceSkeleton(this, &SIKRigRetargetChainList::GetReferenceSkeleton)
			]

        	+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 3.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RootLabel", "Root Motion:"))
				.TextStyle(FAppStyle::Get(), "NormalText")
			]
        	
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 1.0f)
			[
				SNew(SBoneSelectionWidget)
				.OnBoneSelectionChanged(this, &SIKRigRetargetChainList::OnRootBoneComboSelectionChanged)
				.OnGetSelectedBone(this, &SIKRigRetargetChainList::GetRootBoneName)
				.OnGetReferenceSkeleton(this, &SIKRigRetargetChainList::GetReferenceSkeleton)
			]
        ]

        +SVerticalBox::Slot()
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.f, 0.0))
			[
				SNew(SPositiveActionButton)
				.Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				.Text(LOCTEXT("AddNewChainLabel", "Add New Chain"))
				.ToolTipText(LOCTEXT("AddNewChainToolTip", "Add a new retarget bone chain."))
				.OnClicked_Lambda([this]()
				{
					EditorController.Pin()->CreateNewRetargetChains();
					return FReply::Handled();
				})
			]

			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSearchBox)
				.SelectAllTextWhenFocused(true)
				.OnTextChanged( this, &SIKRigRetargetChainList::OnFilterTextChanged )
				.HintText( LOCTEXT( "SearchBoxHint", "Filter Chain List...") )
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.f, 0.0))
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
				.ForegroundColor(FSlateColor::UseStyle())
				.ContentPadding(2.0f)
				.OnGetMenuContent(this, &SIKRigRetargetChainList::CreateFilterMenuWidget)
				.HasDownArrow(true)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]

        +SVerticalBox::Slot()
		[
			SAssignNew(ListView, SRetargetChainListViewType )
			.SelectionMode(ESelectionMode::Multi)
			.IsEnabled(this, &SIKRigRetargetChainList::IsAddChainEnabled)
			.ListItemsSource( &ListViewItems )
			.OnGenerateRow( this, &SIKRigRetargetChainList::MakeListRowWidget )
			.OnMouseButtonClick(this, &SIKRigRetargetChainList::OnItemClicked)
			.OnContextMenuOpening(this, &SIKRigRetargetChainList::CreateContextMenu)
			.HeaderRow
			(
				SNew( SHeaderRow )
				+ SHeaderRow::Column( ColumnId_ChainNameLabel )
				.DefaultLabel( LOCTEXT( "ChainNameColumnLabel", "Chain Name" ) )

				+ SHeaderRow::Column( ColumnId_ChainStartLabel )
				.DefaultLabel( LOCTEXT( "ChainStartColumnLabel", "Start Bone" ) )

				+ SHeaderRow::Column( ColumnId_ChainEndLabel )
				.DefaultLabel( LOCTEXT( "ChainEndColumnLabel", "End Bone" ) )

				+ SHeaderRow::Column( ColumnId_IKGoalLabel )
				.DefaultLabel( LOCTEXT( "IKGoalColumnLabel", "IK Goal" ) )
				
				+ SHeaderRow::Column( ColumnId_DeleteChainLabel )
				.DefaultLabel( LOCTEXT( "DeleteChainColumnLabel", "Delete Chain" ) )
			)
		]
    ];

	RefreshView();
}

TArray<FName> SIKRigRetargetChainList::GetSelectedChains() const
{
	TArray<TSharedPtr<FRetargetChainElement>> SelectedItems = ListView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return TArray<FName>();
	}

	TArray<FName> SelectedChainNames;
	for (const TSharedPtr<FRetargetChainElement>& SelectedItem : SelectedItems)
	{
		SelectedChainNames.Add(SelectedItem->ChainName);
	}
	
	return SelectedChainNames;
}

void SIKRigRetargetChainList::OnPelvisBoneComboSelectionChanged(FName InName)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	Controller->AssetController->SetRetargetRoot(InName);
	RefreshView();
}

void SIKRigRetargetChainList::OnRootBoneComboSelectionChanged(FName InName)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	Controller->AssetController->SetRootMotionBone(InName);
	RefreshView();
}

FName SIKRigRetargetChainList::GetPelvisBoneName(bool& bMultipleValues) const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return NAME_None; 
	}

	bMultipleValues = false;
	return Controller->AssetController->GetRetargetRoot();
}

FName SIKRigRetargetChainList::GetRootBoneName(bool& bMultipleValues) const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return NAME_None; 
	}

	bMultipleValues = false;
	return Controller->AssetController->GetRootMotionBone();
}

const FReferenceSkeleton& SIKRigRetargetChainList::GetReferenceSkeleton() const
{
	static const FReferenceSkeleton DummySkeleton;
	
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return DummySkeleton; 
	}

	USkeletalMesh* SkeletalMesh = Controller->AssetController->GetSkeletalMesh();
	if (SkeletalMesh == nullptr)
	{
		return DummySkeleton;
	}

	return SkeletalMesh->GetRefSkeleton();
}

void SIKRigRetargetChainList::OnFilterTextChanged(const FText& SearchText)
{
	TextFilter->SetFilterText(SearchText);
	RefreshView();
}

TSharedRef<SWidget> SIKRigRetargetChainList::CreateFilterMenuWidget()
{
	const FUIAction FilterSingleBoneAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			ChainFilterOptions.bHideSingleBoneChains = !ChainFilterOptions.bHideSingleBoneChains;
			RefreshView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return ChainFilterOptions.bHideSingleBoneChains;
		}));

	const FUIAction FilterIKChainAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			ChainFilterOptions.bShowOnlyIKChains = !ChainFilterOptions.bShowOnlyIKChains;
			RefreshView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return ChainFilterOptions.bShowOnlyIKChains;
		}));

	const FUIAction MissingBoneChainAction = FUIAction(
		FExecuteAction::CreateLambda([this]
		{
			ChainFilterOptions.bShowOnlyMissingBoneChains = !ChainFilterOptions.bShowOnlyMissingBoneChains;
			RefreshView();
		}),
		FCanExecuteAction(),
		FIsActionChecked::CreateLambda([this]()
		{
			return ChainFilterOptions.bShowOnlyMissingBoneChains;
		}));
	
	static constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);

	MenuBuilder.BeginSection("Chain Filters", LOCTEXT("ChainFiltersSection", "Filter"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SingleBoneLabel", "Hide Single Bone Chains"),
		LOCTEXT("SingleBoneTooltip", "Show only chains that contain multiple bones."),
		FSlateIcon(),
		FilterSingleBoneAction,
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("HasIKLabel", "Show Only IK Chains"),
		LOCTEXT("HasIKTooltip", "Show only chains that have an IK Goal."),
		FSlateIcon(),
		FilterIKChainAction,
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MissingBoneLabel", "Show Only Chains Missing Bones"),
		LOCTEXT("MissingBoneTooltip", "Show only chains that are missing either a Start or End bone."),
		FSlateIcon(),
		MissingBoneChainAction,
		NAME_None,
		EUserInterfaceActionType::Check);
	
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Clear", LOCTEXT("ClearFiltersSection", "Clear"));
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearFilterLabel", "Clear Filters"),
		LOCTEXT("ClearFilterTooltip", "Clear all filters to show all chains."),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateLambda([this]
		{
			ChainFilterOptions = FChainFilterOptions();
			RefreshView();
		})));

	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

bool SIKRigRetargetChainList::IsAddChainEnabled() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false;
	}
	
	if (UIKRigController* AssetController = Controller->AssetController)
	{
		if (AssetController->GetIKRigSkeleton().BoneNames.Num() > 0)
		{
			return true;
		}
	}
	
	return false;
}

void SIKRigRetargetChainList::RefreshView()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}

	auto FilterString = [this](const FString& StringToTest) ->bool
	{
		return TextFilter->TestTextFilter(FBasicStringFilterExpressionContext(StringToTest));
	};
	
	// refresh list of chains
	ListViewItems.Reset();
	const TArray<FBoneChain>& Chains = Controller->AssetController->GetRetargetChains();
	for (const FBoneChain& Chain : Chains)
	{
		// apply text filter to items
		if (!(TextFilter->GetFilterText().IsEmpty() ||
			FilterString(Chain.ChainName.ToString()) ||
			FilterString(Chain.StartBone.BoneName.ToString()) ||
			FilterString(Chain.EndBone.BoneName.ToString()) ||
			FilterString(Chain.IKGoalName.ToString())))
		{
			continue;
		}

		// apply single-bone filter
		if (ChainFilterOptions.bHideSingleBoneChains &&
			Chain.StartBone == Chain.EndBone)
		{
			continue;
		}

		// apply missing-bone filter
		if (ChainFilterOptions.bShowOnlyMissingBoneChains &&
			(Chain.StartBone.BoneName != NAME_None && Chain.EndBone.BoneName != NAME_None))
		{
			continue;
		}
		
		// apply IK filter
		if (ChainFilterOptions.bShowOnlyIKChains &&
			Chain.IKGoalName == NAME_None)
		{
			continue;
		}
		
		TSharedPtr<FRetargetChainElement> ChainItem = FRetargetChainElement::Make(Chain.ChainName);
		ListViewItems.Add(ChainItem);
	}

	// select first item if none others selected
	if (ListViewItems.Num() > 0 && ListView->GetNumItemsSelected() == 0)
	{
		ListView->SetSelection(ListViewItems[0]);
	}

	ListView->RequestListRefresh();

	// refresh the tree to show updated chain column info
	Controller->RefreshTreeView();
}

TSharedRef<ITableRow> SIKRigRetargetChainList::MakeListRowWidget(
	TSharedPtr<FRetargetChainElement> InElement,
    const TSharedRef<STableViewBase>& OwnerTable)
{
	return InElement->MakeListRowWidget(
		OwnerTable,
		InElement.ToSharedRef(),
		SharedThis(this));
}

void SIKRigRetargetChainList::OnItemClicked(TSharedPtr<FRetargetChainElement> InItem)
{
	EditorController.Pin()->SetLastSelectedType(EIKRigSelectionType::RetargetChains);
}

FReply SIKRigRetargetChainList::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	const FKey Key = InKeyEvent.GetKey();

	// handle deleting selected chain
	if (Key == EKeys::Delete)
	{
		TArray<TSharedPtr<FRetargetChainElement>> SelectedItems = ListView->GetSelectedItems();
		if (SelectedItems.IsEmpty())
		{
			return FReply::Unhandled();
		}
		
		const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
		if (!Controller.IsValid())
		{
			return FReply::Unhandled();
		}

		const UIKRigController* AssetController = Controller->AssetController;
		AssetController->RemoveRetargetChain(SelectedItems[0]->ChainName);

		RefreshView();
		
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

TSharedPtr<SWidget> SIKRigRetargetChainList::CreateContextMenu()
{
	static constexpr bool CloseAfterSelection = true;
	FMenuBuilder MenuBuilder(CloseAfterSelection, CommandList);

	MenuBuilder.BeginSection("Chains", LOCTEXT("ChainsSection", "Chains"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("MirrorChainsLabel", "Mirror Chain"),
		LOCTEXT("MirrorChainsTooltip", "Create a new, duplicate chain on the opposite side of the skeleton."),
		FSlateIcon(),
		FUIAction( FExecuteAction::CreateSP(this, &SIKRigRetargetChainList::MirrorSelectedChains)));

	MenuBuilder.AddSeparator();
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("SortChainsLabel", "Sort Chains"),
		LOCTEXT("SortChainsTooltip", "Sort chain list in hierarchical order. This does not affect the retargeting behavior."),
		FSlateIcon(),
		FUIAction( FExecuteAction::CreateSP(this, &SIKRigRetargetChainList::SortChainList)));

	MenuBuilder.EndSection();
	
	return MenuBuilder.MakeWidget();
}

void SIKRigRetargetChainList::SortChainList()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return;
	}
	
	if (const UIKRigController* AssetController = Controller->AssetController)
	{
		AssetController->SortRetargetChains();
		RefreshView();
	}
}

void SIKRigRetargetChainList::MirrorSelectedChains() const
{
	const TArray<FName> SelectedChainNames = GetSelectedChains();
	if (SelectedChainNames.IsEmpty())
	{
		return;
	}

	const FIKRigEditorController& Controller = *EditorController.Pin();
	const UIKRigController* AssetController = Controller.AssetController;
	const FIKRigSkeleton& IKRigSkeleton = AssetController->GetIKRigSkeleton();

	FScopedTransaction Transaction(LOCTEXT("MirrorRetargetChain_Label", "Mirror Retarget Chains"));
	FScopedReinitializeIKRig Reinitialize(AssetController, true /*bGoalsChanged*/);
	AssetController->GetAsset()->Modify();

	for (const FName& SelectedChainName : SelectedChainNames)
	{
		const FBoneChain* Chain = AssetController->GetRetargetChainByName(SelectedChainName);
		if (!Chain)
		{
			continue;
		}
		
		TArray<int32> BonesInChainIndices;
		const bool bIsChainValid = IKRigSkeleton.ValidateChainAndGetBones(*Chain, BonesInChainIndices);
		if (!bIsChainValid)
		{
			continue;
		}

		const EChainSide ChainSide = Controller.ChainAnalyzer.GetSideOfChain(BonesInChainIndices, IKRigSkeleton);
		if (ChainSide == EChainSide::Center)
		{
			continue;
		}

		TArray<int32> MirroredIndices;
		if (!IKRigSkeleton.GetMirroredBoneIndices(BonesInChainIndices, MirroredIndices))
		{
			continue;
		}

		FBoneChain MirroredChain = *Chain;
		MirroredChain.ChainName = UMirrorDataTable::GetSettingsMirrorName(MirroredChain.ChainName);
		MirroredChain.StartBone = IKRigSkeleton.BoneNames[MirroredIndices[0]];
		MirroredChain.EndBone = IKRigSkeleton.BoneNames[MirroredIndices.Last()];
		MirroredChain.IKGoalName = AssetController->GetGoalNameForBone(MirroredChain.EndBone.BoneName);

		const FName NewChainName = Controller.PromptToAddNewRetargetChain(MirroredChain);
		const FBoneChain* NewChain = AssetController->GetRetargetChainByName(NewChainName);
		if (!NewChain)
		{
			// user cancelled mirroring the chain
			continue;
		}
		
		// check old bone has a goal, and the new bone also has a goal
		// so we can connect the new goal to the same solver(s)
		const FName GoalOnOldChainName = AssetController->GetGoalNameForBone(Chain->EndBone.BoneName);
		const UIKRigEffectorGoal* GoalOnNewChain = AssetController->GetGoal(NewChain->IKGoalName);
		if (GoalOnOldChainName != NAME_None && GoalOnNewChain)
		{
			// connect to the same solvers
			const TArray<FIKRigSolverBase*> AllSolvers =  AssetController->GetSolverArray();
			for (int32 SolverIndex=0; SolverIndex<AllSolvers.Num(); ++SolverIndex)
			{
				if (AssetController->IsGoalConnectedToSolver(GoalOnOldChainName, SolverIndex))
				{
					AssetController->ConnectGoalToSolver(GoalOnNewChain->GoalName, SolverIndex);
				}
				else
				{
					AssetController->DisconnectGoalFromSolver(GoalOnNewChain->GoalName, SolverIndex);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

