// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigEditor/SIKRigSolverStack.h"

#include "RigEditor/IKRigEditorStyle.h"
#include "RigEditor/IKRigToolkit.h"
#include "RigEditor/IKRigEditorController.h"
#include "Rig/Solvers/IKRigSolverBase.h"

#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SKismetInspector.h"
#include "Dialogs/Dialogs.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SPositiveActionButton.h"
#include "Rig/IKRigProcessor.h"

#define LOCTEXT_NAMESPACE "SIKRigSolverStack"

TSharedRef<ITableRow> FSolverStackElement::MakeListRowWidget(
	const TSharedRef<STableViewBase>& InOwnerTable,
	TSharedRef<FSolverStackElement> InStackElement,
	TSharedPtr<SIKRigSolverStack> InSolverStack)
{
	return SNew(SIKRigSolverStackItem, InOwnerTable, InStackElement, InSolverStack);
}

void SIKRigSolverStackItem::Construct(
	const FArguments& InArgs,
	const TSharedRef<STableViewBase>& OwnerTable,
	TSharedRef<FSolverStackElement> InStackElement,
	TSharedPtr<SIKRigSolverStack> InSolverStack)
{
	StackElement = InStackElement;
	SolverStack = InSolverStack;
	
	STableRow<TSharedPtr<FSolverStackElement>>::Construct(
        STableRow<TSharedPtr<FSolverStackElement>>::FArguments()
        .OnDragDetected(InSolverStack.Get(), &SIKRigSolverStack::OnDragDetected)
        .OnCanAcceptDrop(InSolverStack.Get(), &SIKRigSolverStack::OnCanAcceptDrop)
        .OnAcceptDrop(InSolverStack.Get(), &SIKRigSolverStack::OnAcceptDrop)
        .Content()
        [
            SNew(SHorizontalBox)
            + SHorizontalBox::Slot()
            .AutoWidth()
            .HAlign(HAlign_Left)
            .Padding(3)
            [
            	SNew(SHorizontalBox)
	            + SHorizontalBox::Slot()
	            .MaxWidth(18)
				.FillWidth(1.0)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
	            [
	                SNew(SImage)
	                .Image(FIKRigEditorStyle::Get().GetBrush("IKRig.DragSolver"))
	            ]

	            + SHorizontalBox::Slot()
	            .AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(3.0f, 1.0f)
				[
					SNew(SCheckBox)
					.IsEnabled_Lambda([this]()
					{
						FText Warning;
						return !GetWarningMessage(Warning);
					})
					.IsChecked_Lambda([InSolverStack, InStackElement]() -> ECheckBoxState
					{
						bool bEnabled = true;
						if (InSolverStack.IsValid() &&
							InSolverStack->EditorController.IsValid() &&
							InSolverStack->EditorController.Pin().IsValid())
						{
							bEnabled = InSolverStack->EditorController.Pin()->AssetController->GetSolverEnabled(InStackElement->IndexInStack);
						}
						return bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
					})
					.OnCheckStateChanged_Lambda([InSolverStack, InStackElement](ECheckBoxState InCheckBoxState)
					{
						if (InSolverStack.IsValid() &&
							InSolverStack->EditorController.IsValid() &&
							InSolverStack->EditorController.Pin().IsValid())
						{
							bool bIsChecked = InCheckBoxState == ECheckBoxState::Checked;
							InSolverStack->EditorController.Pin()->AssetController->SetSolverEnabled(InStackElement->IndexInStack, bIsChecked);
						}
					})
				]
	     
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(3.0f, 1.0f)
	            [
					SNew(STextBlock)
					.Text(InStackElement->DisplayName)
					.IsEnabled_Lambda([this]()
					{
						if (!IsSolverEnabled())
						{
							return false;
						}
						FText Warning;
						return !GetWarningMessage(Warning);
					})
					.TextStyle(FAppStyle::Get(), "NormalText.Important")
	            ]

	            + SHorizontalBox::Slot()
	            .AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(3.0f, 1.0f)
				[
					SNew(STextBlock)
					.Text(InStackElement->DisplayName)
					.Text_Lambda([this]()
					{
						FText Message;
						GetWarningMessage(Message);
						return Message;
					})
					.TextStyle(FAppStyle::Get(), "NormalText.Subdued")
				]
            ]

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(3)
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("DeleteSolver", "Delete solver and remove from stack."))
				.OnClicked_Lambda([InSolverStack, InStackElement]() -> FReply
				{
					InSolverStack.Get()->DeleteSolver(InStackElement);
					return FReply::Handled();
				})
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
        ], OwnerTable);
}

bool SIKRigSolverStackItem::GetWarningMessage(FText& Message) const
{
	constexpr bool bFromAsset = false;
	if (const FIKRigSolverBase* Solver = GetSolver(bFromAsset))
	{
		return Solver->GetWarningMessage(Message);
	}
	
	return false;
}

bool SIKRigSolverStackItem::IsSolverEnabled() const
{
	constexpr bool bFromAsset = true;
	if (const FIKRigSolverBase* Solver = GetSolver(bFromAsset))
	{
		return Solver->IsEnabled();
	}
	
	return false;
}

const FIKRigSolverBase* SIKRigSolverStackItem::GetSolver(const bool bFromAsset) const
{
	if (!(StackElement.IsValid() && SolverStack.IsValid()))
	{
		return nullptr;
	}
	if (!SolverStack.Pin()->EditorController.IsValid())
	{
		return nullptr;
	}
	
	const int32 SolverIndex = StackElement.Pin()->IndexInStack;
	const TSharedPtr<FIKRigEditorController> EditorController = SolverStack.Pin()->EditorController.Pin();
	
	if (bFromAsset)
	{
		// get the solver stored in the asset
		return EditorController->AssetController->GetSolverAtIndex(SolverIndex);
	}
	else
	{
		// get the currently running solver instance in the editor's runtime processor
		// this is needed to report solver warnings dependent on being initialized
		if (FIKRigProcessor* Processor = EditorController->GetIKRigProcessor())
		{
			return Processor->GetSolver(SolverIndex);
		}
	}

	return nullptr;
}

TSharedRef<FIKRigSolverStackDragDropOp> FIKRigSolverStackDragDropOp::New(TWeakPtr<FSolverStackElement> InElement)
{
	TSharedRef<FIKRigSolverStackDragDropOp> Operation = MakeShared<FIKRigSolverStackDragDropOp>();
	Operation->Element = InElement;
	Operation->Construct();
	return Operation;
}

TSharedPtr<SWidget> FIKRigSolverStackDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
        .Visibility(EVisibility::Visible)
        .BorderImage(FAppStyle::GetBrush("Menu.Background"))
        [
            SNew(STextBlock)
            .Text(Element.Pin()->DisplayName)
        ];
}

SIKRigSolverStack::~SIKRigSolverStack()
{
	
}

void SIKRigSolverStack::Construct(const FArguments& InArgs, TSharedRef<FIKRigEditorController> InEditorController)
{
	EditorController = InEditorController;
	EditorController.Pin()->SetSolverStackView(SharedThis(this));
	
	CacheSolverMetaData();
	
	CommandList = MakeShared<FUICommandList>();

	ChildSlot
    [
        SNew(SVerticalBox)
        +SVerticalBox::Slot()
        .AutoHeight()
        .VAlign(VAlign_Top)
        .Padding(0.0f)
        [
            SNew(SBorder)
            .Padding(0.0f)
            .BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
            .BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
            [
                SNew(SVerticalBox)
                + SVerticalBox::Slot()
                .AutoHeight()
                .VAlign(VAlign_Top)
                [
                    SNew(SHorizontalBox)
                    + SHorizontalBox::Slot()
                    .VAlign(VAlign_Center)
                    .HAlign(HAlign_Left)
                    .FillWidth(1.f)
                    .Padding(3.0f, 1.0f)
                    [
	                    SNew(SPositiveActionButton)
				        .Icon(FAppStyle::Get().GetBrush("Icons.Plus"))
				        .Text(LOCTEXT("AddNewSolverLabel", "Add New Solver"))
				        .ToolTipText(LOCTEXT("AddNewToolTip", "Add a new IK solver to the rig."))
				        .IsEnabled(this, &SIKRigSolverStack::IsAddSolverEnabled)
				        .OnGetMenuContent(this, &SIKRigSolverStack::CreateAddNewMenuWidget)
                    ]
                ]
            ]
        ]

        +SVerticalBox::Slot()
        .Padding(0.0f)
        [
			SAssignNew( ListView, SSolverStackListViewType )
			.SelectionMode(ESelectionMode::Multi)
			.IsEnabled(this, &SIKRigSolverStack::IsAddSolverEnabled)
			.ListItemsSource( &ListViewItems )
			.OnGenerateRow( this, &SIKRigSolverStack::MakeListRowWidget )
			.OnSelectionChanged(this, &SIKRigSolverStack::OnSelectionChanged)
			.OnMouseButtonClick(this, &SIKRigSolverStack::OnItemClicked)
        ]
    ];

	RefreshStackView();
}

TSharedRef<SWidget> SIKRigSolverStack::CreateAddNewMenuWidget()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	BuildAddNewMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

void SIKRigSolverStack::BuildAddNewMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection("AddNewSolver", LOCTEXT("AddOperations", "Add New Solver"));

	// add menu option to create each solver type
	for (const FIKRigSolverMetaData& SolverMetaData : AllSolversMetaData)
	{
		FUIAction Action = FUIAction( FExecuteAction::CreateSP(this, &SIKRigSolverStack::AddNewSolver, SolverMetaData.ScriptStruct));
		MenuBuilder.AddMenuEntry(SolverMetaData.NiceName, FText::GetEmpty(), FSlateIcon(), Action);
	}
	
	MenuBuilder.EndSection();
}

bool SIKRigSolverStack::IsAddSolverEnabled() const
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return false; 
	}
	
	if (UIKRigController* AssetController = Controller->AssetController)
	{
		if (AssetController->GetIKRigSkeleton().BoneNames.Num() > 0)
		{
			return true;
		}
	}
	
	return false;
}

void SIKRigSolverStack::CacheSolverMetaData()
{
	AllSolversMetaData.Reset();

	// instantiate all types of solvers to cache their meta data
	const UStruct* BaseStruct = FIKRigSolverBase::StaticStruct();
	for (TObjectIterator<UStruct> StructIt; StructIt; ++StructIt)
	{
		UStruct* Struct = *StructIt;
		if (!Struct->IsChildOf(BaseStruct) || Struct == BaseStruct)
		{
			continue;
		}

		UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Struct);
		if (!ScriptStruct)
		{
			continue;
		}

		FInstancedStruct TempInstancedStruct = FInstancedStruct(ScriptStruct);
		const FIKRigSolverBase& TempSolver = TempInstancedStruct.Get<FIKRigSolverBase>();

		// store meta data for this type
		FIKRigSolverMetaData SolverMetaData;
		SolverMetaData.NiceName = TempSolver.GetNiceName();
		SolverMetaData.ScriptStruct = ScriptStruct;
		AllSolversMetaData.Add(SolverMetaData);
	}
}

void SIKRigSolverStack::AddNewSolver(UScriptStruct* SolverType)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	UIKRigController* AssetController = Controller->AssetController;
	if (!AssetController)
	{
		return;
	}
	
	// add the solver
	const int32 NewSolverIndex = AssetController->AddSolver(SolverType);
	// update stack view
	RefreshStackView();
	Controller->RefreshTreeView(); // updates solver indices in tree items
	// select it
	ListView->SetSelection(ListViewItems[NewSolverIndex]);
	// show details for it
	Controller->ShowDetailsForSolvers({NewSolverIndex});
}

void SIKRigSolverStack::DeleteSolver(TSharedPtr<FSolverStackElement> SolverToDelete)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}
	
	if (!SolverToDelete.IsValid())
	{
		return;
	}

	Controller->ShowAssetDetails();

	const UIKRigController* AssetController = Controller->AssetController;
	AssetController->RemoveSolver(SolverToDelete->IndexInStack);
	RefreshStackView();
	Controller->RefreshTreeView();
}

void SIKRigSolverStack::RefreshStackView()
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	// record/restore selection
	int32 IndexToSelect = 0; // default to first solver selected
	const TArray<TSharedPtr<FSolverStackElement>> SelectedItems = ListView.Get()->GetSelectedItems();
	if (!SelectedItems.IsEmpty())
	{
		IndexToSelect = SelectedItems[0]->IndexInStack;
	}

	// generate all list items
	static const FText UnknownSolverTxt = FText::FromString("Unknown Solver");
	
	ListViewItems.Reset();
	UIKRigController* AssetController = Controller->AssetController;
	const int32 NumSolvers = AssetController->GetNumSolvers();
	for (int32 i=0; i<NumSolvers; ++i)
	{
		const FIKRigSolverBase* Solver = AssetController->GetSolverAtIndex(i);
		const FText DisplayName = Solver ? FText::FromString(AssetController->GetSolverUniqueName(i)) : UnknownSolverTxt;
		TSharedPtr<FSolverStackElement> SolverItem = FSolverStackElement::Make(DisplayName, i);
		ListViewItems.Add(SolverItem);
	}

	if (NumSolvers && ListViewItems.IsValidIndex(IndexToSelect))
	{
		// restore selection
		ListView->SetSelection(ListViewItems[IndexToSelect]);
	}
	else
	{
		// clear selection otherwise
		ListView->ClearSelection();
	}

	ListView->RequestListRefresh();
}

TSharedRef<ITableRow> SIKRigSolverStack::MakeListRowWidget(
	TSharedPtr<FSolverStackElement> InElement,
    const TSharedRef<STableViewBase>& OwnerTable)
{
	return InElement->MakeListRowWidget(
		OwnerTable,
		InElement.ToSharedRef(),
		SharedThis(this));
}

void SIKRigSolverStack::OnSelectionChanged(TSharedPtr<FSolverStackElement> InItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		ShowDetailsForItems(ListView->GetSelectedItems());	
	}
}

void SIKRigSolverStack::OnItemClicked(TSharedPtr<FSolverStackElement> InItem)
{
	ShowDetailsForItems({InItem});
	EditorController.Pin()->SetLastSelectedType(EIKRigSelectionType::SolverStack);
}

void SIKRigSolverStack::ShowDetailsForItems(const TArray<TSharedPtr<FSolverStackElement>>& InItems)
{
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return; 
	}

	// update bones greyed out when not affected
	Controller->RefreshTreeView();

	// the solver selection must be done after we rebuilt the skeleton tree has it keeps the selection
	if (InItems.IsEmpty())
	{
		// clear the details panel only if there's nothing selected in the skeleton view
		if (Controller->DoesSkeletonHaveSelectedItems())
		{
			return;
		}
		
		Controller->ShowAssetDetails();
	}
	else
	{
		TArray<int32> SolverIndices;
		for (const TSharedPtr<FSolverStackElement>& Item : InItems)
		{
			SolverIndices.Add(Item->IndexInStack);
		}
		Controller->ShowDetailsForSolvers(SolverIndices);
	}
}

FReply SIKRigSolverStack::OnDragDetected(
	const FGeometry& MyGeometry,
	const FPointerEvent& MouseEvent)
{
	const TArray<TSharedPtr<FSolverStackElement>> SelectedItems = ListView.Get()->GetSelectedItems();
	if (SelectedItems.Num() != 1)
	{
		return FReply::Unhandled();
	}
	
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		const TSharedPtr<FSolverStackElement> DraggedElement = SelectedItems[0];
		const TSharedRef<FIKRigSolverStackDragDropOp> DragDropOp = FIKRigSolverStackDragDropOp::New(DraggedElement);
		return FReply::Handled().BeginDragDrop(DragDropOp);
	}

	return FReply::Unhandled();
}

TOptional<EItemDropZone> SIKRigSolverStack::OnCanAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
	TSharedPtr<FSolverStackElement> TargetItem)
{
	TOptional<EItemDropZone> ReturnedDropZone;
	
	const TSharedPtr<FIKRigSolverStackDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FIKRigSolverStackDragDropOp>();
	if (DragDropOp.IsValid())
	{
		ReturnedDropZone = DropZone == EItemDropZone::BelowItem ? EItemDropZone::BelowItem : EItemDropZone::AboveItem;
	}
	
	return ReturnedDropZone;
}

FReply SIKRigSolverStack::OnAcceptDrop(
	const FDragDropEvent& DragDropEvent,
	EItemDropZone DropZone,
	TSharedPtr<FSolverStackElement> TargetItem)
{
	const TSharedPtr<FIKRigSolverStackDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FIKRigSolverStackDragDropOp>();
	if (!DragDropOp.IsValid())
	{
		return FReply::Unhandled();
	}
	
	const TSharedPtr<FIKRigEditorController> Controller = EditorController.Pin();
	if (!Controller.IsValid())
	{
		return FReply::Handled();
	}

	const FSolverStackElement& DraggedElement = *DragDropOp.Get()->Element.Pin().Get();
	if (DraggedElement.IndexInStack == TargetItem->IndexInStack)
	{
		return FReply::Handled();
	}
	
	UIKRigController* AssetController = Controller->AssetController;
	int32 TargetIndex = TargetItem.Get()->IndexInStack;
	if ( DropZone == EItemDropZone::AboveItem)
	{
		TargetIndex = FMath::Max(0, TargetIndex-1);
	}
	const bool bWasMoved = AssetController->MoveSolverInStack(DraggedElement.IndexInStack, TargetIndex);
	if (bWasMoved)
	{
		RefreshStackView();
		Controller->RefreshTreeView(); // update solver indices in effector items
		Controller->ShowDetailsForSolvers({TargetIndex});
		ListView->SetSelection(ListViewItems[TargetIndex]);
	}
	
	return FReply::Handled();
}

FReply SIKRigSolverStack::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{	
	// handle deleting selected solver
	FKey KeyPressed = InKeyEvent.GetKey();
	if (KeyPressed == EKeys::Delete || KeyPressed == EKeys::BackSpace)
	{
		TArray<TSharedPtr<FSolverStackElement>> SelectedItems = ListView->GetSelectedItems();
		if (!SelectedItems.IsEmpty())
		{
			// only delete 1 at a time to avoid messing up indices
			DeleteSolver(SelectedItems[0]);
		}
		
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

#undef LOCTEXT_NAMESPACE
