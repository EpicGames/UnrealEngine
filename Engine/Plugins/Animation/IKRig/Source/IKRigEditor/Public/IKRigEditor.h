// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API IKRIGEDITOR_API

class FIKRigEditor : public IModuleInterface
{
public:
	UE_API void StartupModule() override;
	UE_API void ShutdownModule() override;

private:
	TSharedPtr<class FAssetTypeActions_AnimationAssetRetarget> RetargetAnimationAssetAction;
	TSharedPtr<class FAssetTypeActions_IKRigDefinition> IKRigDefinitionAssetAction;
	TSharedPtr<class FAssetTypeActions_IKRetargeter> IKRetargeterAssetAction;

	TArray<FName> ClassesToUnregisterOnShutdown;
};

DECLARE_LOG_CATEGORY_EXTERN(LogIKRigEditor, Warning, All);

#undef UE_API
