// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetTypeActions_Base.h"
#include "EditorAnimUtils.h"

class UIKRetargeter;

class FAssetTypeActions_IKRetargeter : public FAssetTypeActions_Base
{
public:
	// BEGIN IAssetTypeActions
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_IKRetargeter", "IK Retargeter"); }
	virtual FColor GetTypeColor() const override { return FColor(0, 128, 255); }
	virtual UClass* GetSupportedClass() const override;
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual UThumbnailInfo* GetThumbnailInfo(UObject* Asset) const override;
	virtual const TArray<FText>& GetSubMenus() const override;
	// END IAssetTypeActions

	static void ExtendIKRigMenuToMakeRetargeter();
	static void CreateNewIKRetargeterFromIKRig(UObject* InSelectedObject);
	static void ExtendAnimAssetMenusForBatchRetargeting();
};
