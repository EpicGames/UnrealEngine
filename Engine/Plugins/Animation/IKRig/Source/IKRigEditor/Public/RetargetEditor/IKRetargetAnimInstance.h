// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimPreviewInstance.h"
#include "Animation/AnimNodeBase.h"
#include "AnimNodes/AnimNode_RetargetPoseFromMesh.h"

#include "IKRetargetAnimInstance.generated.h"

enum class ERetargeterOutputMode : uint8;
class UIKRetargeter;

// a node just to preview a retarget pose
USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_PreviewRetargetPose : public FAnimNode_Base
{
	GENERATED_BODY()

	FPoseLink InputPose;

	/** Retarget asset holding the pose to preview.*/
	TObjectPtr<UIKRetargeter> IKRetargeterAsset = nullptr;

	/** Whether to preview the source or the target */
	ERetargetSourceOrTarget SourceOrTarget;

	/** Amount to blend the retarget pose with the reference pose when shoing the retarget pose */
	float RetargetPoseBlend = 1.0f;

	// FAnimNode_Base interface
	virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	// End of FAnimNode_Base interface

	const TArray<FTransform>& GetLocalRetargetPose() const { return RetargetLocalPose; };
	const TArray<FTransform>& GetGlobalRetargetPose() const { return RetargetGlobalPose; };

private:
	// mapping from required bone indices (from LODs) to actual bone indices within the skeletal mesh 
	TArray<int32> RequiredBoneToMeshBoneMap;

	TArray<FTransform> RetargetLocalPose;
	TArray<FTransform> RetargetGlobalPose;
};

UCLASS(transient, NotBlueprintable)
class UIKRetargetAnimInstance : public UAnimPreviewInstance
{
	GENERATED_UCLASS_BODY()

public:

	void SetRetargetMode(const ERetargeterOutputMode& OutputMode);

	void SetRetargetPoseBlend(const float& RetargetPoseBlend);

	void ConfigureAnimInstance(
		const ERetargetSourceOrTarget& SourceOrTarget,
		UIKRetargeter* InIKRetargetAsset,
		TWeakObjectPtr<USkeletalMeshComponent> InSourceMeshComponent);

	void SetOverrideSets(const TArray<FName>& InOverrideSets);

	FIKRetargetProcessor* GetRetargetProcessor();
	
	const FAnimNode_RetargetPoseFromMesh& GetRetargetAnimNode();

	void ForceInitializeProcessor(USkeletalMeshComponent* TargetComponent);

	const TArray<FTransform>& GetLocalRetargetPose() const { return PreviewPoseNode.GetLocalRetargetPose(); };
	const TArray<FTransform>& GetGlobalRetargetPose() const { return PreviewPoseNode.GetGlobalRetargetPose(); };

protected:
	virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;

	UPROPERTY(Transient)
	FAnimNode_PreviewRetargetPose PreviewPoseNode;
	
	UPROPERTY(Transient)
	FAnimNode_RetargetPoseFromMesh RetargetNode;
};
