// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimPreviewInstance.h"
#include "IKRetargetAnimInstance.h"
#include "Animation/AnimNode_LinkedInputPose.h"

#include "IKRetargetAnimInstanceProxy.generated.h"

class UIKRetargeter;
enum class ERetargetSourceOrTarget : uint8;
struct FAnimNode_PreviewRetargetPose;
struct FAnimNode_RetargetPoseFromMesh;

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FIKRetargetAnimInstanceProxy : public FAnimPreviewInstanceProxy
{
	GENERATED_BODY()

public:
	
	FIKRetargetAnimInstanceProxy() = default;
	FIKRetargetAnimInstanceProxy(
		UAnimInstance* InAnimInstance,
		FAnimNode_PreviewRetargetPose* InPreviewPoseNode,
		FAnimNode_RetargetPoseFromMesh* InRetargetNode);
	virtual ~FIKRetargetAnimInstanceProxy() override = default;

	/** FAnimPreviewInstanceProxy interface */
	virtual void Initialize(UAnimInstance* InAnimInstance) override;
	virtual void CacheBones() override;
	virtual bool Evaluate(FPoseContext& Output) override;
	virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	/** END FAnimPreviewInstanceProxy interface */

	/** FAnimInstanceProxy interface */
	/** Called when the anim instance is being initialized. These nodes can be provided */
	virtual FAnimNode_Base* GetCustomRootNode() override;
	virtual void GetCustomNodes(TArray<FAnimNode_Base*>& OutNodes) override;
	/** END FAnimInstanceProxy interface */

	void ConfigureAnimInstance(
		const ERetargetSourceOrTarget& InSourceOrTarget,
		UIKRetargeter* InIKRetargetAsset,
		TWeakObjectPtr<USkeletalMeshComponent> InSourceMeshComponent);

	void SetRetargetMode(const ERetargeterOutputMode& InOutputMode);

	void SetRetargetPoseBlend(const float& InRetargetPoseBlend) const;

	FAnimNode_PreviewRetargetPose* PreviewPoseNode;
	FAnimNode_RetargetPoseFromMesh* RetargetNode;

	ERetargetSourceOrTarget SourceOrTarget;
	ERetargeterOutputMode OutputMode;
};