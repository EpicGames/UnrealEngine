// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FIKRetargetEditor;

class FIKRetargetApplicationMode : public FApplicationMode
{
	
public:
	
	FIKRetargetApplicationMode(TSharedRef<class FWorkflowCentricApplication> InHostingApp, TSharedRef<class IPersonaPreviewScene> InPreviewScene);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	/** END FApplicationMode interface */

protected:
	
	/** The hosting app */
	TWeakPtr<FIKRetargetEditor> IKRetargetEditorPtr;

	/** The tab factories we support */
	FWorkflowAllowedTabSet TabFactories;
};
