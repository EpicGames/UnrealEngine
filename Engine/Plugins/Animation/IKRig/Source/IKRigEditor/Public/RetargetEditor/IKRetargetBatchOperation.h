// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorAnimUtils.h"
#include "Retargeter/RetargetOps/RetargetCurvesOp.h"

#include "IKRetargetBatchOperation.generated.h"

#define UE_API IKRIGEDITOR_API

class UIKRetargeter;
struct FScopedSlowTask;
class USkeletalMesh;
class UAnimationAsset;
class UAnimBlueprint;
class UAnimSequence;
class USkeleton;
class IAnimationDataModel;
class IAnimationDataController;

struct FAdditiveRetargetSettings
{
	// the asset this operates on
	TObjectPtr<UAnimSequence> SequenceAsset;
	
	// the settings to save/restore before/after retargeting
	TEnumAsByte<EAdditiveAnimationType> AdditiveAnimType;
	TEnumAsByte<EAdditiveBasePoseType> RefPoseType;
	int32 RefFrameIndex;
	TObjectPtr<UAnimSequence> RefPoseSeq;
	
	void PrepareForRetarget(UAnimSequence* InSequenceAsset);
	void RestoreOnAsset() const;
};

// which skeleton are we referring to?
UENUM()
enum class ERetargetRootLockMode : uint8
{
	// Uses the "ForceRootLock" setting in the source animation
	FromSourceAnimation,
	// Force the root to be locked, regardless of whether the source animation has the root locked or not.
	ForceRootLocked,
	// Force the root to be unlocked, regardless of whether the source animation has the root locked or not.
	ForceRootUnlocked,
};

/// Data needed to run a batch "duplicate and retarget" operation on a set of animation assets
struct FIKRetargetBatchOperationContext
{
	
public:
	
	// The source assets to duplicate and retarget
	TArray<TWeakObjectPtr<UObject>> AssetsToRetarget;

	// Source mesh to use to copy animation FROM.
	USkeletalMesh* SourceMesh = nullptr;

	// Target mesh to use to copy animation TO.
	USkeletalMesh* TargetMesh = nullptr;

	// The retargeter used to copy animation
	UIKRetargeter* IKRetargetAsset = nullptr;

	// Rename rules for duplicated assets
	EditorAnimUtils::FNameDuplicationRule NameRule;

	// Any files with the same name will be overwritten instead of creating a new file with a numeric suffix.
	// This is useful when iterating on a batch process.
	bool bUseSourcePath = false;
	
	// Any files with the same name will be overwritten instead of creating a new file with a numeric suffix.
	// This is useful when iterating on a batch process.
	bool bOverwriteExistingFiles = false;

	// Duplicates and retargets any animation assets referenced by the input assets. For example, sequences in an animation blueprint or blendspace.
	bool bIncludeReferencedAssets = true;

	// Will not produce keys on bones that are not animated, reducing size on disk of the resulting files.
	bool bExportOnlyAnimatedBones = true;

	// Keep the additive animation sequence attributes on the retargeted results.
	// NOTE: results may not be WYSIWYG with the editor, but the behavior should remain intact.
	bool bRetainAdditiveFlags = true;

	// An array of property override sets to apply. These must be present in the supplied IK Retarget asset.
	TArray<FName> OverrideSetNames;

	// Reset all data (called when window re-opened
	void Reset()
	{
		SourceMesh = nullptr;
		TargetMesh = nullptr;
		IKRetargetAsset = nullptr;
		bIncludeReferencedAssets = true;
		NameRule.Prefix = "";
		NameRule.Suffix = "";
		NameRule.ReplaceFrom = "";
		NameRule.ReplaceTo = "";
		OverrideSetNames.Reset();
	}

	// Is the data configured in such a way that we could run the retarget?
	bool IsValid() const
	{
		// todo validate compatibility
		return SourceMesh && TargetMesh && IKRetargetAsset && (SourceMesh != TargetMesh);
	}
};

/** * Input data for running a batch animation retarget. 
 * Using a struct allows for future expansion without breaking existing Blueprint nodes.
 */
USTRUCT(BlueprintType)
struct FIKRetargetBatchOperationInputs
{
	GENERATED_BODY()

	/** A list of animation assets to retarget (sequences, blendspaces or montages) */
	UPROPERTY(BlueprintReadWrite, Category="IK Batch Retarget")
	TArray<FAssetData> AssetsToRetarget;

	/** The skeletal mesh with desired proportions to playback the assets to retarget */
	UPROPERTY(BlueprintReadWrite, Category="IK Batch Retarget")
	TObjectPtr<USkeletalMesh> SourceMesh = nullptr;

	/** The skeletal mesh to retarget the animation onto */
	UPROPERTY(BlueprintReadWrite, Category="IK Batch Retarget")
	TObjectPtr<USkeletalMesh> TargetMesh = nullptr;

	/** The IK Retargeter asset with IK Rigs appropriate for the source and target skeletal mesh */
	UPROPERTY(BlueprintReadWrite, Category="IK Batch Retarget")
	TObjectPtr<UIKRetargeter> IKRetargetAsset = nullptr;

	/** An array of names of the property override sets to apply to the retarget (must be stored in the IK Retarget Asset) */
	UPROPERTY(BlueprintReadWrite, Category="IK Batch Retarget")
	TArray<FName> InOverrideSetNames;

	/** A string to search for in the file name (replaced with "Replace" string) */
	UPROPERTY(BlueprintReadWrite, Category="Naming")
	FString Search = TEXT("");

	/** A string to replace with in the file name */
	UPROPERTY(BlueprintReadWrite, Category="Naming")
	FString Replace = TEXT("");

	/** A string to add to the start of the new file name */
	UPROPERTY(BlueprintReadWrite, Category="Naming")
	FString Prefix = TEXT("");

	/** A string to add at the end of the new file name */
	UPROPERTY(BlueprintReadWrite, Category="Naming")
	FString Suffix = TEXT("");

	/** A string containing the destination path for the retargeted assets. Ignored if bUseSourcePath is set to true. */
	UPROPERTY(BlueprintReadWrite, Category="Pathing")
	FString TargetPath = TEXT("");

	/** Place the assets in the same folder as their source location instead of the given target path. */
	UPROPERTY(BlueprintReadWrite, Category="Pathing")
	bool bUseSourcePath = false;

	/** Whether to retarget animation assets referenced by the AssetsToRetarget */
	UPROPERTY(BlueprintReadWrite, Category="IK Batch Retarget")
	bool bIncludeReferencedAssets = true;

	/** Whether to overwrite any existing output files with the same name, or create new files with a unique name */
	UPROPERTY(BlueprintReadWrite, Category="IK Batch Retarget")
	bool bOverwriteExistingFiles = false;

	/** Keep the additive animation sequence attributes on the retargeted results.
	 * NOTE: results may not be WYSIWYG with the editor, but the behavior should remain intact. */
	UPROPERTY(BlueprintReadWrite, Category="IK Batch Retarget")
	bool bRetainAdditiveFlags = true;
};

// Encapsulate ability to batch duplicate and retarget a set of animation assets
UCLASS(MinimalAPI, BlueprintType)
class UIKRetargetBatchOperation : public UObject
{
	GENERATED_BODY()
	
public:

	/* Convenience function to run a batch animation retarget from Blueprint / Python. This function will duplicate a list of
	 * AssetsToRetarget and use the supplied IK Retargeter to retarget the animation from the source to the target using the
	 * settings in the provided IK Retargeter asset.
	 * @param Inputs a struct with all the inputs required to run a batch process
	 * @return a list of new animation files created
	 */
	UFUNCTION(BlueprintCallable, Category=IKBatchRetarget)
	static UE_API TArray<FAssetData> RunBatchRetarget(const FIKRetargetBatchOperationInputs& Inputs);

	/** * @deprecated This function is deprecated. Please use RunBatchRetarget which takes a FIKRetargetBatchOperationInputs struct. */
	UFUNCTION(
		BlueprintCallable, 
		Category=IKBatchRetarget, 
		meta=(
			DeprecatedFunction,
			DeprecationMessage="Use RunBatchRetarget instead. It uses a struct for better future-proofing and cleaner Blueprint graphs."
		)
	)
	UE_DEPRECATED(5.8, "DuplicateAndRetarget is deprecated. Use RunBatchRetarget instead.")
	static UE_API TArray<FAssetData> DuplicateAndRetarget(
		const TArray<FAssetData>& AssetsToRetarget,
		USkeletalMesh* SourceMesh,
		USkeletalMesh* TargetMesh,
		UIKRetargeter* IKRetargetAsset,
		const FString& Search = TEXT(""),
		const FString& Replace = TEXT(""),
		const FString& Prefix = TEXT(""),
		const FString& Suffix = TEXT(""),
		const FString& TargetPath = TEXT(""),
		const bool bUseSourcePath = false,
		const bool bIncludeReferencedAssets = true,
		const bool bOverwriteExistingFiles = false);
	
	// Actually run the process to duplicate and retarget the assets for the given context
	UE_API void RunRetarget(FIKRetargetBatchOperationContext& Context);

private:

	UE_API void Reset();

	// Initialize set of referenced assets to retarget.
	// @return	Number of assets that need retargeting. 
	UE_API int32 GenerateAssetLists(const FIKRetargetBatchOperationContext& Context);

	// Duplicate all the assets to retarget
	UE_API void DuplicateRetargetAssets(const FIKRetargetBatchOperationContext& Context, FScopedSlowTask& Progress);

	// Retarget skeleton and animation on all the duplicates
	UE_API void RetargetAssets(const FIKRetargetBatchOperationContext& Context, FScopedSlowTask& Progress);

	// Convert animation on all the duplicates
	UE_API void ConvertAnimation(const FIKRetargetBatchOperationContext& Context, FIKRetargetProcessor& OutProcessor, FScopedSlowTask& Progress);

	// Apply any curve operations on all the duplicates
	UE_API void ApplyCurveOps(const FIKRetargetBatchOperationContext& Context, const FIKRetargetProcessor& InProcessor, FScopedSlowTask& Progress);

	// Replace existing assets (optional)
	UE_API void OverwriteExistingAssets(const FIKRetargetBatchOperationContext& Context, FScopedSlowTask& Progress);

	// Output notifications of results
	UE_API void NotifyUserOfResults(const FIKRetargetBatchOperationContext& Context, FScopedSlowTask& Progress) const;

	// Generate list of newly created assets to report to user
	UE_API void GetNewAssets(TArray<UObject*>& NewAssets) const;

	// If user cancelled half way, cleanup all the duplicated assets
	UE_API void CleanupIfCancelled(const FScopedSlowTask& Progress) const;

	// Set the processed curve data into the target animation sequence.
	// Curves not modified by any op are copied directly from source to preserve tangents and exact key times.
	UE_API void AddCurveValuesToAnimSequence(const UAnimSequence* InSourceSequence, USkeleton* InSourceSkeleton, USkeleton* InTargetSkeleton,
		const FIKRetargetCurvesOpBase::FCurveData& InCurveMetaData, const FIKRetargetCurvesOpBase::FFrameValues& InCurveValuesPerFrame,
		const FIKRetargetCurvesOpBase::FFrameValues& InOriginalCurveValuesPerFrame, const FFrameRate& InFrameRate,
		const TArray<FFrameTime>& InFrameTimes, bool bInShouldTransact, IAnimationDataController& OutTargetSeqController) const;

	
	// Lists of assets to retarget. Populated from selection during init
	TArray<UAnimationAsset*>	AnimationAssetsToRetarget;
	TArray<UAnimBlueprint*>		AnimBlueprintsToRetarget;

	// Lists of original assets map to duplicate assets
	TMap<UAnimationAsset*, UAnimationAsset*>	DuplicatedAnimAssets;
	TMap<UAnimBlueprint*, UAnimBlueprint*>		DuplicatedBlueprints;

	TMap<UAnimationAsset*, UAnimationAsset*>	RemappedAnimAssets;
};

#undef UE_API
