// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "IKRetargetEditorStyle.h"

class FIKRetargetCommands : public TCommands<FIKRetargetCommands>
{
public:
	FIKRetargetCommands() : TCommands<FIKRetargetCommands>
	(
		"IKRetarget",
		NSLOCTEXT("Contexts", "IKRetarget", "IK Retarget"),
		NAME_None,
		FIKRetargetEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}
	
	// run the retargeter
	TSharedPtr< FUICommandInfo > RunRetargeter;
	// show current retarget pose
	TSharedPtr< FUICommandInfo > ShowRetargetPose;
	// edit retarget pose
	TSharedPtr< FUICommandInfo > EditRetargetPose;

	// open asset settings
	TSharedPtr< FUICommandInfo > ShowAssetSettings;
	// open the override set manager
	TSharedPtr< FUICommandInfo > ShowOverrideSets;
	// open the variables editor
	TSharedPtr< FUICommandInfo > ShowVariables;

	// enable Root retarget pass
	TSharedPtr< FUICommandInfo > EnableRoot;
	// enable FK retarget pass
	TSharedPtr< FUICommandInfo > EnableFK;
	// enable IK retarget pass
	TSharedPtr< FUICommandInfo > EnableIK;
	// enable IK retarget pass
	TSharedPtr< FUICommandInfo > EnablePostPass;

	// reset retarget pose to ref pose
	TSharedPtr< FUICommandInfo > ResetAllBones;
	// reset retarget pose to ref pose
	TSharedPtr< FUICommandInfo > ResetSelectedBones;
	// reset retarget pose to ref pose
	TSharedPtr< FUICommandInfo > ResetSelectedAndChildrenBones;

	// auto align all bones in skeleton
	TSharedPtr< FUICommandInfo > AlignAllBones;
	// auto align selected bones
	TSharedPtr< FUICommandInfo > AlignSelected;
	// auto align selected bones and children
	TSharedPtr< FUICommandInfo > AlignSelectedAndChildren;
	// auto align selected bones
	TSharedPtr< FUICommandInfo > SnapCharacterToGround;
	
	// delete retarget pose
	TSharedPtr< FUICommandInfo > DeleteRetargetPose;
	// rename retarget pose
	TSharedPtr< FUICommandInfo > RenameRetargetPose;
	// new retarget pose
	TSharedPtr< FUICommandInfo > NewRetargetPose;
	// duplicate retarget pose
	TSharedPtr< FUICommandInfo > DuplicateRetargetPose;

	// import retarget pose
	TSharedPtr< FUICommandInfo > ImportRetargetPose;
	// import retarget pose from anim
	TSharedPtr< FUICommandInfo > ImportRetargetPoseFromAnim;
	// export retarget pose
	TSharedPtr< FUICommandInfo > ExportRetargetPose;
	// export animation
	TSharedPtr< FUICommandInfo > ExportAnimation;

	// initialize commands
	virtual void RegisterCommands() override;
};
