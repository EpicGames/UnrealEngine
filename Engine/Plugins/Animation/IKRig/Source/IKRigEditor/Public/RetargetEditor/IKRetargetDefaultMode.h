// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IKRetargetEditorController.h"

#include "Retargeter/IKRetargeter.h"
#include "IPersonaEditMode.h"

class FIKRetargetEditorController;
class FIKRetargetEditor;
class FIKRetargetPreviewScene;
struct FGizmoState;

class FIKRetargetDefaultMode : public IPersonaEditMode
{
public:
	static FName ModeName;
	
	FIKRetargetDefaultMode() = default;

	/** glue for all the editor parts to communicate */
	void SetEditorController(const TSharedPtr<FIKRetargetEditorController> InEditorController) { EditorController = InEditorController; };

	/** IPersonaEditMode interface */
	virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	virtual IPersonaPreviewScene& GetAnimPreviewScene() const override;
	/** END IPersonaEditMode interface */

	/** FEdMode interface */
	virtual void Initialize() override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return true; };
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual void Enter() override;
	virtual void Exit() override;
	/** END FEdMode interface */

private:
	void RenderDebugProxies(FPrimitiveDrawInterface* PDI, const FIKRetargetEditorController* Controller);
	
	// the skeleton currently being edited
	ERetargetSourceOrTarget SkeletonMode;
	
	/** The hosting app */
	TWeakPtr<FIKRetargetEditorController> EditorController;

	UE::Widget::EWidgetMode CurrentWidgetMode;
	bool bIsTranslating = false;

	bool bIsInitialized = false;
};
