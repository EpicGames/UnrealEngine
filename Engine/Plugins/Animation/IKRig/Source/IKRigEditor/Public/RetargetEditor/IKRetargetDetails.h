// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "RetargetEditor/IKRetargeterController.h"
#include "Retargeter/IKRetargetProcessor.h"

#include "IKRetargetDetails.generated.h"

#define UE_API IKRIGEDITOR_API

class UDebugSkelMeshComponent;

enum class EIKRetargetTransformType : int8
{
	Current,
	Reference,
	RelativeOffset,
	Bone,
};

UCLASS(MinimalAPI, config = Engine, hidecategories = UObject)
class UIKRetargetBoneDetails : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(VisibleAnywhere, Category = "Selection")
	FName SelectedBone;

	UPROPERTY()
	FTransform LocalTransform;
	
	UPROPERTY()
	FTransform OffsetTransform;
	
	UPROPERTY()
	FTransform CurrentTransform;

	UPROPERTY()
	FTransform ReferenceTransform;
	
	TWeakPtr<FIKRetargetEditorController> EditorController;

#if WITH_EDITOR

	UE_API FEulerTransform GetTransform(EIKRetargetTransformType TransformType, const bool bLocalSpace) const;
	UE_API bool IsComponentRelative(ESlateTransformComponent::Type Component, EIKRetargetTransformType TransformType) const;
	UE_API void OnComponentRelativeChanged(ESlateTransformComponent::Type Component, bool bIsRelative, EIKRetargetTransformType TransformType);
	UE_API void OnCopyToClipboard(ESlateTransformComponent::Type Component, EIKRetargetTransformType TransformType) const;
	UE_API void OnPasteFromClipboard(ESlateTransformComponent::Type Component, EIKRetargetTransformType TransformType);
	UE_API void OnNumericValueCommitted(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal Value,
		ETextCommit::Type CommitType,
		EIKRetargetTransformType TransformType,
		bool bIsCommit);

	UE_API TOptional<FVector::FReal> GetNumericValue(
		EIKRetargetTransformType TransformType,
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent);
	
	/** Method to react to changes of numeric values in the widget */
	static UE_API void OnMultiNumericValueCommitted(
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal Value,
		ETextCommit::Type CommitType,
		EIKRetargetTransformType TransformType,
		TArrayView<TObjectPtr<UIKRetargetBoneDetails>> Bones,
		bool bIsCommit);

	template<typename DataType>
	void GetContentFromData(const DataType& InData, FString& Content) const;

#endif

private:

	UE_API void CommitValueAsRelativeOffset(
		UIKRetargeterController* AssetController,
		const FReferenceSkeleton& RefSkeleton,
		const ERetargetSourceOrTarget SourceOrTarget,
		const int32 BoneIndex,
		UDebugSkelMeshComponent* Mesh,
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal Value,
		bool bShouldTransact);

	UE_API void CommitValueAsBoneSpace(
		UIKRetargeterController* AssetController,
		const FReferenceSkeleton& RefSkeleton,
		const ERetargetSourceOrTarget SourceOrTarget,
		const int32 BoneIndex,
		UDebugSkelMeshComponent* Mesh,
		ESlateTransformComponent::Type Component,
		ESlateRotationRepresentation::Type Representation,
		ESlateTransformSubComponent::Type SubComponent,
		FVector::FReal Value,
		bool bShouldTransact);

	static UE_API TOptional<FVector::FReal> CleanRealValue(TOptional<FVector::FReal> InValue);
	
	UE_API bool IsRootBone() const;

	bool BoneRelative[3] = { false, true, false };
	bool RelativeOffsetTransformRelative[3] = { false, true, false };
	bool CurrentTransformRelative[3];
	bool ReferenceTransformRelative[3];
};

struct FIKRetargetTransformWidgetData
{
	FIKRetargetTransformWidgetData(EIKRetargetTransformType InType, FText InLabel, FText InTooltip)
	{
		TransformType = InType;
		ButtonLabel = InLabel;
		ButtonTooltip = InTooltip;
	};
	
	EIKRetargetTransformType TransformType;
	FText ButtonLabel;
	FText ButtonTooltip;
};

struct FIKRetargetTransformUIData
{
	TArray<EIKRetargetTransformType> TransformTypes;
	TArray<FText> ButtonLabels;
	TArray<FText> ButtonTooltips;
	TAttribute<TArray<EIKRetargetTransformType>> VisibleTransforms;
	TArray<TSharedRef<IPropertyHandle>> Properties;
};

class FIKRetargetBoneDetailCustomization : public IDetailCustomization, FGCObject
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FIKRetargetBoneDetailCustomization);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;
	// End FGCObject interface

private:

	void GetTransformUIData(
		const bool bIsEditingPose,
		const IDetailLayoutBuilder& DetailBuilder,
		FIKRetargetTransformUIData& OutData) const;

	TArray<TObjectPtr<UIKRetargetBoneDetails>> Bones;
};

#undef UE_API
