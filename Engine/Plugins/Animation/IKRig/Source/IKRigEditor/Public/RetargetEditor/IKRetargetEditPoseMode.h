// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IKRetargetEditorController.h"

#include "Retargeter/IKRetargeter.h"
#include "IPersonaEditMode.h"
#include "Retargeter/IKRetargetProcessor.h"

class FIKRetargetEditorController;
class FIKRetargetEditor;
class FIKRetargetPreviewScene;
struct FGizmoState;

enum class FIKRetargetTrackingState : int8
{
	None,
	RotatingBone,
	TranslatingRoot,
};

struct FBoneEdit
{
	FName Name;							// name of last selected bone
	int32 Index;						// index of last selected bone
	FTransform ParentGlobalTransform;	// global transform of parent of last selected bone
	FTransform GlobalTransform;			// global transform of last selected bone
	FTransform LocalTransform;			// local transform of last selected bone
	FQuat AccumulatedGlobalOffset;		// the accumulated offset from rotation gizmo
	
	TArray<FQuat> PreviousDeltaRotation;		// the prev stored local offsets of all selected bones

	// translation edits
	FVector AccumulatedPositionOffset;	// accumulated offset since start of edit

	void Reset()
	{
		*this = FBoneEdit();
	}
};

class FIKRetargetEditPoseMode : public IPersonaEditMode
{
public:
	static FName ModeName;
	
	FIKRetargetEditPoseMode() = default;

	/** glue for all the editor parts to communicate */
	void SetEditorController(const TSharedPtr<FIKRetargetEditorController> InEditorController) { EditorController = InEditorController; };

	/** IPersonaEditMode interface */
	virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override;
	/** END IPersonaEditMode interface */

	/** FEdMode interface */
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return true; }
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool UsesTransformWidget() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual void Enter() override;
	virtual void Exit() override;
	/** END FEdMode interface */
	
private:

	// get the scale and offset associated with the currently edited skeletal mesh component
	void GetEditedComponentScaleAndOffset(float& OutScale, FVector& OutOffset) const;

	UE::Widget::EWidgetMode CurrentWidgetMode;

	bool IsRootSelected() const;
	bool IsOnlyRootSelected() const;
	void UpdateWidgetTransform();

	bool HandleBeginTransform(const FEditorViewportClient* InViewportClient);
	bool HandleEndTransform(FEditorViewportClient* InViewportClient);

	// the bone(s) currently being edited
	FBoneEdit BoneEdit;

	// the skeleton currently being edited
	ERetargetSourceOrTarget SourceOrTarget;
	
	/** The hosting app */
	TWeakPtr<FIKRetargetEditorController> EditorController;

	/** viewport selection/editing state */
	FIKRetargetTrackingState TrackingState;
};
