// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IHasPersonaToolkit.h"
#include "IKRetargetEditorController.h"
#include "IKRetargetApplicationMode.h"
#include "IPersonaPreviewScene.h"
#include "PersonaAssetEditorToolkit.h"

class IAnimationSequenceBrowser;
class UIKRetargetSkeletalMeshComponent;
class IDetailsView;
class FGGAssetEditorToolbar;
class UIKRetargeter;
class FIKRetargetToolbar;
class SIKRetargetViewportTabBody;
class FIKRetargetPreviewScene;
struct FIKRetargetPose;
class SEditableTextBox;

namespace IKRetargetApplicationModes
{
	extern const FName IKRetargetApplicationMode;
}

class FIKRetargetEditor :
	public FPersonaAssetEditorToolkit,
	public IHasPersonaToolkit,
	public FTickableEditorObject
{
public:

	FIKRetargetEditor();
	virtual ~FIKRetargetEditor() override = default;

	void InitAssetEditor(
		const EToolkitMode::Type Mode,
		const TSharedPtr< IToolkitHost >& InitToolkitHost,
		UIKRetargeter* Asset);

	/** FAssetEditorToolkit interface */
	virtual void OnClose() override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	/** END FAssetEditorToolkit interface */

	//** FTickableEditorObject Interface
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;
	//~ END FTickableEditorObject Interface

	/** IHasPersonaToolkit interface */
	virtual TSharedRef<IPersonaToolkit> GetPersonaToolkit() const override { return PersonaToolkit.ToSharedRef(); }
	/** END IHasPersonaToolkit interface */

	TSharedRef<FIKRetargetEditorController> GetController() const {return EditorController;};
	
	void SetupAnimInstance();

private:

	/** toolbar */
	void BindCommands();
	void ExtendToolbar();
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);
	/** END toolbar */
	
	/** preview scene setup */
	void HandleViewportCreated(const TSharedRef<class IPersonaViewport>& InViewport);
	void HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene);
	void HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView);
	void OnFinishedChangingDetails(const FPropertyChangedEvent& PropertyChangedEvent);
	void HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder) const;
	/** END preview scene setup */
	
	/** centralized management across all views */
	TSharedRef<FIKRetargetEditorController> EditorController;

	/** Preview scene to be supplied by IHasPersonaToolkit::GetPersonaToolkit */
	TSharedPtr<IPersonaToolkit> PersonaToolkit;

	/** record previous playback time of source anim instance to trigger reset when scrubbing / looping */
	float PreviousTime;
	
	friend FIKRetargetApplicationMode;
};
