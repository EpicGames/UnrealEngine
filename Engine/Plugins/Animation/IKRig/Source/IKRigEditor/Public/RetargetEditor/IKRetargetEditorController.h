// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IKRetargetDetails.h"
#include "IKRetargeterPoseGenerator.h"
#include "IKRetargetPoseExporter.h"
#include "IPersonaToolkit.h"
#include "SIKRetargetAssetBrowser.h"
#include "UObject/ObjectPtr.h"
#include "Input/Reply.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SWindow.h"
#include "EditorUndoClient.h"

#include "Misc/TransactionObjectEvent.h"

class SIKRetargetVariablesEditor;
class SIKRetargetOverrideManager;
class SRetargetOpStack;
enum class ERetargetSourceOrTarget : uint8;
class SIKRetargetHierarchy;
class SIKRigOutputLog;
struct FIKRetargetProcessor;
class SIKRetargetChainMapList;
class UIKRetargetAnimInstance;
class FIKRetargetEditor;
class FPrimitiveDrawInterface;
class UDebugSkelMeshComponent;
class UIKRigDefinition;
class UIKRetargeterController;
class UIKRetargetBoneDetails;
struct FRetargetSkeleton;


// retarget editor modes
enum class ERetargeterOutputMode : uint8
{
	RunRetarget,		// output the retargeted target pose
	EditRetargetPose,	// allow editing the retarget pose
};

enum class ESelectionEdit : uint8
{
	Add,	// add to selection set
	Remove,	// remove from selection
	Replace	// replace selection entirely
};

struct FBoundIKRig
{
	FBoundIKRig(UIKRigDefinition* InIKRig, const FIKRetargetEditorController& InController);
	void UnBind() const;
	
	TWeakObjectPtr<UIKRigDefinition> IKRig;
	FDelegateHandle ReInitIKDelegateHandle;
	FDelegateHandle AddedChainDelegateHandle;
	FDelegateHandle RenameChainDelegateHandle;
	FDelegateHandle RemoveChainDelegateHandle;
};

struct FRetargetPlaybackManager : public TSharedFromThis<FRetargetPlaybackManager>
{
	FRetargetPlaybackManager(const TWeakPtr<FIKRetargetEditorController>& InEditorController);
	void PlayAnimationAsset(UAnimationAsset* AssetToPlay);
	void StopPlayback();
	void PausePlayback();
	void ResumePlayback() const;
	bool IsStopped() const;
	
private:

	TWeakPtr<FIKRetargetEditorController> EditorController;
	UAnimationAsset* AnimThatWasPlaying = nullptr;
	float TimeWhenPaused = 0.0f;
	bool bWasPlayingAnim = false;
};

// a home for cross-widget communication to synchronize state across all tabs and viewport
class FIKRetargetEditorController :
	public TSharedFromThis<FIKRetargetEditorController>,
	public FSelfRegisteringEditorUndoClient,
	FGCObject 
{
public:

	virtual ~FIKRetargetEditorController() override = default;

	// Initialize the editor
	void Initialize(TSharedPtr<FIKRetargetEditor> InEditor, UIKRetargeter* InAsset);
	// Close the editor
	void Close();

	/** FSelfRegisteringEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess );
	virtual void PostRedo(bool bSuccess);
	virtual bool MatchesContext( const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts ) const;	
	/** END FSelfRegisteringEditorUndoClient interface */
	
	// Bind callbacks to this IK Rig
	void BindToIKRigAssets();
	// callback when IK Rig asset requires reinitialization
	void HandleIKRigNeedsInitialized(UIKRigDefinition* ModifiedIKRig) const;
	// callback when IK Rig asset's retarget chain's have been added or removed
	void HandleRetargetChainAdded(UIKRigDefinition* ModifiedIKRig) const;
	// callback when IK Rig asset's retarget chain has been renamed
	void HandleRetargetChainRenamed(UIKRigDefinition* ModifiedIKRig, FName OldName, FName NewName) const;
	// callback when IK Rig asset's retarget chain has been removed
	void HandleRetargetChainRemoved(UIKRigDefinition* ModifiedIKRig, const FName InChainRemoved) const;
	// callback when IK Retargeter asset requires reinitialization
	void HandleRetargeterNeedsInitialized() const;
	// reinitialize retargeter without refreshing UI
	void ReinitializeRetargeterNoUIRefresh() const;
	// delegate handles to un-bind
	FDelegateHandle RetargeterReInitDelegateHandle;
	FDelegateHandle OpStackModifiedDelegateHandle;
	FDelegateHandle OverrideSetsModifiedDelegateHandle;
	FDelegateHandle VariablesModifiedDelegateHandle;
	// callback when IK Rig asset has been swapped out
	void HandleIKRigReplaced(ERetargetSourceOrTarget SourceOrTarget);
	FDelegateHandle IKRigReplacedDelegateHandle;
	// callback when Preview Mesh asset has been swapped out
	void HandlePreviewMeshReplaced(ERetargetSourceOrTarget SourceOrTarget);
	FDelegateHandle PreviewMeshReplacedDelegateHandle;
	FDelegateHandle RetargeterInitializedDelegateHandle;
	
	// all modifications to the data model should go through this controller
	TObjectPtr<UIKRetargeterController> AssetController;

	// the persona toolkit
	TWeakPtr<FIKRetargetEditor> Editor;

	// import / export retarget poses
	TSharedPtr<FIKRetargetPoseExporter> PoseExporter;

	// manage playback of animation in the editor
	TUniquePtr<FRetargetPlaybackManager> PlaybackManager;

	// viewport skeletal mesh
	UDebugSkelMeshComponent* GetSkeletalMeshComponent(const ERetargetSourceOrTarget SourceOrTarget) const;
	UDebugSkelMeshComponent* SourceSkelMeshComponent;
	UDebugSkelMeshComponent* TargetSkelMeshComponent;
	// this root component is used as a parent of the source skeletal mesh to allow us to translate the source.
	// we can't offset the source mesh component itself because that conflicts with root motion
	USceneComponent* SourceRootComponent;

	// viewport anim instance
	UIKRetargetAnimInstance* GetAnimInstance(const ERetargetSourceOrTarget SourceOrTarget) const;
	UPROPERTY(transient, NonTransactional)
	TObjectPtr<UIKRetargetAnimInstance> SourceAnimInstance;
	UPROPERTY(transient, NonTransactional)
	TObjectPtr<UIKRetargetAnimInstance> TargetAnimInstance;
	
	// store pointers to various tabs of UI,
	// have to manage access to these because they can be null if the tabs are closed
	void SetDetailsView(const TSharedPtr<IDetailsView>& InDetailsView) { DetailsView = InDetailsView; };
	void SetAssetBrowserView(const TSharedPtr<SIKRetargetAssetBrowser>& InAssetBrowserView) { AssetBrowserView = InAssetBrowserView; };
	void SetOutputLogView(const TSharedPtr<SIKRigOutputLog>& InOutputLogView) { OutputLogView = InOutputLogView; };
	void SetHierarchyView(const TSharedPtr<SIKRetargetHierarchy>& InHierarchyView) { HierarchyView = InHierarchyView; };
	void SetOpStackView(const TSharedPtr<SRetargetOpStack>& InOpStackView) { OpStackView = InOpStackView; };
	void SetOverrideSetsView(const TSharedPtr<SIKRetargetOverrideManager>& InOverrideSetView) { OverrideSetView = InOverrideSetView; };
	void SetVariablesView(const TSharedPtr<SIKRetargetVariablesEditor>& InVariablesView) { VariablesView = InVariablesView; };
	bool IsObjectInDetailsView(const UObject* Object);
	
	// force refresh all views in the editor
	void RefreshAllViews() const;
	void RefreshDetailsView() const;
	void RefreshAssetBrowserView() const;
	void RefreshHierarchyView() const;
	void RefreshOpStackView() const;
	void RefreshOverrideSetView() const;
	void RefreshVariablesView() const;
	void RefreshPoseList() const;
	void SetDetailsObject(UObject* DetailsObject) const;
	void SetDetailsObjects(const TArray<UObject*>& DetailsObjects) const;
	void ShowDetailsForOp(const int32 OpIndex) const;

	// retargeter state
	bool IsReadyToRetarget() const;
	bool IsCurrentMeshLoaded() const;
	bool IsEditingPose() const;

	// clear the output log
	void ClearOutputLog() const;

	// get the USkeletalMesh we are transferring animation between (either source or target)
	USkeletalMesh* GetSkeletalMesh(const ERetargetSourceOrTarget SourceOrTarget) const;
	// get the USkeleton we are transferring animation between (either source or target)
	const USkeleton* GetSkeleton(const ERetargetSourceOrTarget SourceOrTarget) const;
	// get currently edited debug skeletal mesh
	UDebugSkelMeshComponent* GetEditedSkeletalMesh() const;
	// get the currently edited retarget skeleton
	const FRetargetSkeleton& GetCurrentlyEditedSkeleton(const FIKRetargetProcessor& Processor) const;
	
	// get world space pose of a bone (with component scale / offset applied)
	FTransform GetGlobalRetargetPoseOfBone(
		const ERetargetSourceOrTarget SourceOrTarget,
		const int32& BoneIndex,
		const float& Scale,
		const FVector& Offset) const;
	
	// get world space positions of all immediate children of bone (with component scale / offset applied)
	static void GetGlobalRetargetPoseOfImmediateChildren(
		const FRetargetSkeleton& RetargetSkeleton,
		const int32& BoneIndex,
		const float& Scale,
		const FVector& Offset,
		TArray<int32>& OutChildrenIndices,
		TArray<FVector>& OutChildrenPositions);

	// get the retargeter that is running in the viewport (which is a duplicate of the source asset)
	FIKRetargetProcessor* GetRetargetProcessor() const;
	// Reset the planting state of the IK (when scrubbing or animation loops over)
	void OnPlaybackReset() const;

	// Set viewport / editor tool mode
	void SetRetargeterMode(ERetargeterOutputMode Mode);
	void SetRetargetModeToPreviousMode() { SetRetargeterMode(PreviousMode); };
	ERetargeterOutputMode GetRetargeterMode() const { return OutputMode; }
	FText GetRetargeterModeLabel();
	FSlateIcon GetCurrentRetargetModeIcon() const;
	FSlateIcon GetRetargeterModeIcon(ERetargeterOutputMode Mode) const;
	float GetRetargetPoseAmount() const;
	void SetRetargetPoseAmount(float InValue);
	// END viewport / editor tool mode
	
	// general editor mode can be either viewing/editing source or target
	ERetargetSourceOrTarget GetSourceOrTarget() const { return CurrentlyEditingSourceOrTarget; };
	void SetSourceOrTargetMode(ERetargetSourceOrTarget SourceOrTarget);

	// ------------------------- SELECTION -----------------------------

	// GENERAL SELECTION
	//
	// access to all selection state for the editor
	const FIKRetargetDebugDrawState& GetSelectionState() const { return Selection; };
	void CleanSelection(ERetargetSourceOrTarget SourceOrTarget);
	void ClearSelection(const bool bKeepBoneSelection=false);
	// to frame selection when pressing "f" in viewport
	bool GetCameraTargetForSelection(FSphere& OutTarget) const;

	// SELECTION - BONES (viewport or hierarchy view)
	void EditBoneSelection(
		const TArray<FName>& InBoneNames,
		ESelectionEdit EditMode,
		const bool bFromHierarchyView = false);
	const TArray<FName>& GetSelectedBones() const {return Selection.SelectedBoneNames[CurrentlyEditingSourceOrTarget]; };
	void SetRootSelected(const bool bIsSelected);
	bool GetRootSelected() const { return Selection.bIsRootSelected; };
	bool IsEditingPoseWithAnyBoneSelected() const;
	TArray<FName> GetSelectedBonesAndChildren() const;
	// END bone selection

	// SELECTION - CHAINS (viewport or chains view)
	void EditChainSelection(
		const TArray<FName>& InChainNames,
		ESelectionEdit EditMode,
		const bool bFromChainsView);
	const TArray<FName>& GetSelectedChains() const {return Selection.SelectedChains; };
	// END chain selection

	// SELECTION - OP STACK
	void SetOpSelected(const int32 InOpIndex);
	FName GetSelectedOpName() const;
	FIKRetargetOpBase* GetSelectedOp() const;
	int32 GetSelectedOpIndex() const;
	// END op selection

	// SELECTION - OVERRIDE SETS
	FName GetSelectedOverrideSet() const;
	// END override set selection

	// ------------------------- END SELECTION -----------------------------

	// determine if bone in the specified skeleton is part of the retarget (in a mapped chain)
	bool IsBoneRetargeted(const FName& BoneName, ERetargetSourceOrTarget SourceOrTarget) const;
	// get the name of the chain that contains this bone
	FName GetChainDisplayNameForBone(const FName& BoneName, ERetargetSourceOrTarget SourceOrTarget) const;

	// factory to get/create bone details UObject
	TObjectPtr<UIKRetargetBoneDetails> GetOrCreateBoneDetailsObject(const FName& BoneName);

	// ------------------------- RETARGET POSES -----------------------------
	
	// toggle current retarget pose
	TArray<TSharedPtr<FName>> PoseNames;
	FText GetCurrentPoseName() const;
	void OnPoseSelected(TSharedPtr<FName> InPoseName, ESelectInfo::Type SelectInfo) const;

	// reset retarget pose
	void HandleResetAllBones() const;
	void HandleResetSelectedBones() const;
	void HandleResetSelectedAndChildrenBones() const;
	
	// auto generate retarget pose
	void HandleAlignBones(const bool bIncludeChildren, const bool bIncludeAllBones) const;
	void HandleSnapToGround() const;
	ERetargetAutoAlignMethod CurrentPoseAlignmentMode;

	// create new retarget pose
	void HandleNewPose();
	bool CanCreatePose() const;
	FReply CreateNewPose() const;
	TSharedPtr<SWindow> NewPoseWindow;
	TSharedPtr<SEditableTextBox> NewPoseEditableText;

	// duplicate current retarget pose
	void HandleDuplicatePose();
	FReply CreateDuplicatePose() const;

	// delete retarget pose
	void HandleDeletePose();
	bool CanDeletePose() const;

	// rename retarget pose
	void HandleRenamePose();
	FReply RenamePose() const;
	bool CanRenamePose() const;
	TSharedPtr<SWindow> RenamePoseWindow;
	TSharedPtr<SEditableTextBox> NewNameEditableText;

	// detect and auto-fix retarget pose that causes root height to be on the ground
	void FixZeroHeightRetargetRoot(ERetargetSourceOrTarget SourceOrTarget) const;
	
	// ------------------------- END RETARGET POSES -----------------------------

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("Retarget Editor"); };
	// END FGCObject interface

	// render the skeleton's in the viewport (either source or target)
	void RenderSkeleton(FPrimitiveDrawInterface* PDI, ERetargetSourceOrTarget SourceOrTarget) const;

	void UpdateSkeletalMeshComponents() const;

private:
	
	// modal dialog to ask user if they want to fix root bones that are "on the ground"
	bool PromptToFixPelvisHeight(ERetargetSourceOrTarget SourceOrTarget) const;
	bool bAskedToFixRoot = false;

	// asset properties tab
	TSharedPtr<IDetailsView> DetailsView;
	// asset browser view
	TSharedPtr<SIKRetargetAssetBrowser> AssetBrowserView;
	// output log view
	TSharedPtr<SIKRigOutputLog> OutputLogView;
	// hierarchy view
	TSharedPtr<SIKRetargetHierarchy> HierarchyView;
	// op stack widget
	TSharedPtr<SRetargetOpStack> OpStackView;
	// overrides widget
	TSharedPtr<SIKRetargetOverrideManager> OverrideSetView;
	// variables widget
	TSharedPtr<SIKRetargetVariablesEditor> VariablesView;

	// when prompting user to assign an IK Rig
	TSharedPtr<SWindow> IKRigPickerWindow;

	// the current output mode of the retargeter
	ERetargeterOutputMode OutputMode = ERetargeterOutputMode::RunRetarget;
	ERetargeterOutputMode PreviousMode;
	// slider value to blend between reference pose and retarget pose
	float RetargetPosePreviewBlend = 1.0f;
	
	// which skeleton are we editing / viewing?
	ERetargetSourceOrTarget CurrentlyEditingSourceOrTarget = ERetargetSourceOrTarget::Target;

	// current selection set
	FIKRetargetDebugDrawState Selection;
	UPROPERTY()
	TMap<FName,TObjectPtr<UIKRetargetBoneDetails>> AllBoneDetails;

	// ik rigs bound to this editor (will receive callbacks when requiring reinitialization
	TArray<FBoundIKRig> BoundIKRigs;
};

