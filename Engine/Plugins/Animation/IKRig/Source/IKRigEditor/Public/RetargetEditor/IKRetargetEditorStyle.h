// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/CoreStyle.h"

class FIKRetargetEditorStyle final	: public FSlateStyleSet
{
public:
	
	FIKRetargetEditorStyle() : FSlateStyleSet("IKRetargetEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);
		
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Animation/IKRig/Content"));
		Set("IKRetarget.Tree.Bone", new IMAGE_BRUSH("Slate/Bone_16x", Icon16x16));
		Set("ClassIcon.IKRetargeter", new IMAGE_BRUSH_SVG("Slate/IKRigRetargeter", Icon16x16));
		Set("ClassThumbnail.IKRetargeter", new IMAGE_BRUSH_SVG("Slate/IKRigRetargeter_64", Icon64x64));
		
		Set("IKRetarget.AssetSettings", new IMAGE_BRUSH_SVG("Slate/AssetSettings", Icon64x64));
		Set("IKRetarget.GlobalSettings", new IMAGE_BRUSH_SVG("Slate/GlobalSettings", Icon64x64));
		Set("IKRetarget.RootSettings", new IMAGE_BRUSH_SVG("Slate/RootSettings", Icon64x64));
		Set("IKRetarget.PostSettings", new IMAGE_BRUSH_SVG("Slate/PostSettings", Icon64x64));
		Set("IKRetarget.PostSettingsSmall", new IMAGE_BRUSH_SVG("Slate/PostSettings", Icon16x16));
		Set("IKRetarget.ChainMapping", new IMAGE_BRUSH_SVG("Slate/ChainMapping", Icon64x64));

		Set("IKRetarget.RunRetargeter", new IMAGE_BRUSH_SVG("Slate/RunRetargeter", Icon64x64));
		Set("IKRetarget.EditRetargetPose", new IMAGE_BRUSH_SVG("Slate/EditRetargetPose", Icon64x64));
		Set("IKRetarget.ShowRetargetPose", new IMAGE_BRUSH_SVG("Slate/ShowRetargetPose", Icon64x64));
		Set("IKRetarget.AutoAlign", new IMAGE_BRUSH_SVG("Slate/AutoRetargetPose", Icon16x16));
		Set("IKRetarget.Debug", new IMAGE_BRUSH_SVG("Slate/Debug", Icon16x16));

		FSlateColor OpOutlineColor = FSlateColor(FLinearColor(0.1843f, 0.1843f, 0.1843f, 0.5f));
		FSlateColor OpOutlineColorSelected = FSlateColor(FLinearColor(0.1843f, 0.1843f, 0.1843f, 1.0f));
		Set("IKRetarget.OpBorder", new FSlateRoundedBoxBrush(
			FStyleColors::Header, CoreStyleConstants::InputFocusRadius,
			OpOutlineColor, CoreStyleConstants::InputFocusThickness));
		Set("IKRetarget.OpBorderSelected", new FSlateRoundedBoxBrush(
			FStyleColors::Select, CoreStyleConstants::InputFocusRadius,
			OpOutlineColorSelected, CoreStyleConstants::InputFocusThickness));

		Set("IKRetarget.OpGroupBorder", new FSlateRoundedBoxBrush(FStyleColors::Recessed, CoreStyleConstants::InputFocusRadius));

		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		Set( "IKRetarget.Viewport.Border", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f, FLinearColor(1.0f,1.0f,1.0f,1.0f) ) );
		
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FIKRetargetEditorStyle& Get()
	{
		static FIKRetargetEditorStyle Inst;
		return Inst;
	}
	
	~FIKRetargetEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};
