// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "IKRetargetFactory.generated.h"

#define UE_API IKRIGEDITOR_API

class SWindow;

UCLASS(MinimalAPI, hidecategories=Object)
class UIKRetargetFactory : public UFactory
{
	GENERATED_BODY()

	UPROPERTY()
	TWeakObjectPtr<class UIKRigDefinition>	SourceIKRig;

public:

	UE_API UIKRetargetFactory();

	// UFactory Interface
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual uint32 GetMenuCategories() const override;
	UE_API virtual FText GetToolTip() const override;
	UE_API virtual FString GetDefaultNewAssetName() const override;
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	UE_API virtual bool ShouldShowInNewMenu() const override;
};

#undef UE_API
