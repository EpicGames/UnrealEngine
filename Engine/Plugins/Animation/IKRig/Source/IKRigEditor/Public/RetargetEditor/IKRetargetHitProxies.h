// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HitProxies.h"
#include "IKRetargetAnimInstanceProxy.h"

// allow bone selection to edit retarget pose
struct HIKRetargetEditorBoneProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	FName BoneName;
	int32 BoneIndex;
	ERetargetSourceOrTarget SourceOrTarget;
	
	HIKRetargetEditorBoneProxy(
		const FName& InBoneName,
		const int32 InBoneIndex,
		ERetargetSourceOrTarget InSourceOrTarget)
		: HHitProxy(HPP_World),
		BoneName(InBoneName),
		BoneIndex(InBoneIndex),
		SourceOrTarget(InSourceOrTarget){}

	virtual EMouseCursor::Type GetMouseCursor()
	{
		return EMouseCursor::Crosshairs;
	}

	virtual bool AlwaysAllowsTranslucentPrimitives() const override
	{
		return true;
	}
};