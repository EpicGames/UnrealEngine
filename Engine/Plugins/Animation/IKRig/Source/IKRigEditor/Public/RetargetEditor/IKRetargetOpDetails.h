// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"
#include "Retargeter/RetargetOps/AlignPoleVectorOp.h"
#include "Retargeter/RetargetOps/FKChainsOp.h"
#include "Retargeter/RetargetOps/FloorConstraintOp.h"
#include "Retargeter/RetargetOps/IKChainsOp.h"
#include "Retargeter/RetargetOps/AdditivePoseOp.h"
#include "Retargeter/RetargetOps/SpeedPlantingOp.h"
#include "Retargeter/RetargetOps/StretchChainOp.h"
#include "RigEditor/IKRigStructViewer.h"
#include "Templates/SharedPointer.h"
#include "RetargetEditor/IKRetargeterController.h"
#include "Retargeter/RetargetOps/BlendToSourceOp.h"
#include "Retargeter/RetargetOps/OffsetGoalsOp.h"
#include "Retargeter/RetargetOps/RunIKRigOp.h"
#include "Retargeter/RetargetOps/ScaleGoalsOp.h"
#include "IKRetargetOpDetails.generated.h"

enum class EFKChainRotationMode : uint8;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;

class FIKRetargetOpBaseSettingsCustomization : public IPropertyTypeCustomization
{
public:

	// IPropertyTypeCustomization
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InStructPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InStructCustomizationUtils) override {};
	// End IPropertyTypeCustomization
	
	bool LoadAndValidateStructToCustomize(
		const TSharedRef<IPropertyHandle>& InStructPropertyHandle,
		const IPropertyTypeCustomizationUtils& InStructCustomizationUtils);

	static void AddPropertyToGroup(
		const TSharedRef<IPropertyHandle>& InPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder);

	static void AddChildPropertiesToCategoryGroups(
		const TSharedRef<IPropertyHandle>& InParentPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder);

	static void AddChildPropertiesInCategory(
		const TSharedRef<IPropertyHandle>& InParentPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		const FName& InCategoryName,
		const TArray<FName>& InPropertiesToIgnore = {});

	static void AddBaseOpProperties(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder);

protected:

	using FChainSettingsGetter = TFunction<UObject*(const FName)>;
	
	template<typename TSettings, typename TWrapper>
	FChainSettingsGetter TMakeChainSettingsGetter(
	const FName InOpName,
	const FName SettingsMemberPropertyName,
	const TArray<FName>& HiddenProps = {})
	{
		static_assert(TIsDerivedFrom<TWrapper, UIKRigStructWrapperBase>::IsDerived,"TWrapper must derive from UIKRigStructWrapperBase");

		UIKRetargeterController* Controller = AssetController;
		
		return [Controller, InOpName, SettingsMemberPropertyName, HiddenProps](const FName InTargetChainName) -> UObject*
			{
				FIKRigStructToView StructToView;
				StructToView.Owner = Controller->GetAsset();
				StructToView.Type = TSettings::StaticStruct();
				StructToView.UniqueName = InOpName;
				StructToView.MemoryProvider = [Controller, InOpName, InTargetChainName]() -> uint8*
					{
						return Controller->GetChainSettingsMemory(InOpName, InTargetChainName);
					};

				TWrapper* Wrapper = NewObject<TWrapper>(GetTransientPackage(), NAME_None, RF_Standalone);
				Wrapper->InitializeWithRetargeter(StructToView, SettingsMemberPropertyName, Controller);

				for (const FName Prop : HiddenProps)
				{
					Wrapper->SetPropertyHidden(Prop, true);
				}

				return Wrapper;
			};
	}

	// the op that owns the settings being customized / edited
	FName OpName;
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
	UIKRigStructViewer* StructViewer;
	UIKRetargeter* RetargetAsset;
	UIKRetargeterController* AssetController;
};

//
// FK Chain Op Customization
//

class FChainsFKOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FChainsFKOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class URetargetFKChainSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "FK Chain Settings")
	FRetargetFKChainSettings Settings;
};

//
// Run IK Op Customization
//

class FRunIKRigOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FRunIKRigOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class URunIKRigSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "IK Chain Settings")
	FIKRetargetRunIKRigChainSettings Settings;
};

//
// IK Chains Op Customization
//

class FIKChainOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FIKChainOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class URetargetIKChainSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "IK Chain Settings")
	FRetargetIKChainSettings Settings;
};

//
// Stride Warp Op Customization
//

class FStrideWarpOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FStrideWarpOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class URetargetStrideWarpSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Stride Warp Settings")
	FRetargetStrideWarpChainSettings Settings;
};

//
// Speed Plant Op Customization
//

class FSpeedPlantOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FSpeedPlantOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class URetargetSpeedPlantSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Stride Warp Settings")
	FRetargetSpeedPlantingSettings Settings;
};

//
// Pole Vector Op Customization
//

class FPoleVectorOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FPoleVectorOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class UPoleVectorSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Pole Vector Settings")
	FRetargetPoleVectorSettings Settings;
};

//
// Retarget Pose Op Customization
//

class FAdditivePoseOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FAdditivePoseOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization

private:
	
	void UpdatePoseNameOptions();

	TSharedPtr<IPropertyHandle> PoseToApplyProperty;
	
	TSharedPtr<FName> CurrentPoseOption;
	TArray<TSharedPtr<FName>> PoseNameOptions;
};

UCLASS()
class URetargetPoseOpSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Retarget Pose Settings")
	FIKRetargetAdditivePoseOpSettings Settings;
};

//
// Stretch Chain Op Customization
//

class FStretchChainOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FStretchChainOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class UStretchChainSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Stretch Chain Settings")
	FRetargetStretchChainSettings Settings;
};

//
// Floor Constraint Op Customization
//

class FFloorConstraintOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FFloorConstraintOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class UFloorConstraintSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Floor Constraint Chain Settings")
	FFloorConstraintChainSettings Settings;
};

//
// Blend to Source Op Customization
//

class FBlendToSourceOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FBlendToSourceOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class UBlendToSourceSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Blend to Source Chain Settings")
	FIKRetargetBlendToSourceChainSettings Settings;
};

//
// Offset Goals Op Customization
//

class FOffsetGoalsOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FOffsetGoalsOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class UOffsetGoalsSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Offset Goals Chain Settings")
	FIKRetargetOffsetGoalsChainSettings Settings;
};

//
// Scale Goals Op Customization
//

class FScaleGoalsOpCustomization : public FIKRetargetOpBaseSettingsCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShared<FScaleGoalsOpCustomization>();
	}

	// IPropertyTypeCustomization
	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> StructPropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// End IPropertyTypeCustomization
};

UCLASS()
class UScaleGoalsSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Scale Goals Chain Settings")
	FIKRetargetScaleGoalsChainSettings Settings;
};
