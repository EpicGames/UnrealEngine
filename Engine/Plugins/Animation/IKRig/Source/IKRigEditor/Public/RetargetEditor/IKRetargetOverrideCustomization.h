// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"
#include "Widgets/Input/SMenuAnchor.h"

class UIKRetargeterController;
class IDetailGroup;
class FStructOnScope;

// This builds a custom details view for FRetargetOverrideSet types.
// It supports editing retarget op settings overrides stored within a retarget override set
class FRetargetOverrideCustomization : public IPropertyTypeCustomization
{
	
public:
	
	static TSharedRef<IPropertyTypeCustomization> MakeInstance() 
	{ 
		return MakeShared<FRetargetOverrideCustomization>(); 
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:

	/** We customize the FRetargetOverrideSet details differently when outside the retarget editor context. */
	void CustomizeOutsideEditor(TSharedRef<IPropertyHandle> OverrideSetHandle, IDetailChildrenBuilder& ChildBuilder);

	/** Get the controller for the retarget asset by traversing through the UIKRigStructViewer being customized
	 * @return a pointer to the asset controller this set belongs to (or nullptr if outside editor) */
	TObjectPtr<UIKRetargeterController> GetAssetController(const IPropertyTypeCustomizationUtils& CustomizationUtils);
	
	/**
	* Injects a custom row for a property override into a detail group.
	* @param InPropertyHandle  Handle to the property being displayed.
	* @param InPropertyGroup   The detail group where this row will be added.
	* @param InOverrideSetName The name of the current override set.
	* @param InOpName          The name of the retarget op the override belongs to.
	* @param InPropertyPath    The full path to the property within the op's settings.
	*/
	void AddRowForPropertyOverride(
		TSharedPtr<IPropertyHandle> InPropertyHandle,
		IDetailGroup& InPropertyGroup,
		const FName& InOverrideSetName,
		const FName& InOpName,
		const FString& InPropertyPath);

	/** Generates the binding dropdown menu content for a specific property override */
	TSharedRef<SWidget> GetBindingMenuContent(
		TSharedPtr<IPropertyHandle> InPropertyHandle, 
		FName InOverrideSetName, 
		FName InOpName, 
		FString InPropertyPath);

	/**
	* Iterates through the overrides in an override set and groups them by their property path.
	* @param OpOverridesHandle  Handle to the FRetargetOpOverrides struct.
	* @return A map where keys are group names and values are indices into the override array.
	*/
	TMap<FString, TArray<int32>> MapOverridesToGroups(const TSharedPtr<IPropertyHandle>& OpOverridesHandle) const;

	/**
	* Retrieves or creates a cached FStructOnScope for a specific retarget op.
	* Used for temporary value manipulation or default value comparisons.
	* @param InOpName          The name of the operation.
	* @param OpSettingsType    The script struct type to instantiate if not cached.
	* @return A reference to the shared pointer containing the scoped struct.
	*/
	TSharedPtr<FStructOnScope>& GetCachedScopedStruct(const FName& InOpName, const UScriptStruct* OpSettingsType);

	/**
	* Copies a serialized string value from a property handle into the corresponding property of a live struct instance.
	* @param InScopedStruct     The destination struct instance.
	* @param OpSettingsType     The struct type definition.
	* @param InPropertyPath     The path to the property to be updated.
	* @param InRawValueHandle   The handle containing the source serialized value.
	* @return True if the value was successfully resolved and copied.
	*/
	bool CopyStoredValueToScopedStruct(
		const TSharedPtr<FStructOnScope>& InScopedStruct,
		const UScriptStruct* OpSettingsType,
		const FString& InPropertyPath,
		const TSharedPtr<IPropertyHandle>& InRawValueHandle);

	/**
	* Resolves the underlying UScriptStruct type for a retarget operation's settings.
	* @param OpOverridesHandle   Handle to the FRetargetOpOverrides.
	* @return The UScriptStruct pointer if found, otherwise nullptr.
	*/
	const UScriptStruct* GetSettingsStruct(TSharedPtr<IPropertyHandle>& OpOverridesHandle) const;

	/**
	* Navigates a property handle hierarchy based on a string path.
	* @param InRootHandle      The starting point for the search.
	* @param InPath            The string path (e.g., "Settings->SubStruct->Value").
	* @return A handle to the leaf property if the path is valid, otherwise null.
	*/
	TSharedPtr<IPropertyHandle> ResolveHandleFromPath(TSharedPtr<IPropertyHandle> InRootHandle, const FString& InPath);

	/**
	* Extracts the property path string from a specific override entry in the override set.
	* @param InOpOverridesHandle Handle to the parent FRetargetOpOverrides.
	* @param InOverrideIndex   Index of the override in the TArray.
	* @param OutPath           [Out] The resulting property path string.
	*/
	void GetOverridePropertyPath(
		const TSharedPtr<IPropertyHandle>& InOpOverridesHandle,
		const int32 InOverrideIndex,
		FString& OutPath) const;

	/**
	* Gets the property handle for the 'Value' field of a specific override entry.
	* @param InOpOverridesHandle Handle to the parent FRetargetOpOverrides.
	* @param InOverrideIndex   Index of the override in the TArray.
	* @return The handle to the override's value string.
	*/
	TSharedPtr<IPropertyHandle> GetOverrideValueHandle(
		const TSharedPtr<IPropertyHandle>& InOpOverridesHandle,
		const int32 InOverrideIndex) const;
	
	/** Stores the binary data for temp op settings structs while being edited <OpName, OpSettingsMemory> */
	TMap<FName, TSharedPtr<FStructOnScope>> ScopedInstances;

	/** Stored and referenced by UI callbacks for transactional edits to override sets. */
	TWeakObjectPtr<UIKRetargeterController> AssetController;
};
