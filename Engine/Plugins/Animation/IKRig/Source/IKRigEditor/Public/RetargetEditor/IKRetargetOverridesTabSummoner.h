// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FIKRetargetEditor;

/** Summons the "Retarget Override Sets" tab in the IK Retarget editor. */
struct FIKRetargetOverridesTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
	FIKRetargetOverridesTabSummoner(const TSharedRef<FIKRetargetEditor>& InIKRetargetEditor);
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FIKRetargetEditor> IKRetargetEditor;
};

