// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Notifications/SNotificationList.h"

class FIKRetargetEditorController;

class FIKRetargetPoseExporter : public TSharedFromThis<FIKRetargetPoseExporter>
{
public:
	
	FIKRetargetPoseExporter() = default;

	void Initialize(TSharedPtr<FIKRetargetEditorController> InController);

	/** import retarget pose from asset*/
	void HandleImportFromPoseAsset();
	FReply ImportPoseAsset() const;
	bool OnShouldFilterPoseToImport(const struct FAssetData& AssetData) const;
	void OnPoseAssetSelected(const FAssetData& SelectedAsset);
	TSharedRef<SWidget> OnGeneratePoseComboWidget(TSharedPtr<FName> Item) const;
	void OnSelectPoseFromPoseAsset(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo);
	void RefreshPoseList();
	FSoftObjectPath RetargetPoseToImport;
	TSharedPtr<SWindow> ImportPoseWindow;
	TArray<TSharedPtr<FName>> PosesInSelectedAsset;
	TSharedPtr<FName> SelectedPose;

	/** import retarget pose from animation sequence*/
	void HandleImportFromSequenceAsset();
	bool OnShouldFilterSequenceToImport(const struct FAssetData& AssetData) const;
	FReply OnImportPoseFromSequence();
	void OnSequenceSelectedForPose(const FAssetData& SelectedAsset);
	TSharedPtr<SWindow> ImportPoseFromSequenceWindow;
	FSoftObjectPath SequenceToImportAsPose;
	int32 FrameOfSequenceToImport;
	FText ImportedPoseName;

	/** export retarget pose to asset*/
	void HandleExportPoseAsset();

private:

	static void NotifyUser(const FText& Message, SNotificationItem::ECompletionState NotificationType);
	
	TWeakPtr<FIKRetargetEditorController> Controller;
};
