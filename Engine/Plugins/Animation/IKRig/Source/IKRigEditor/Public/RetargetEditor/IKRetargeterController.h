// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "RetargetEditor/IKRetargeterPoseGenerator.h"
#include "Retargeter/IKRetargeter.h"

#include "IKRetargeterController.generated.h"

#define UE_API IKRIGEDITOR_API

struct FRetargetGlobalSettings;
struct FTargetRootSettings;
struct FTargetChainSettings;
struct FIKRetargetPose;
struct FIKRetargetOpBase;
enum class ERetargetSourceOrTarget : uint8;
enum class ERetargetAutoAlignMethod : uint8;
class FIKRetargetEditorController;
class URetargetChainSettings;
class UIKRigDefinition;
class UIKRetargeter;
class USkeletalMesh;
class UIKRigStructViewer;

struct FEdGraphPinType;

// used to index into the map of cached struct viewers in the editor
UENUM()
enum class ERetargetStructViewerMode : uint8
{
	Settings,
	OverrideSets
};

// A stateless singleton (1-per-asset) class used to make modifications to a UIKRetargeter asset.
// Use UIKRetargeter.GetController() to get the controller for the asset you want to modify.  
UCLASS(MinimalAPI, BlueprintType, hidecategories = UObject)
class UIKRetargeterController : public UObject
{
	GENERATED_BODY()

public:

	UE_API UIKRetargeterController();

	// UObject
	UE_API virtual void PostInitProperties() override;
	
	// Get access to the retargeter asset.
	// Warning: Do not make modifications to the asset directly. Using the controller API guarantees correctness.
	UE_API UIKRetargeter* GetAsset() const;
  
private:
	
	// The actual asset that this Controller modifies. This is the only field this class should have.
	TObjectPtr<UIKRetargeter> Asset = nullptr;

public:

	//
	// GENERAL PUBLIC/SCRIPTING API
	//
	
	/** Get the controller for the given retargeter asset
	 * @param InRetargeterAsset an IK Retarget asset
	 * @return the controller with an API for modifying the given retarget asset */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe, DisplayName = "Get IK Retarget Controller"))
	static UE_API UIKRetargeterController* GetController(const UIKRetargeter* InRetargeterAsset);
	
	/** Set the IK Rig to use as the source or target (to copy animation FROM/TO)
	 * @param SourceOrTarget an enum specifying either "Source" or "Target"
	 * @param the IK Rig asset to apply as either the "Source" (to copy FROM) or "Target" (to copy TO)*/
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe, DisplayName = "Set IK Rig"))
	UE_API void SetIKRig(const ERetargetSourceOrTarget SourceOrTarget, UIKRigDefinition* IKRig) const;
	
	/** Get either source or target IK Rig
	 * @param SourceOrTarget an enum specifying either "Source" or "Target"
	 * @return the IK Rig asset associated with either the Source or Target of the IK Retargeter */ 
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe, DisplayName = "Get IK Rig"))
	UE_API const UIKRigDefinition* GetIKRig(const ERetargetSourceOrTarget SourceOrTarget) const;
	
	/** Get all target IK Rigs referenced by all ops
	 * @return an array of all target IK Rig assets associated with all the ops in the stack */ 
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API TArray<UIKRigDefinition*> GetAllTargetIKRigs() const;

	/** Set the preview skeletal mesh for either source or target
	* @param SourceOrTarget an enum specifying either "Source" or "Target"
	* @param InPreviewMesh a skeletal mesh asset to use as the preview mesh in the retarget editor (may be used on other meshes at runtime) */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API void SetPreviewMesh(const ERetargetSourceOrTarget SourceOrTarget, USkeletalMesh* InPreviewMesh) const;
	
	/** Get the preview skeletal mesh
	 * @param SourceOrTarget an enum specifying either "Source" or "Target"
	 * @return the skeletal mesh asset currently being used as a preview mesh */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API USkeletalMesh* GetPreviewMesh(const ERetargetSourceOrTarget SourceOrTarget) const;

	//
	// RETARGET OPS PUBLIC/SCRIPTING API
	//

	/** Add a new retarget op of the given type to the bottom of the stack. Returns the stack index.
	 * @param InIKRetargetOpType: the full package path of the UStruct type, ie /Script/IKRig.IKRetargetPinBoneOp */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget")
	UE_API int32 AddRetargetOp(const FString InIKRetargetOpType) const;

	/** Remove the retarget op at the given stack index
	 * NOTE: if this op is a parent, all it's children will be removed as well
	 * @param InOpIndex: the index of the op to remove
	 * @return true if op was found and removed, false otherwise */ 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget")
	UE_API bool RemoveRetargetOp(const int32 InOpIndex) const;

	/** Remove all ops in the stack.
	 * @return true if any ops were removed, false otherwise */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API bool RemoveAllOps() const;

	/** Set the name of the op at the given index in the stack.
	 * @param InName the new name to use
	 * @param InOpIndex the index of the op to be renamed
	 * @return the new name of the op (after it's made unique) */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API FName SetOpName(const FName InName, const int32 InOpIndex) const;
	
	/** Get the name of the op at the given index in the stack.
	 * @param InOpIndex the index of the op to get the name of
	 * @return the name of the op or None if the index was invalid */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API FName GetOpName(const int32 InOpIndex) const ;

	/** Set the name of the op to parent this op to
	 * Children ops are forced to execute before their parent
	 * @param InChildOpName the op to be parented
	 * @param InParentOpName the name of the op to parent to
	 * @return true if both parent and child were found, and the parent relationship was allowed */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API bool SetParentOpByName(const FName InChildOpName, const FName InParentOpName) const;
	
	/** Get the name of the parent op for the given op.
	 * @param InOpName the name of the op to get the parent of
	 * @return the name of the parent op or None if the op was not found or did not have a parent */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API FName GetParentOpByName(const FName InOpName) const ;

	/** Get the index of an op.
	 * @param InOpName the name of the op to get the index of
	 * @return the integer index of the op in the stack or -1 if op not found. */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API int32 GetIndexOfOpByName(const FName InOpName) const;

	/** Automatically add basic retargeting operations
	 * Adds these ops in the following order: Pelvis Motion, FK Chains, IK Chains, IK Solve and Root Motion
	 * If any of these ops are already present, they will not be re-added.
	 * @param bRunInitialSetup: if true, will run the initial setup routine on each op after they are all added to the stack. */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API void AddDefaultOps() const;

	/** Force the op to run the initial setup. This is normally run when an Op is added through the editor UI.
	 * NOTE: ops may not have custom initial setup routines; the exact behavior is op-dependent
	 * @param InOpIndex the index of the op to run setup on */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API void RunOpInitialSetup(const int32 InOpIndex) const;

	/** Force all ops to use the assigned IK Rig and update their chain mappings.
	 * NOTE: some ops may not reference an IK Rig at all; the exact behavior is op-dependent
	 * @param InSourceOrTarget whether to assign the provided IK Rig as a source or target IK Rig
	 * @param InIKRig the IK Rig asset to assign */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API void AssignIKRigToAllOps(const ERetargetSourceOrTarget InSourceOrTarget, const UIKRigDefinition* InIKRig) const;

	// Tell each op to reset any settings associated with this chain
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API void ResetChainSettingsInAllOps(const FName InTargetChainName) const;

	/** Get the number of Ops in the stack.
	 * @return int, the number of ops */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API int32 GetNumRetargetOps() const;

	/** Move the retarget op at the given index to the target index.
	 * NOTE: due to constraints on execution order, the actual index may differ from what is requested
	 * @InOpToMoveIndex the index of the op to be moved
	 * @InTargetIndex the index where the op should be moved to */ 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget")
	UE_API bool MoveRetargetOpInStack(int32 InOpToMoveIndex, int32 InTargetIndex) const;

	/** Toggle an op on/off.
	 * @param InRetargetOpIndex the index of the op to modify
	 * @param bIsEnabled if true, turns the op On, else Off
	 * @return true if op was found at index */ 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget")
	UE_API bool SetRetargetOpEnabled(int32 InRetargetOpIndex, bool bIsEnabled) const;

	/** Get enabled status of the given Op.
	 * @param InRetargetOpIndex the index of the op to get the enabled state for
	 * @return true if op is enabled, false if op is disabled or not found */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API bool GetRetargetOpEnabled(int32 InRetargetOpIndex) const;
	
	/** Get a controller to get/set the settings for a given op in the stack
	 * NOTE: the returned UIKRetargetOpControllerBase* should be cast to the type specific to your op to get full functionality for that op.
	 * @param InOpIndex: the index of the op in the stack to return a controller for
	 * @return a pointer to the base op controller UClass or null if the op was not found */
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API UIKRetargetOpControllerBase* GetOpController(int32 InOpIndex);

	//
	// RETARGET OPS C++ ONLY API
	//
	
	// add a new retarget op to the stack
	UE_API int32 AddRetargetOp(const UScriptStruct* InRetargetOpType, const FName InParentOpName = NAME_None) const;

	// Gather up all the ops that are children of this op
	UE_API TArray<int32> GetChildOpIndices(const int32 InOpIndex) const;

	// Returns true if this op type can contain child ops (ops forced to be ordered before the parent)
	UE_API bool GetCanOpHaveChildren(const int32 InOpIndex) const;

	// Get the index of the parent op or INDEX_NONE if it's root level
	UE_API int32 GetParentOpIndex(const int32 InOpIndex) const;

	// Return NameToMakeUnique with a numbered suffix that makes it unique in the stack
	UE_API FName GetUniqueOpName(const FName NameToMakeUnique, int32 OpIndexToIgnore) const;

	// Get access to the first retarget operation of the given type
	template <typename T>
	T* GetFirstRetargetOpOfType() const
	{
		return Asset->GetFirstRetargetOpOfType<T>();
	}

	// Get access to the given retarget operation by index.
	UE_API FIKRetargetOpBase* GetRetargetOpByIndex(const int32 InOpIndex) const;

	// Get access to the given retarget operation by name.
	UE_API FIKRetargetOpBase* GetRetargetOpByName(const FName InOpName) const;

	// Get access to the given retarget operation.
	UE_API FInstancedStruct* GetRetargetOpStructAtIndex(int32 Index) const;

	// Get the index of a given retarget operation. 
	UE_API int32 GetIndexOfRetargetOp(FIKRetargetOpBase* RetargetOp) const;

	// A callback whenever the property of an op is modified
	UE_API void OnOpPropertyChanged(const FName& InOpName, const FPropertyChangedEvent& InPropertyChangedEvent) const;

	// Return pointer to the memory associated with the given chain on the given op.
	UE_API uint8* GetChainSettingsMemory(const FName& InOpName, const FName InChainName) const;

	//
	// GENERAL C++ ONLY API
	//
	
	// Ensures all internal data is compatible with assigned meshes and ready to edit.
	// - Removes bones from retarget poses that are no longer in skeleton
	// - Removes chain settings for chains that are no longer in target IK Rig
	UE_API void CleanAsset() const;
	
	// Get either source or target IK Rig 
	UE_API UIKRigDefinition* GetIKRigWriteable(const ERetargetSourceOrTarget SourceOrTarget) const;
	
	// Get if we've already asked to fix the root height for the given skeletal mesh 
	UE_API bool GetAskedToFixRootHeightForMesh(USkeletalMesh* Mesh) const;
	
	// Set if we've asked to fix the root height for the given skeletal mesh 
	UE_API void SetAskedToFixRootHeightForMesh(USkeletalMesh* Mesh, bool InAsked) const;
	
	// Get name of the Root bone used for retargeting. 
	UE_API FName GetPelvisBone(const ERetargetSourceOrTarget SourceOrTarget) const;

	//
	// RETARGET CHAIN MAPPING PUBLIC/SCRIPTING API
	//
	
	/** Use string comparision to find "best" Source chain to map to each Target chain or clear the mappings
	 * @param AutoMapType an enum specifying the type of mapping to perform (ie, Exact, Fuzzy or Clear)
	 * @param bForceRemap if false, will only remap those mappings that unset/None
	 * @param InOpName if specified, will auto-map only the chain mapping in the specified op, otherwise applies to all ops*/
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API void AutoMapChains(const EAutoMapChainType AutoMapType, const bool bForceRemap, const FName InOpName=NAME_None) const;

	/** Assign a source chain to the given target chain. Animation will be copied from the source to the target.
	 * @param InSourceChainName the name of the source retarget chain to assign
	 * @param InTargetChainName the name of the target chain to map
	 * @param InOpName if specified, will set the source chain only for that op (otherwise applies to all ops with a chain mapping)*/
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API bool SetSourceChain(FName InSourceChainName, FName InTargetChainName, const FName InOpName=NAME_None) const;

	/** Get the name of the source chain mapped to a given target chain (the chain animation is copied FROM).
	 * @param InTargetChainName the name of the target retarget chain to get a mapping for
	 * @param InOpName if specified, will get the source chain for that op (otherwise gets the first chain mapping in the op stack)
	 * @return the name of the source chain mapped to the given target chain (or None if not mapped) */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API FName GetSourceChain(const FName& InTargetChainName, const FName InOpName=NAME_None) const;

	/** Reset the settings for the given chain in the given op back to the defaults.
	 * @param InTargetChainName the name of the chain
	 * @param InOpName the name of the op containing the chain settings */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API void ResetChainSettingsToDefault(const FName InTargetChainName, const FName InOpName) const;

	/** Get the target IK Rig associated with the given Op. May be different than the default IK Rig.
	 * NOTE: all ops use the global SOURCE IK rig, but each op may use its own custom TARGET IK Rig.
	 * NOTE: not all ops maintain their own target IK Rig in which case this function returns nullptr
	 * NOTE: some ops may refer to the target IK Rig used by their parent op. In that case, this function returns the parent Op's IK Rig.
	 * @param InOpName the name of the retarget op
	 * @return the IK Rig assset associated with this op */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API const UIKRigDefinition* GetTargetIKRigForOp(const FName InOpName) const;

	//
	// RETARGET CHAIN MAPPING C++ ONLY API
	//

	/** Get access to the mapping between source/target chains for the particular op (may be null)
	 * If no op name is specified, it returns the first chain mapping it finds*/
	UE_API const FRetargetChainMapping* GetChainMapping(const FName InOpName = NAME_None) const;

	// Clean all chain mappings in all ops, or if InOpName is specified, just that op
	UE_API void CleanChainMaps(const FName InOpName=NAME_None) const;
	
	// Ask if the given op has settings for the given chain that are not at the default
	UE_API bool AreChainSettingsAtDefault(const FName InTargetChainName, const FName InOpName) const;
	
	// Get whether the given chain's IK goal is connected to a solver 
	UE_API bool IsChainGoalConnectedToASolver(const FName& GoalName) const;
	
	// Call this when IK Rig chain is added or removed. 
	UE_API void HandleRetargetChainAdded(UIKRigDefinition* IKRig) const;
	
	// Call this when IK Rig chain is renamed. Retains existing mappings using the new name 
	UE_API void HandleRetargetChainRenamed(UIKRigDefinition* InIKRig, FName OldChainName, FName NewChainName) const;
	
	// Call this when IK Rig chain is removed. 
	UE_API void HandleRetargetChainRemoved(UIKRigDefinition* IKRig, const FName& InChainRemoved) const;

	//
	// RETARGET POSE PUBLIC/SCRIPTING API
	//
	
	/** Add new retarget pose.
	 * @param NewPoseName The name to assign to the new retarget pose
	 * @param SourceOrTarget Specifies whether the pose applies to the source or target skeleton
	 * @return The name of the newly created retarget pose */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API FName CreateRetargetPose(const FName& NewPoseName, const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Remove a retarget pose.
	 * @param PoseToRemove The name of the retarget pose to remove
	 * @param SourceOrTarget Specifies whether the pose belongs to the source or target skeleton
	 * @return True if the pose was found and successfully removed, false otherwise */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API bool RemoveRetargetPose(const FName& PoseToRemove, const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Duplicate a retarget pose.
	 * @param PoseToDuplicate The name of the retarget pose to duplicate
	 * @param NewName The name to assign to the duplicated pose
	 * @param SourceOrTarget Specifies whether the pose belongs to the source or target skeleton
	 * @return The name of the new duplicated pose, or NAME_None if PoseToDuplicate is not found */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API FName DuplicateRetargetPose(const FName PoseToDuplicate, const FName NewName, const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Rename current retarget pose.
	 * @param OldPoseName The current name of the retarget pose to rename
	 * @param NewPoseName The new name to assign to the retarget pose
	 * @param SourceOrTarget Specifies whether the pose belongs to the source or target skeleton
	 * @return True if the pose was found and successfully renamed, false otherwise */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API bool RenameRetargetPose(const FName OldPoseName, const FName NewPoseName, const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Change which retarget pose is used by the retargeter at runtime.
	 * @param CurrentPose The name of the retarget pose to set as current
	 * @param SourceOrTarget Specifies whether the pose applies to the source or target skeleton
	 * @return True if the current pose was successfully set, false otherwise */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API bool SetCurrentRetargetPose(FName CurrentPose, const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Get the name of the current retarget pose.
	 * @param SourceOrTarget Specifies whether to retrieve the pose from the source or target skeleton
	 * @return The name of the current retarget pose */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API FName GetCurrentRetargetPoseName(const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Get access to array of retarget poses.
	 * @param SourceOrTarget Specifies whether to retrieve poses from the source or target skeleton
	 * @return A reference to the map of retarget pose names to their pose data */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API TMap<FName, FIKRetargetPose>& GetRetargetPoses(const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Get the current retarget pose.
	 * @param SourceOrTarget Specifies whether to retrieve the pose from the source or target skeleton
	 * @return A reference to the current retarget pose data */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API FIKRetargetPose& GetCurrentRetargetPose(const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Reset a retarget pose for the specified bones.
	 * @param PoseToReset The name of the retarget pose to reset
	 * @param BonesToReset The array of bone names to reset; if empty, resets all bones to the reference pose
	 * @param SourceOrTarget Specifies whether the pose belongs to the source or target skeleton */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API void ResetRetargetPose(const FName& PoseToReset, const TArray<FName>& BonesToReset, const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Set a delta rotation for a given bone in the current retarget pose.
	 * @param BoneName The name of the bone to apply the rotation offset to
	 * @param RotationOffset The quaternion representing the rotation offset to apply
	 * @param SkeletonMode Specifies whether the bone belongs to the source or target skeleton */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API void SetRotationOffsetForRetargetPoseBone(const FName& BoneName, const FQuat& RotationOffset, const ERetargetSourceOrTarget SkeletonMode) const;

	/** Get a delta rotation for a given bone in the current retarget pose.
	 * @param BoneName The name of the bone to retrieve the rotation offset for
	 * @param SourceOrTarget Specifies whether the bone belongs to the source or target skeleton
	 * @return The quaternion representing the rotation offset for the specified bone */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API FQuat GetRotationOffsetForRetargetPoseBone(const FName& BoneName, const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Set the translation offset on the retarget pelvis bone for the current retarget pose.
	 * @param TranslationOffset The vector representing the translation offset to apply to the pelvis bone
	 * @param SourceOrTarget Specifies whether the pelvis bone belongs to the source or target skeleton */
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta =(BlueprintThreadSafe))
	UE_API void SetRootOffsetInRetargetPose(const FVector& TranslationOffset, const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Get the translation offset on the retarget pelvis bone for the current retarget pose.
	 * @param SourceOrTarget Specifies whether the pelvis bone belongs to the source or target skeleton
	 * @return The vector representing the translation offset of the pelvis bone */
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API FVector GetRootOffsetInRetargetPose(const ERetargetSourceOrTarget SourceOrTarget) const;

	/** Automatically align all bones in mapped chains and store in the current retarget pose.
	 * @param SourceOrTarget Specifies whether to align bones in the source or target skeleton
	 * @param Method The method to use for automatic alignment (defaults to ChainToChain) */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API void AutoAlignAllBones(ERetargetSourceOrTarget SourceOrTarget, const ERetargetAutoAlignMethod Method = ERetargetAutoAlignMethod::ChainToChain) const;

	/** Automatically align an array of bones and store in the current retarget pose.
	 * @param BonesToAlign The array of bone names to align; bones not in mapped chains are ignored
	 * @param Method The method to use for automatic alignment
	 * @param SourceOrTarget Specifies whether to align bones in the source or target skeleton */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API void AutoAlignBones(const TArray<FName>& BonesToAlign, const ERetargetAutoAlignMethod Method, ERetargetSourceOrTarget SourceOrTarget) const;

	/** Moves the entire skeleton vertically until the specified bone is the same height off the ground as in the reference pose.
	 * @param ReferenceBone The name of the bone to use as a reference for ground snapping
	 * @param SourceOrTarget Specifies whether the skeleton belongs to the source or target */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API void SnapBoneToGround(FName ReferenceBone, ERetargetSourceOrTarget SourceOrTarget);

	//
	// RETARGET POSE C++ ONLY API
	//
	
	// Add a numbered suffix to the given pose name to make it unique. 
	UE_API FName MakePoseNameUnique(
		const FString& PoseName,
		const ERetargetSourceOrTarget SourceOrTarget) const;

	TWeakPtr<const FIKRigSkeleton> GetCachedIKRigSkeleton(const USkeletalMesh* InSkeletalMesh) const;

	//
	// RETARGET OVERRIDES PUBLIC/SCRIPTING API
	//

	/** Add a new override set to store retarget op setting overrides.
	* @param InOverrideSetName    The name of the new set.
	* @return The name of the new retarget override set (guaranteed unique). */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API FName AddNewRetargetOverrideSet(FName InOverrideSetName);

	/** Remove an override set by name.
	* @param InOverrideSetName    The name of the override set to remove.
	* @return True if the override set was removed, false otherwise. */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API bool RemoveRetargetOverrideSet(FName InOverrideSetName);

	/** Rename an existing override set.
	* @param InOverrideSetName    The name of the override set to rename.
	* @param InNewName            The new name.
	* @return True if the override set was renamed, false otherwise. */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API bool RenameRetargetOverrideSet(FName InOverrideSetName, FName InNewName);

	/** Duplicate an existing override set, creating a new set with all the same property overrides.
	* @param InOverrideSetName    The name of the override set to duplicate.
	* @return The name of the new duplicated override set (guaranteed unique). */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API FName DuplicateRetargetOverrideSet(FName InOverrideSetName);

	/** Reorder an override set relative to another set.
	* The dragged set becomes a sibling of the target (same parent) and is placed before or after it.
	* @param InSetToMove      The override set to move.
	* @param InTargetSet      The override set to position relative to.
	* @param bInsertAfter     If true, insert after target. If false, insert before. */
	UE_API void ReorderOverrideSet(FName InSetToMove, FName InTargetSet, bool bInsertAfter);

	/** Assign a parent to the given override set.
	* NOTE: Override sets can be applied hierarchically to make for easy composition of property overrides.
	* @param InOverrideSetName    The name of the override set to re-parent.
	* @param InParentName         The new parent name or NAME_None.
	* @return True if the override set was re-parented, false otherwise. */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API bool SetParentOverrideSet(FName InOverrideSetName, FName InParentName);

	/** Get the name of the parent for the given override set.
	* @param InOverrideSetName    The name of the override set.
	* @return The name of the parent override set or NAME_None. */
	UFUNCTION(BlueprintCallable, Category="IK Retarget")
	UE_API FName GetParentOverrideSet(FName InOverrideSetName);

	//
	// RETARGET OVERRIDES C++ ONLY API
	//
	
	/** Return NameToMakeUnique with a numbered suffix that makes it unique among existing override sets. */
	UE_API FName GetUniqueOverrideSetName(const FName NameToMakeUnique) const;

	/** Get read-only access to all override sets in the retargeter. */
	UE_API const TMap<FName, FRetargetOverrideSet>& GetAllOverrideSets() const;

	/** Get access to an override set by name. */
	UE_API FRetargetOverrideSet* FindOverrideSet(const FName InOverrideSetName) const;

	/** Helper to get a single property override */
	FRetargetOpPropertyOverride* FindPropertyOverride(
		const FName InOverrideSetName,
		const FName InOpName,
		const FString& InPropertyPath) const;

	/** Get a list of all override sets that are children of the given set (recursive). */
	UE_API const TArray<FName> GetAllChildrenOverrideSets(const FName InOverrideSetName) const;

	/** Get a list of all override sets that are direct children of the given set (non-recursive). */
	UE_API const TArray<FName> GetDirectChildrenOverrideSets(const FName InOverrideSetName) const;

	/** Find the specific op overrides within a given override set. */
	UE_API FRetargetOpOverrides* FindOpOverrides(const FName InOverrideSetName, const FName InOpName) const;

	/** Add an Op override entry to the given override set. */
	UE_API FRetargetOpOverrides* AddOpOverrides(const FName InOverrideSetName, const FName InOpName) const;

	/** Remove all property overrides for the given op in the given override set. */
	UE_API bool RemoveOpOverrides(const FName InOverrideSetName, const FName InOpName) const;

	/** Update the value of a stored property override inside the given override set, for the given op and property path. */
	UE_API bool UpdateOverrideValue(
		const FName InOverrideSetName,
		const FName InOpName,
		const FString& InPropertyPath,
		const FStructOnScope& InNewValue,
		const bool bShouldTransact=true) const;

	/** Bind a property override to a curve. */
	UE_API bool BindPropertyOverrideToCurve(
		const FName& InCurveName,
		const FName InOverrideSetName,
		const FName InOpName,
		const FString& InPropertyPath) const;

	/** * Bind the given variable to the given property override.
	* @param InVariableName The name of the variable to bind. Use NAME_None to unbind.
	* @param InOverrideSetName The name of the override set containing the op.
	* @param InOpName The name of the specific retarget op containing the property.
	* @param InPropertyPath The leaf property path within the op settings.
	* @return true if the binding was successful, false otherwise
	*/
	UE_API bool BindPropertyOverrideToVariable(
		const FName InVariableName,
		const FName InOverrideSetName,
		const FName InOpName,
		const FString& InPropertyPath) const;

	/** * Clear all bindings on the given property override.
	* @param InOverrideSetName The name of the override set containing the op.
	* @param InOpName The name of the specific retarget op containing the property.
	* @param InPropertyPath The leaf property path within the op settings.
	* @return true if the binding was found and cleared, false otherwise
	*/
	UE_API bool ClearPropertyOverrideBinding(
		const FName InOverrideSetName,
		const FName InOpName,
		const FString& InPropertyPath) const;

	/** Check if this property override is compatible with curve binding (is a numeric type).
	* @param InOpName The name of the specific retarget op containing the property.
	* @param InPropertyPath The leaf property path within the op settings.
	* @return true if the override can be bound to a curve, false otherwise
	*/
	UE_API bool CanPropertyOverrideBindToCurve(
		const FName InOpName,
		const FString& InPropertyPath) const;
	
	/** Add a property override for a specific op within a specific override set. */
	UE_API bool AddPropertyOverrideToOp(const FName InOverrideSetName, const FName InOpName, const FString& InPropertyPath) const;

	/** Add a property override for all properties within the given op. */
	UE_API bool OverrideAllProperties(const FName InOverrideSetName, const FName InOpName) const;

	/** Add a property override for all edited properties within the given op (properties not at their default value). */
	UE_API bool OverrideAllEditedProperties(const FName InOverrideSetName, const FName InOpName) const;

	/** Remove a given property override for a specific op within a specific override set. */
	UE_API bool RemovePropertyOverrideFromOp(const FName InOverrideSetName, const FName InOpName, const FString& InPropertyPath) const;

	/** Remove all property overrides from all ops stored in the given override set. */
	UE_API bool RemoveAllPropertyOverridesFromSet(const FName InOverrideSetName) const;

	/** Set the given override set to be active by default (unless otherwise overridden by the user).
	 * @return True if the override set was found and modified, false otherwise. */
	UE_API bool SetOverrideSetActiveByDefault(const FName InOverrideSetName, const bool bActiveByDefault) const;

	/** Get if the given override set is configured to be active by default.
	 * @return True if the override set was found and is active by default, false otherwise. */
	UE_API bool GetOverrideSetActiveByDefault(const FName InOverrideSetName) const;

	/** Returns true if the given override set has an override for the given property on the given op. */
	UE_API bool HasPropertyOverride(const FName InOverrideSetName, const FName InOpName, const FString& InPropertyPath) const;

	/** Returns the name of the given override set. */
	UE_API FName GetOverrideSetName(const FRetargetOverrideSet* InOverrideSet) const;

	/** Helper to get an FProperty* for a given property override */
	FProperty* GetPropertyForOverride(const FName InOpName, const FString& InPropertyPath) const;

	//
	// VARIABLES C++ ONLY API
	//
	
	/** Get read-only access to the set of variables for this retarget asset. */
	UE_API const FRetargetVariableContainer& GetVariables() const;

	/** * Get the FProperty of a variable by name. 
	 * @param InVariableName The name of the variable to find in the property bag.
	 * @return The FProperty if found, otherwise nullptr.
	 */
	UE_API const FProperty* GetVariableProperty(const FName InVariableName) const;

	/** * Determine if the source property (override) can be bound to the target (variable). 
	 * This checks for type compatibility, including allowed casts like float to double.
	 * @param Source The property override being bound. 
	 * @param Target The property of the variable being bound to.
	 * @return True if the properties are compatible for a data binding.
	 */
	UE_API static bool IsPropertyCompatibleForBinding(const FProperty* Source, const FProperty* Target);

	/** * Get the name of the variable bound to the given property override. 
	 * @param InOverrideSetName The name of the override set.
	 * @param InOpName The name of the operation.
	 * @param InPropertyPath The specific property path to check.
	 * @return The name of the bound variable, or NAME_None if no binding exists.
	 */
	UE_API FName GetVariableBoundToPropertyOverride(
	    const FName InOverrideSetName,
	    const FName InOpName,
	    const FString& InPropertyPath) const;

	/** * Get the name of the curve bound to the given property override. 
	 * @param InOverrideSetName The name of the override set.
	 * @param InOpName The name of the operation.
	 * @param InPropertyPath The specific property path to check.
	 * @return The name of the bound curve, or NAME_None if no binding exists.
	 */
	UE_API FName GetCurveBoundToPropertyOverride(
		const FName InOverrideSetName,
		const FName InOpName,
		const FString& InPropertyPath) const;

	/** Get if the property override is bound to anything (curve or variable).
	 * @param InOverrideSetName The name of the override set.
	 * @param InOpName The name of the operation.
	 * @param InPropertyPath The specific property path to check.
	 * @return true if the override is bound to anything, false otherwise
	 */
	UE_API bool GetPropertyOverrideHasBinding(
		const FName InOverrideSetName,
		const FName InOpName,
		const FString& InPropertyPath) const;

	/** * Adds a new variable to the retargeter's variable container (defaults to float type)
	 * @param InVariableName The desired name for the new variable (if Name_None, a default name is created)
	 * @return The actual name assigned to the variable (may be suffixed if a name collision occurred).
	 */
	UE_API FName AddNewVariable(const FName InVariableName) const;

	/** * Deletes a variable from the retargeter and clears any associated property bindings.
	 * @param InVariableName The name of the variable to remove.
	 * @return True if the variable was found and successfully removed.
	 */
	UE_API bool DeleteVariable(const FName InVariableName) const;

	/** * Renames an existing variable and updates all internal property bindings to reflect the new name.
	 * @param InVariableName The current name of the variable.
	 * @param InNewName The new name to assign to the variable.
	 * @return True if the rename was successful.
	 */
	UE_API bool RenameVariable(const FName InVariableName, const FName InNewName) const;

	/** * Changes the underlying data type of a variable. This may clear existing bindings if types become incompatible.
	 * @param InVariableName The name of the variable to modify.
	 * @param NewPinType The new type definition (float, bool, etc.) via a Graph Pin Type.
	 * @return True if the type was successfully changed.
	 */
	UE_API bool SetVariableType(const FName InVariableName, const FEdGraphPinType& NewPinType) const;

	/** * Check if a variable with the given name exists in this asset.
	 * @param InVariableName The name to check.
	 * @return True if the variable exists in the property bag.
	 */
	UE_API bool GetVariableExists(const FName InVariableName) const;

	/** * Helper to iterate every PropertyOverride in the asset.
	 * @param Visitor A lambda or function to execute for every override found in the asset.
	 */
	void IterateAllOverrides(const TFunctionRef<void(FRetargetOpPropertyOverride&)>& Visitor) const;

private:

	/** Controller functions call this anytime the override sets are modified */
	void PostEditOverrideSets() const;
	
	// Remove bones from retarget poses that are no longer in skeleton 
	UE_API void CleanPoseList(const ERetargetSourceOrTarget SourceOrTarget) const;
	
	//
	// BEGIN DEPRECATED PUBLIC API
	//
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
public:

	// get access to the mapping between source/target chains
	UE_DEPRECATED(5.6, "Chain mappings are stored on individual ops. Use the version that takes an op name. ")
	const FRetargetChainMapping& GetChainMapping() const;
	
	// Get a copy of the retarget root settings for this asset.
	UE_DEPRECATED(5.6, "Root settings are now accessed through the Pelvis Motion Op controller.")
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API FTargetRootSettings GetRootSettings() const;
	
	// Set the retarget root settings for this asset.
	UE_DEPRECATED(5.6, "Root settings are now accessed through the Pelvis Motion Op controller.")
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API void SetRootSettings(const FTargetRootSettings& RootSettings) const;

	// Get a copy of the global settings for this asset.
	UE_DEPRECATED(5.6, "Global settings are now accessed through Op controllers that perform the same duties.")
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API FRetargetGlobalSettings GetGlobalSettings() const;

	// Get a copy of the global settings for this asset.
	UE_DEPRECATED(5.6, "Global settings are now accessed through Op controllers that perform the same duties.")
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API void SetGlobalSettings(const FRetargetGlobalSettings& GlobalSettings) const;

	// Get a copy of the settings for the target chain by name.
	UE_DEPRECATED(5.6, "Access to chain settings must go through an Op controller now.")
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API FTargetChainSettings GetRetargetChainSettings(const FName& TargetChainName) const;

	// Set the settings for the target chain by name. Returns true if the chain settings were applied, false otherwise.
	UE_DEPRECATED(5.6, "Access to chain settings must go through an Op controller now.")
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API bool SetRetargetChainSettings(const FName& TargetChainName, const FTargetChainSettings& Settings) const;

	// Get read-only access to the list of settings for each target chain
	UE_DEPRECATED(5.6, "Access to chain settings must go through an Op controller now.")
	UFUNCTION(BlueprintCallable, Category="IK Retarget", meta = (BlueprintThreadSafe))
	UE_API const TArray<URetargetChainSettings*>& GetAllChainSettings() const;

	// convenience to get chain settings UObject by name
	UE_DEPRECATED(5.6, "Access to chain settings must go through an Op controller now.")
	UE_API URetargetChainSettings* GetChainSettings(const FName& TargetChainName) const;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	//
	// END DEPRECATED PUBLIC API
	//
	
private:

	//
	// DELEGATE CALLBACKS FOR OUTSIDE SYSTEMS
	//
	DECLARE_MULTICAST_DELEGATE(FOnRetargeterNeedsInitialized);
	FOnRetargeterNeedsInitialized RetargeterNeedsInitialized;

	DECLARE_MULTICAST_DELEGATE(FOnOpStackModified);
	FOnOpStackModified OpStackModified;

	DECLARE_MULTICAST_DELEGATE(FOnOverrideSetsModified);
	FOnOverrideSetsModified OverrideSetsModified;

	DECLARE_MULTICAST_DELEGATE(FOnVariablesModified);
	FOnVariablesModified VariablesModified;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnIKRigReplaced, ERetargetSourceOrTarget);
	FOnIKRigReplaced IKRigReplaced;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreviewMeshReplaced, ERetargetSourceOrTarget);
	FOnPreviewMeshReplaced PreviewMeshReplaced;

	// auto pose generator
	TUniquePtr<FRetargetAutoPoseGenerator> AutoPoseGenerator;

	// only allow modifications to data model from one thread at a time
	mutable FCriticalSection ControllerLock;

	// prevent reinitializing from inner operations
	mutable int32 ReinitializeScopeCounter = 0;

private:
	// a dummy UObject to wrap UStructs for the details panel
	// maps each mode to its corresponding UIKRigStructViewer wrapper
	UPROPERTY(Transient)
	TMap<ERetargetStructViewerMode, TObjectPtr<UIKRigStructViewer>> StructViewers;

	// transient cache used for duration of editing
	mutable TMap<const USkeletalMesh*, TSharedPtr<FIKRigSkeleton>> CachedSkeletons;
	
public:

	// Attach a delegate to be notified whenever either the source or target Preview Mesh asset's are swapped out.
	FOnPreviewMeshReplaced& OnPreviewMeshReplaced(){ return PreviewMeshReplaced; };
	
	// Attach a delegate to be notified whenever either the source or target IK Rig asset's are swapped out.
	FOnIKRigReplaced& OnIKRigReplaced(){ return IKRigReplaced; };
	
	// Attach a delegate to be notified whenever the retargeter is modified in such a way that would require re-initialization of the processor.
	FOnRetargeterNeedsInitialized& OnRetargeterNeedsInitialized(){ return RetargeterNeedsInitialized; };

	// Attach a delegate to be notified whenever the op stack is modified.
	FOnOpStackModified& OnOpStackModified(){ return OpStackModified; };

	// Attach a delegate to be notified whenever the override sets are modified.
	FOnOverrideSetsModified& OnOverrideSetsModified(){ return OverrideSetsModified; };

	// Attach a delegate to be notified whenever the variables are modified.
	FOnVariablesModified& OnVariablesModified(){ return VariablesModified; };

	// get the singleton struct viewer for editing structs belonging to the asset this controller controls
	// NOTE: there are 2 details views in the retarget editor, the InMode specifies which viewer you want to reset and reuse
	UE_API UIKRigStructViewer* GetStructViewer(ERetargetStructViewerMode InMode);
	
	friend class UIKRetargeter;
	friend struct FScopedReinitializeIKRetargeter;
};

/** Retargeter can reinit just the processor or other UI as well depending on the data model modification type */
enum class ERetargetRefreshMode : uint8
{
	ProcessorOnly,
	ProcessorAndOpStack,
	ProcessorAndFullUI
};

struct FScopedReinitializeIKRetargeter
{
	FScopedReinitializeIKRetargeter(const UIKRetargeterController *InController, ERetargetRefreshMode InRefreshMode = ERetargetRefreshMode::ProcessorOnly)
	{
		InController->ReinitializeScopeCounter++;
		Controller = InController;
		RefreshMode = InRefreshMode;
	}
	~FScopedReinitializeIKRetargeter()
	{
		if (--Controller->ReinitializeScopeCounter == 0)
		{
			Controller->RetargeterNeedsInitialized.Broadcast();

			if (RefreshMode == ERetargetRefreshMode::ProcessorAndOpStack)
			{
				Controller->OpStackModified.Broadcast();
			}
		}
	};

	const UIKRetargeterController* Controller;
	ERetargetRefreshMode RefreshMode = ERetargetRefreshMode::ProcessorOnly;
};

#undef UE_API
