// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "UObject/WeakObjectPtr.h"

#include "IKRetargeterPoseGenerator.generated.h"

class UIKRetargeterController;
class FIKRetargetEditorController;
struct FIKRetargetProcessor;
enum class ERetargetSourceOrTarget : uint8;

// A Note on Determining the "Direction" of a Bone...
//
// What are we using to determine the "direction" of the source and target bones to align?
// If chain only has one bone OR this bone is located at the end of a chain, then we cannot compute a tangent!
// So we fallback to an analysis of the geometry... this is necessary because we cannot rely solely on the shape of the skeleton.
// Any assumptions we make could be invalidated in different circumstances. For example,
// 1. we cannot assume the local axes are aligned in any meaningful way, and even if they are we cannot assume WHICH axis is the correct one
// 2. we cannot assume that any child bones are located in a position that would create a meaningful vector (ie twist bones in arbitrary locations)
// 3. we cannot assume that extrapolating the direction from the parent is meaningful. For example, feet are usually perpendicular to the knee.
//
// Since the skeleton cannot be relied upon to provide robust directional information for individual and/or leaf bones, we must instead fallback
// on looking at the mesh that is skinned to it and try to discern a direction vector from that. If there is no skinning data, then we simply
// do NOT auto-align that bone.
UENUM()
enum class ERetargetAutoAlignMethod : uint8
{
	// use the chain to determine the source and target directions to align
	ChainToChain,
	// use mesh direction as source, and mesh direction as the target to align
	MeshToMesh,
	// match the local rotation axes of each bone individually (will be horribly wrong if characters don't share axis orientations)
	LocalRotationAxes,
	// match the global rotation axes of each bone individually (will be horribly wrong if characters don't share axis orientations)
	GlobalRotationAxes
};

// a sub-system used by the IKRetargetEditorController for automatically generating a retarget pose
struct FRetargetAutoPoseGenerator
{
	// created and owned by the retarget editor controller
	FRetargetAutoPoseGenerator(const TWeakObjectPtr<UIKRetargeterController> InController);
	
	// align the specified bones on the current skeleton to the OTHER skeleton, using the specified method
	// NOTE: if BonesToAlign is empty, it will align ALL bones in the skeleton
	void AlignBones(
		const TArray<FName>& BonesToAlign,
		const ERetargetAutoAlignMethod Method,
		const ERetargetSourceOrTarget SourceOrTarget,
		const bool bSuppressWarnings) const;

	// adjust the vertical height of the pelvis to place the character back on the ground
	// if BoneToPutOnGround is not specified (NAME_None), then will find the lowest bone and use that
	void SnapToGround(const FName BoneToPutOnGround, const ERetargetSourceOrTarget SourceOrTarget) const;

private:
	
	// generates a local rotation offset to align BoneToAlign to the opposite skeleton, and stores it in the current retarget pose
	void AlignBone(
		const FName BoneToAlign,
		const ERetargetAutoAlignMethod Method,
		const ERetargetSourceOrTarget SourceOrTarget) const;

	// generates a local rotation offset to align the pelvis based on the spine orientation
	void AlignPelvisBone(const ERetargetSourceOrTarget SourceOrTarget) const;

	// get the chain most likely to be the spine
	bool GetSpineChain(const ERetargetSourceOrTarget SourceOrTarget, FName& OutChainName) const;

	// get the chain most likely to be the spine
	FVector GetChainDirection(const FName ChainName, const ERetargetSourceOrTarget SourceOrTarget) const;
	
	// takes a world space rotation offset, converts it to local space and stores it in the current retarget pose
	void ApplyWorldRotationToRetargetPose(
		const FQuat& WorldRotationOffset,
		const FName BoneToAffect,
		const ERetargetSourceOrTarget SourceOrTarget) const;

	// get the tangent vector of a retarget chain at a given param (0-1)
	bool GetChainTangent(
		const FName ChainName,
		const float Param,
		const ERetargetSourceOrTarget SourceOrTarget,
		FVector& OutDirection) const;

	// discern the direction of a bone based on the mesh it is skinned to
	bool GetChainTangentFromMesh(
		const FName ChainName,
		const float Param,
		const ERetargetSourceOrTarget SourceOrTarget,
		FVector& OutDirection) const;

	// get the locations of vertices skinned to a sub-set of bones
	void GetPointsWeightedToBones(
		const TArray<FName>& Bones,
		const ERetargetSourceOrTarget SourceOrTarget,
		TArray<FVector>& OutVertexPositions) const;

	// get all children (recursive) of the given bone
	void GetAllChildrenOfBone(
		const FName Bone,
		const ERetargetSourceOrTarget SourceOrTarget,
		TArray<FName>& OutAllChildren) const;

	// get the vertex ref pose and skinning weights for a given mesh (source or target)
	void GetRefPoseVerticesAndWeights(
		const ERetargetSourceOrTarget SourceOrTarget,
		TArray<TMap<FName, float>>& OutWeights,
		TArray<FVector>& OutPositions) const;

	// calculates the skinned vertex positions for a subset of the mesh using the supplied bone transforms
	void GetDeformedVertexPositions(
		const TArray<int32>& VerticesToDeform,
		const TArray<FVector>& VertexRefPose,
		const TArray<FTransform>& RefPoseBoneTransforms,
		const TArray<FTransform>& CurrentBoneTransforms,
		const TArray<TMap<FName, float>>& SkinWeights,
		const ERetargetSourceOrTarget SourceOrTarget,
		TArray<FVector>& OutDeformedPositions) const;

	// gets the bone at this point in the chain, returns true if non of it's children are retargeted 
	bool IsARetargetLeaf(
		const FName ChainName,
		const float Param,
		ERetargetSourceOrTarget SourceOrTarget) const;

	// get the processor from the editor and return true if it's initialized
	bool CheckReadyToAlignBones() const;

	TWeakObjectPtr<UIKRetargeterController> Controller;
	TSharedPtr<FIKRetargetProcessor> Processor;
	static constexpr float ParamStepSize = 0.1f;
};
