// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"
#include "IKRetargeterThumbnailRenderer.generated.h"

#define UE_API IKRIGEDITOR_API

class FCanvas;
class FRenderTarget;
class ASkeletalMeshActor;
enum class ERetargetSourceOrTarget : uint8;

class FIKRetargeterThumbnailScene : public FThumbnailPreviewScene
{
public:
	
	UE_API FIKRetargeterThumbnailScene();

	// sets the retargeter to use in the next CreateView()
	UE_API void SetSkeletalMeshes(USkeletalMesh* SourceMesh, USkeletalMesh* TargetMesh) const;

protected:
	// FThumbnailPreviewScene implementation
	UE_API virtual void GetViewMatrixParameters(const float InFOVDegrees, FVector& OutOrigin, float& OutOrbitPitch, float& OutOrbitYaw, float& OutOrbitZoom) const override;

private:
	// the skeletal mesh actors used to display all skeletal mesh thumbnails
	ASkeletalMeshActor* SourceActor;
	ASkeletalMeshActor* TargetActor;
};


// this thumbnail renderer displays a given IK Rig in the asset icon
UCLASS(config=Editor, MinimalAPI)
class UIKRetargeterThumbnailRenderer : public UDefaultSizedThumbnailRenderer
{
	GENERATED_BODY()

	// Begin UThumbnailRenderer Object
	IKRIGEDITOR_API virtual bool CanVisualizeAsset(UObject* Object) override;
	IKRIGEDITOR_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	IKRIGEDITOR_API virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;
	// End UThumbnailRenderer Object
	
	// UObject implementation
	virtual void BeginDestroy() override;
	// End UObject implementation

	USkeletalMesh* GetPreviewMeshFromAsset(UObject* Object, ERetargetSourceOrTarget SourceOrTarget) const;
	bool HasSourceOrTargetMesh(UObject* Object) const;

protected:
	TObjectInstanceThumbnailScene<FIKRetargeterThumbnailScene, 128> ThumbnailSceneCache;
};

#undef UE_API
