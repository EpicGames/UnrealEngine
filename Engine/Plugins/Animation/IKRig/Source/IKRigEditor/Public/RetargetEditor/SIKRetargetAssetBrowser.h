// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ContentBrowserDelegates.h"
#include "IKRetargetBatchOperation.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWindow.h"


class FIKRetargetEditorController;

class SIKRetargetAssetBrowser : public SBox
{
public:
	SLATE_BEGIN_ARGS(SIKRetargetAssetBrowser) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FIKRetargetEditorController> InEditorController);

	void RefreshView();
	
private:
	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

	void ExecuteFindInContentBrowserAction();
	bool CanExecuteFindInContentBrowserAction();

	FReply OnExportButtonClicked();
	bool IsExportButtonEnabled() const;

	FReply OnPlayRefPoseClicked();
	bool IsPlayRefPoseEnabled() const;
	
	void OnAssetDoubleClicked(const FAssetData& AssetData);
	bool OnShouldFilterAsset(const struct FAssetData& AssetData);
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets) const;

	/** Used to get the currently selected assets */
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;
		
	/** editor controller */
	TWeakPtr<FIKRetargetEditorController> EditorController;

	/** the animation asset browser */
	TSharedPtr<SBox> AssetBrowserBox;

	/** remember the path the user chose between exports */
	FString PrevBatchOutputPath = "";
	
	/** Commands to execute */
	TSharedPtr<FUICommandList> CommandList;

	friend FIKRetargetEditorController;
};
