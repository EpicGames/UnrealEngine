// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "IKRetargetEditorController.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Framework/Commands/UICommandList.h"
#include "RigEditor/IKRigDetailCustomizations.h"

class FTextFilterExpressionEvaluator;
class FIKRigEditorController;
class SIKRetargetChainMapList;
class FIKRetargetEditor;
struct FBoneChain;

class FRetargetChainMapElement
{
public:

	TSharedRef<ITableRow> MakeListRowWidget(
		const TSharedRef<STableViewBase>& InOwnerTable,
        TSharedRef<FRetargetChainMapElement> InStackElement,
        TSharedPtr<SIKRetargetChainMapList> InChainList);

	static TSharedRef<FRetargetChainMapElement> Make(FName InTargetChainName)
	{
		return MakeShareable(new FRetargetChainMapElement(InTargetChainName));
	}

	FName TargetChainName;

private:
	
	/** Hidden constructor, always use Make above */
	FRetargetChainMapElement(FName InTargetChainName) : TargetChainName(InTargetChainName) {}

	/** Hidden constructor, always use Make above */
	FRetargetChainMapElement() = default;
};

typedef TSharedPtr< FRetargetChainMapElement > FRetargetChainMapElementPtr;
class SIKRetargetChainMapRow : public SMultiColumnTableRow< FRetargetChainMapElementPtr >
{
public:
	
	void Construct(
        const FArguments& InArgs,
        const TSharedRef<STableViewBase>& InOwnerTableView,
        TSharedRef<FRetargetChainMapElement> InChainElement,
        TSharedPtr<SIKRetargetChainMapList> InChainList);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the table row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	void OnSourceChainComboSelectionChanged(TSharedPtr<FString> InName, ESelectInfo::Type SelectInfo);
	
private:

	FReply OnResetToDefaultClicked();

	EVisibility GetResetToDefaultVisibility() const;
	
	FText GetSourceChainName() const;

	FText GetTargetIKGoalName() const;

	TArray<TSharedPtr<FString>> SourceChainOptions;

	TWeakPtr<FRetargetChainMapElement> ChainMapElement;
	
	TWeakPtr<SIKRetargetChainMapList> ChainMapList;

	friend SIKRetargetChainMapList;
};

struct FChainMapFilterOptions
{
	bool bHideUnmappedChains = false;
	bool bHideMappedChains = false;
	bool bHideChainsWithoutIK = false;
	
	bool bNeverShowChainsWithoutIK = false;

	void Reset()
	{
		bool bNeverShowNonIK = bNeverShowChainsWithoutIK;
		*this = FChainMapFilterOptions();
		bNeverShowChainsWithoutIK = bNeverShowNonIK;
	}
};

struct FChainMapListConfig
{
	// the name of the op that owns the chain settings
	FName OpWithChainSettings = NAME_None;
	// the name of the op that owns the chain mapping
	FName OpWithChainMapping = NAME_None;
	// the asset controller that owns the op
	TWeakObjectPtr<UIKRetargeterController> Controller;
	// the default filter options for the list
	FChainMapFilterOptions Filter;
	// whether to show the column for IK goals or not
	bool bEnableGoalColumn = true;
	// whether to allow remapping target chains to source chains
	bool bEnableChainMapping = true;
	// a callback to get the chain settings for a given chain
	TFunction<UObject*(FName)> ChainSettingsGetterFunc;

	bool IsValid() const
	{
		// chain mapping is essential, but chain settings are optional
		return OpWithChainMapping != NAME_None && Controller != nullptr;
	}
};

typedef SListView< TSharedPtr<FRetargetChainMapElement> > SRetargetChainMapListViewType;

class SIKRetargetChainMapList : public SCompoundWidget, FEditorUndoClient, FGCObject
{
	
public:
	
	SLATE_BEGIN_ARGS(SIKRetargetChainMapList) {}
		SLATE_ARGUMENT(FChainMapListConfig, InChainMapListConfig)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void ClearSelection() const;

	void ResetChainSettings(const FName InTargetChainName) const;

	/** call to refresh the list view */
	void RefreshView();

	// FGCObject interface
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("SIKRetargetChainMapList");
	}
	// End of FGCObject interface
	
private:

	/** the options for this chain map list */
	FChainMapListConfig Config;

	/** list view */
	TSharedPtr<SRetargetChainMapListViewType> ListView;
	TArray< TSharedPtr<FRetargetChainMapElement> > ListViewItems;
	/** END list view */

	/** the details view for editing selected chain properties */
	TSharedPtr<IDetailsView> DetailsView;
	
	/** callbacks */
	bool IsListEnabled() const;
	bool IsChainMappingEnabled() const;
	/** when a chain is clicked on in the table view */
	void OnItemClicked(TSharedPtr<FRetargetChainMapElement> InItem);
	TArray<TObjectPtr<UObject>> AllStructWrappers;

	/** auto-map chain button*/
	TSharedRef<SWidget> CreateChainMapMenuWidget();
	void AutoMapChains(const EAutoMapChainType AutoMapType, const bool bSetUnmatchedToNone);
	/** END auto-map chain button*/

	/** filtering the list with search box */
	TSharedRef<SWidget> CreateFilterMenuWidget();
	void OnFilterTextChanged(const FText& SearchText);
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;
	/** menu for adding new solver commands */
	TSharedPtr<FUICommandList> MenuCommandList;

	/** list view generate row callback */
	TSharedRef<ITableRow> MakeListRowWidget(TSharedPtr<FRetargetChainMapElement> InElement, const TSharedRef<STableViewBase>& OwnerTable);

	friend SIKRetargetChainMapRow;
	friend FIKRetargetEditorController;
};
