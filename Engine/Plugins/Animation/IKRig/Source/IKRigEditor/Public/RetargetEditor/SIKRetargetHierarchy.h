// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "IKRetargetEditorController.h"
#include "SBaseHierarchyTreeView.h"
#include "SIKRetargetPoseEditor.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/SCompoundWidget.h"

class FTextFilterExpressionEvaluator;
class SIKRetargetPoseEditor;

class FIKRetargetHierarchyElement : public TSharedFromThis<FIKRetargetHierarchyElement>
{
public:
	
	FIKRetargetHierarchyElement(const FName& InName, const TSharedRef<FIKRetargetEditorController>& InEditorController);

	FText Key;
	TWeakPtr<FIKRetargetHierarchyElement> Parent;
	TArray<TWeakPtr<FIKRetargetHierarchyElement>> Children;
	FName Name;
	bool bIsHidden = false;

	FName GetChainName() const;

private:
	
	TWeakPtr<FIKRetargetEditorController> EditorController;
};

class SIKRetargetHierarchyRow : public SMultiColumnTableRow<TSharedPtr<FIKRetargetHierarchyElement>>
{
	
public:

	SLATE_BEGIN_ARGS(SIKRetargetHierarchyRow) {}
	SLATE_ARGUMENT(TSharedPtr<FIKRetargetEditorController>, EditorController)
	SLATE_ARGUMENT(TSharedPtr<FIKRetargetHierarchyElement>, TreeElement)
	SLATE_END_ARGS()
	
	void Construct(
		const FArguments& InArgs,
		const TSharedRef<STableViewBase>& InOwnerTable)
	{
		OwnerTable = InOwnerTable;
		WeakTreeElement = InArgs._TreeElement;
		EditorController = InArgs._EditorController;
		
		SMultiColumnTableRow<TSharedPtr<FIKRetargetHierarchyElement>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	TSharedPtr<STableViewBase> OwnerTable;
	TWeakPtr<FIKRetargetHierarchyElement> WeakTreeElement;
	TWeakPtr<FIKRetargetEditorController> EditorController;
};

struct FRetargetHierarchyFilterOptions
{
	bool bHideBonesNotInChain;
	bool bHideNotRetargetedBones;
	bool bHideRetargetedBones;
};

typedef SBaseHierarchyTreeView<FIKRetargetHierarchyElement> SIKRetargetHierarchyTreeView;

class SIKRetargetHierarchy : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SIKRetargetHierarchy) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FIKRetargetEditorController> InEditorController);

	void ShowItemAfterSelection(FName ItemKey);

	void RefreshPoseList() const { EditPoseView.Get()->RefreshPoseNames(); };

private:
	
	/** centralized editor controls (facilitate cross-communication between multiple UI elements)*/
	TWeakPtr<FIKRetargetEditorController> EditorController;
	
	/** command list we bind to */
	TSharedPtr<FUICommandList> CommandList;

	/** edit pose widget */
	TSharedPtr<SIKRetargetPoseEditor> EditPoseView;
	
	/** tree view widget */
	TSharedPtr<SIKRetargetHierarchyTreeView> TreeView;
	TArray<TSharedPtr<FIKRetargetHierarchyElement>> RootElements;
	TArray<TSharedPtr<FIKRetargetHierarchyElement>> AllElements;

	/** filtering the tree with search box */
	TSharedRef<SWidget> CreateFilterMenuWidget();
	void OnFilterTextChanged(const FText& SearchText);
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;
	FRetargetHierarchyFilterOptions FilterOptions;
	
	/** tree view callbacks */
	void RefreshTreeView(bool IsInitialSetup=false);
	void HandleGetChildrenForTree(TSharedPtr<FIKRetargetHierarchyElement> InElement, TArray<TSharedPtr<FIKRetargetHierarchyElement>>& OutChildren);
	void OnSelectionChanged(TSharedPtr<FIKRetargetHierarchyElement> Selection, ESelectInfo::Type SelectInfo);
	void OnItemDoubleClicked(TSharedPtr<FIKRetargetHierarchyElement> InElement);
	void OnSetExpansionRecursive(TSharedPtr<FIKRetargetHierarchyElement> InElement, bool bShouldBeExpanded);
	void SetExpansionRecursive(TSharedPtr<FIKRetargetHierarchyElement> InElement, bool bTowardsParent, bool bShouldBeExpanded);
	/** END tree view callbacks */

	friend SIKRetargetHierarchyRow;
	friend FIKRetargetEditorController;
};
