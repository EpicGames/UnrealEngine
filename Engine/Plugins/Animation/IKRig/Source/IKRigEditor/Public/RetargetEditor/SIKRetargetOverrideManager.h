// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "SBaseHierarchyTreeView.h"
#include "SIKRetargetPoseEditor.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/SCompoundWidget.h"

class FTextFilterExpressionEvaluator;
class SIKRetargetOverrideManager;
class SInlineEditableTextBlock;

/** Tree element representing a single override set in the hierarchy */
class FIKRetargetOverrideSetElement : public TSharedFromThis<FIKRetargetOverrideSetElement>
{
public:
    
    FIKRetargetOverrideSetElement(
       const FName& InName,
       const TSharedRef<FIKRetargetEditorController>& InEditorController);

    FText Key;
    TWeakPtr<FIKRetargetOverrideSetElement> Parent;
    TArray<TWeakPtr<FIKRetargetOverrideSetElement>> Children;
    TWeakPtr<SIKRetargetOverrideManager> OverrideManager;
    FName Name;
    bool bIsHidden = false;

    FText GetNumOpsLabel() const;
    FText GetNumOverridesLabel() const;

private:
    
    TWeakPtr<FIKRetargetEditorController> EditorController;
};

/** Row widget for the override set tree view */
class SIKRetargetOverrideSetRow : public SMultiColumnTableRow<TSharedPtr<FIKRetargetOverrideSetElement>>
{
    
public:

    SLATE_BEGIN_ARGS(SIKRetargetOverrideSetRow) {}
    SLATE_ARGUMENT(TSharedPtr<FIKRetargetEditorController>, EditorController)
    SLATE_ARGUMENT(TSharedPtr<FIKRetargetOverrideSetElement>, TreeElement)
    SLATE_ARGUMENT(TSharedPtr<SIKRetargetOverrideManager>, OverrideManager)
    SLATE_END_ARGS()
    
    void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable);

    /** Overridden from SMultiColumnTableRow. Generates a widget for this column of the tree view. */
    virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

    /** Requests that the inline text block enters edit mode */
    void OnRequestRename();
    
private:
    
    FText GetItemText() const;
    bool OnVerifyTextChanged(const FText& InText, FText& OutErrorMessage);
    void OnTextCommitted(const FText& InText, ETextCommit::Type CommitInfo);

    ECheckBoxState IsOverrideSetActive() const;
    void OnOverrideSetActiveChanged(ECheckBoxState NewState) const;

    TSharedPtr<STableViewBase> OwnerTable;
    TWeakPtr<FIKRetargetOverrideSetElement> WeakTreeElement;
    TWeakPtr<FIKRetargetEditorController> EditorController;
    TWeakPtr<SIKRetargetOverrideManager> OverrideManager;
    TSharedPtr<SInlineEditableTextBlock> InlineTextBlock;
};

/** Drag and drop operation for re-parenting override sets in the tree */
class FRetargetOverrideSetDragDropOp : public FDecoratedDragDropOp
{
public:
    
    DRAG_DROP_OPERATOR_TYPE(FRetargetOverrideSetDragDropOp, FDecoratedDragDropOp)

    static TSharedRef<FRetargetOverrideSetDragDropOp> New(TWeakPtr<FIKRetargetOverrideSetElement> InElement);
    
    virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
    
    TWeakPtr<FIKRetargetOverrideSetElement> Element;
};

struct FRetargetOverrideSetFilterOptions
{
	FRetargetOverrideSetFilterOptions() = default;
    bool bHideEmptySets = false;
};

typedef SBaseHierarchyTreeView<FIKRetargetOverrideSetElement> SIKRetargetOverrideSetTreeView;

/** Widget for managing and hierarchy of retarget override sets */
class SIKRetargetOverrideManager : public SCompoundWidget, public FEditorUndoClient
{
public:
    SLATE_BEGIN_ARGS(SIKRetargetOverrideManager) {}
    SLATE_END_ARGS()

    void Construct(const FArguments& InArgs, TSharedRef<FIKRetargetEditorController> InEditorController);

    /** Gets the name of the override set currently selected in the tree */
    FName GetSelectedOverrideSetName() const;

private:
    
    /** centralized editor controls (facilitate cross-communication between multiple UI elements)*/
    TWeakPtr<FIKRetargetEditorController> EditorController;

    /** the details view for editing selected override set properties */
    TSharedPtr<IDetailsView> DetailsView;
    
    /** tree view widget */
    TSharedPtr<SIKRetargetOverrideSetTreeView> TreeView;
    TArray<TSharedPtr<FIKRetargetOverrideSetElement>> RootElements;
    TArray<TSharedPtr<FIKRetargetOverrideSetElement>> AllElements;

    /** filtering the tree with search box */
    void OnFilterTextChanged(const FText& SearchText);
    TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;
    FRetargetOverrideSetFilterOptions FilterOptions;
    
    /** tree view callbacks */
    void RefreshTreeView(bool IsInitialSetup=false);
    void HandleGetChildrenForTree(TSharedPtr<FIKRetargetOverrideSetElement> InElement, TArray<TSharedPtr<FIKRetargetOverrideSetElement>>& OutChildren);
    void OnSelectionChanged(TSharedPtr<FIKRetargetOverrideSetElement> Selection, ESelectInfo::Type SelectInfo);
    void OnItemDoubleClicked(TSharedPtr<FIKRetargetOverrideSetElement> InElement);
    void OnSetExpansionRecursive(TSharedPtr<FIKRetargetOverrideSetElement> InElement, bool bShouldBeExpanded);
    void SetExpansionRecursive(TSharedPtr<FIKRetargetOverrideSetElement> InElement, bool bTowardsParent, bool bShouldBeExpanded);
    TSharedPtr<SWidget> OnGenerateContextMenu();
    /** END tree view callbacks */

    /** drag and drop operations */
    FReply OnDragDetected(
       const FGeometry& MyGeometry,
       const FPointerEvent& MouseEvent);
    TOptional<EItemDropZone> OnCanAcceptDrop(
       const FDragDropEvent& DragDropEvent,
       EItemDropZone DropZone,
       TSharedPtr<FIKRetargetOverrideSetElement> TargetElement);
    FReply OnAcceptDrop(
       const FDragDropEvent& DragDropEvent,
       EItemDropZone DropZone,
       TSharedPtr<FIKRetargetOverrideSetElement> TargetElement);
    /** END drag and drop operations */

    /** keyboard shortcuts */
    void BindCommands();
    void DeleteSelectedOverrideSet();
    void RenameSelectedOverrideSet();
    bool HasSelectedOverrideSet() const;
    TSharedPtr<FUICommandList> CommandList;

    //~ Begin SWidget
    virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
    //~ End SWidget

    friend SIKRetargetOverrideSetRow;
    friend FIKRetargetEditorController;
};