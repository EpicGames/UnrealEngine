// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Input/SComboBox.h"

class UIKRetargeterController;
class FIKRetargetEditorController;

class SIKRetargetPoseEditor : public SCompoundWidget
{
	
public:
	
	SLATE_BEGIN_ARGS(SIKRetargetPoseEditor) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FIKRetargetEditorController> InEditorController);

	void RefreshPoseNames();
	
private:

	TSharedPtr<SVerticalBox> Contents;
	
	TSharedRef<SWidget> MakeToolbar(TSharedPtr<FUICommandList> Commands);
	TSharedRef<SWidget> GenerateResetMenuContent(TSharedPtr<FUICommandList> Commands);
	TSharedRef<SWidget> GenerateEditMenuContent(TSharedPtr<FUICommandList> Commands);
	TSharedRef<SWidget> GenerateNewMenuContent(TSharedPtr<FUICommandList> Commands);
	TObjectPtr<UIKRetargeterController> GetAssetControllerFromSelectedObjects(IDetailLayoutBuilder& DetailBuilder) const;
	
	TWeakPtr<FIKRetargetEditorController> EditorController;
	TArray<TSharedPtr<FName>> PoseNames;
};
