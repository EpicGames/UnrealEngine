// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Retargeter/IKRetargeter.h"
#include "Widgets/SCompoundWidget.h"
#include "StructUtils/PropertyBag.h"
#include "Templates/SharedPointer.h"
#include "Misc/NotifyHook.h"
#include "Widgets/Views/SListView.h"

class FIKRetargetEditorController;
class IStructureDetailsView;

struct FVariableRow
{
	FName PropertyName;
};

class SIKRetargetVariablesEditor : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SIKRetargetVariablesEditor) {}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, TSharedRef<FIKRetargetEditorController> InEditorController);

	// FNotifyHook Interface
	virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	// END FNotifyHook Interface

	void RefreshList();
	
private:
	
	UIKRetargeter* GetAsset() const;
	TSharedRef<ITableRow> OnGenerateVariableRow(TSharedPtr<FVariableRow> InItem, const TSharedRef<STableViewBase>& OwnerTable);
	TSharedRef<SWidget> MakeTypeSelectorForVariable(TSharedPtr<FVariableRow> InRow) const;
	TSharedRef<SWidget> MakeValueWidgetForVariable(TSharedPtr<FVariableRow> InRow);
	
	TArray<TSharedPtr<FVariableRow>> VariableRows;
	TSharedPtr<SListView<TSharedPtr<FVariableRow>>> VariablesListView;
	TWeakPtr<FIKRetargetEditorController> EditorController;
};