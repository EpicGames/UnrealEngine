// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "IKRetargetBatchOperation.h"
#include "IKRigLogger.h"
#include "SEditorViewport.h"
#include "PreviewScene.h"
#include "RigEditor/SIKRigOutputLog.h"
#include "RigEditor/IKRigAutoCharacterizer.h"
#include "RigEditor/IKRigAutoFBIK.h"

#include "SRetargetAnimAssetsWindow.generated.h"

class FAssetThumbnailPool;
class UIKRetargeter;
class SRetargetAnimAssetsWindow;
class UIKRetargetAnimInstance;
class UDebugSkelMeshComponent;

// dialog to select path to export to
class SBatchExportPathDialog: public SWindow
{
	
public:
	
	SLATE_BEGIN_ARGS(SBatchExportPathDialog){}
	SLATE_ARGUMENT(FIKRetargetBatchOperationContext*, BatchContext)
	SLATE_ARGUMENT(bool, ExportRetargetAssets)
	SLATE_END_ARGS()

	SBatchExportPathDialog() : UserResponse(EAppReturnType::Cancel){}

	void Construct(const FArguments& InArgs);

	// displays the dialog in a blocking fashion
	EAppReturnType::Type ShowModal();

private:
	
	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	
	// example rename text
	void UpdateExampleText();
	
	// modify folder output path
	FText GetFolderPath() const;

	// remove characters not allowed in asset names
	static FString ConvertToCleanString(const FText& ToClean);
	
	FText ExampleText; // The rename rule sample text
	EAppReturnType::Type UserResponse;

	// the context/data-model we are editing with this pop-up window
	FIKRetargetBatchOperationContext* BatchContext;

	bool bExportingRetargetAssets;
};

// settings object used in details view of the batch retarget window
UCLASS(Blueprintable, config=EditorPerProjectUserSettings, Category = "BatchExportOptions")
class UBatchExportOptions : public UObject
{
	GENERATED_BODY()

public:

	// singleton class
	static UBatchExportOptions* GetInstance();

	// Any files with the same name will be overwritten instead of creating a new file with a numeric suffix.
	// This is useful when iterating on a batch process.
	UPROPERTY(EditAnywhere, Config, Config, Category = "File")
	bool bOverwriteExistingFiles = false;

	// Duplicates and retargets any animation assets referenced by the input assets. For example, sequences in an animation blueprint or blendspace.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = "File")
	bool bIncludeReferencedAssets = true;

	// If retargeting additive animations, they will have their additive settings reset so that the retargeter evaluates the motion in a non-additive way.
	// Setting this flag to true will ensure that the resulting animation sequences will retain their additive settings after the retarget operation.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = "File")
	bool bRetainAdditiveFlags = true;

	// TODO - Kiaran Feb 2024 - Naively leaving out non-retargeted keys results in flipped skeletons, needs work.
	// Will not produce keys on bones that are not animated, reducing size on disk of the resulting files.
	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Config, Category = "Animation")
	//bool bExportOnlyAnimatedBones = true;

private:
	static UBatchExportOptions* SingletonInstance;
};

// dialog to select batch export options
class SBatchExportOptionsDialog: public SWindow
{
	
public:
	
	SLATE_BEGIN_ARGS(SBatchExportOptionsDialog){}
	SLATE_ARGUMENT(FIKRetargetBatchOperationContext*, BatchContext)
	SLATE_END_ARGS()

	SBatchExportOptionsDialog();
	void Construct(const FArguments& InArgs);
	EAppReturnType::Type ShowModal();

protected:
	
	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	
	FIKRetargetBatchOperationContext* BatchContext;
	EAppReturnType::Type UserResponse;
};

// viewport in retarget dialog that previews results of retargeting before exporting animations
class SRetargetPoseViewport: public SEditorViewport
{
	
public:
	
	SLATE_BEGIN_ARGS(SRetargetPoseViewport)
	{}
	SLATE_ARGUMENT(FIKRetargetBatchOperationContext*, BatchContext)
	SLATE_END_ARGS()

	SRetargetPoseViewport() : PreviewScene(FPreviewScene::ConstructionValues()){};

	void Construct(const FArguments& InArgs);
	
	void SetSkeletalMesh(USkeletalMesh* InSkeletalMesh, ERetargetSourceOrTarget SourceOrTarget);
	void SetRetargetAsset(UIKRetargeter* RetargetAsset);
	void SetOverrideSets(const TArray<FName>& InOverrideSets);
	void PlayAnimation(UAnimationAsset* AnimationAsset);
	bool IsRetargeterValid();

protected:
	
	// SEditorViewport interface
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	// END SEditorViewport interface

private:

	UAnimationAsset* AnimThatWasPlaying;
	float TimeWhenPaused;
	
	FPreviewScene PreviewScene;
	TObjectPtr<UDebugSkelMeshComponent> SourceComponent;
	TObjectPtr<UDebugSkelMeshComponent> TargetComponent;
	TObjectPtr<UIKRetargetAnimInstance> SourceAnimInstance;
	TObjectPtr<UIKRetargetAnimInstance> TargetAnimInstance;

	// the data model
	FIKRetargetBatchOperationContext* BatchContext;
};

// a container for procedurally generated retarget assets (used in memory and discarded after batch retarget operation)
// can optionally be saved to disk as well
struct FProceduralRetargetAssets : FGCObject
{
	FProceduralRetargetAssets();
	
	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FProceduralRetargetAssets"); }
	// END FGCObject
	
	void AutoGenerateIKRigAsset(USkeletalMesh* Mesh, ERetargetSourceOrTarget SourceOrTarget);
	void AutoGenerateIKRetargetAsset();

	// resulting assets used to retarget animation, or export to disk
	TObjectPtr<UIKRigDefinition> SourceIKRig;
	TObjectPtr<UIKRigDefinition> TargetIKRig;
	TObjectPtr<UIKRetargeter> Retargeter;

	// results of characterizing for error reporting
	FAutoCharacterizeResults SourceCharacterizationResults;
	FAutoCharacterizeResults TargetCharacterizationResults;
	FAutoFBIKResults SourceIKResults;
	FAutoFBIKResults TargetIKResults;

private:
	void CreateNewRetargetAsset();
};

// client for SRetargetPoseViewport
class FRetargetPoseViewportClient: public FEditorViewportClient
{
	
public:
	
	FRetargetPoseViewportClient(FPreviewScene& InPreviewScene, const TSharedRef<SRetargetPoseViewport>& InRetargetPoseViewport);

	// FEditorViewportClient interface
	virtual void Tick(float DeltaTime) override;
	virtual FSceneInterface* GetScene() const override { return PreviewScene->GetScene(); }
	virtual FLinearColor GetBackgroundColor() const override { return FLinearColor(0.36f, 0.36f, 0.36f, 1); }
	virtual void SetViewMode(EViewModeIndex Index) override final { FEditorViewportClient::SetViewMode(Index); }
	// END FEditorViewportClient interface
};

// settings object used in details view of the batch retarget window
UCLASS(Blueprintable, Category = "BatchRetargetSettings")
class UBatchRetargetSettings : public UObject
{
	GENERATED_BODY()

public:

	// singleton class
	static UBatchRetargetSettings* GetInstance();

	// The skeletal mesh with the proportions you want to copy animation FROM.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Source")
	TObjectPtr<USkeletalMesh> SourceSkeletalMesh;

	// The skeletal mesh with the proportions you want to copy animation TO.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Target")
	TObjectPtr<USkeletalMesh> TargetSkeletalMesh;

	// When true, the system will attempt to generate an IK Retargeter compatible with the supplied source and target skeletal meshes.
	// If the skeletons are successfully characterized, it will align the retarget poses automatically.
	// Automatic retargeting is currently limited to common, predefined skeleton types that Unreal knows about (see documentation for full list).
	// If you attempt to use a skeletal mesh that is not compatible with a predefined template, warnings will be displayed in the output log and the
	// export button will be disabled. In that case, you must supply a custom retargeter asset.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Retargeter")
	bool bAutoGenerateRetargeter = true;
	
	// You may also supply a custom IK Retargeter if needed.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Retargeter", meta = (EditCondition = "!bAutoGenerateRetargeter"))
	TObjectPtr<UIKRetargeter> RetargetAsset;

	// Override set from the retargeter to apply during preview and export.
	// Only available when using a manually supplied retargeter asset.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Retargeter", meta = (EditCondition = "!bAutoGenerateRetargeter", GetOptions = "GetOverrideSetOptions"))
	FName OverrideSetToApply;

	UFUNCTION()
	TArray<FString> GetOverrideSetOptions() const;

private:
	static UBatchRetargetSettings* SingletonInstance;
};

// asset browser for user to select animation assets to duplicate/retarget
class SRetargetExporterAssetBrowser : public SBox
{
public:
	SLATE_BEGIN_ARGS(SRetargetExporterAssetBrowser) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SRetargetAnimAssetsWindow> RetargetWindow);
	void RefreshView();
	void GetSelectedAssets(TArray<FAssetData>& OutSelectedAssets) const;
	bool AreAnyAssetsSelected() const;
	
private:
	
	void OnAssetDoubleClicked(const FAssetData& AssetData);
	bool OnShouldFilterAsset(const struct FAssetData& AssetData);
	
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;
	TSharedPtr<SRetargetAnimAssetsWindow> RetargetWindow;
	TSharedPtr<SBox> AssetBrowserBox;
};

enum class EBatchRetargetUIState
{
	MISSING_MESH,
	AUTO_RETARGET_INVALID,
	MANUAL_RETARGET_INVALID,
	NO_ANIMATIONS_SELECTED,
	READY_TO_EXPORT
};

// window to display when configuring batch duplicate & retarget process
class SRetargetAnimAssetsWindow : public SCompoundWidget, FGCObject
{
	
public:

	SRetargetAnimAssetsWindow();

	SLATE_BEGIN_ARGS(SRetargetAnimAssetsWindow)
		: _CurrentSkeletalMesh(nullptr)
		, _WidgetWindow(nullptr)
		, _ShowRemapOption(false)
	{}

	SLATE_ARGUMENT(USkeletalMesh*, CurrentSkeletalMesh)
	SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_ARGUMENT(bool, ShowRemapOption)
	SLATE_END_ARGS()	
	
	void Construct( const FArguments& InArgs );

	// FGCObject
	virtual FString GetReferencerName() const override	{ return TEXT("SRetargetAnimAssetsWindow"); }
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	// END FGCObject

	TObjectPtr<UBatchRetargetSettings> GetSettings() { return Settings; };
	TSharedPtr<SRetargetPoseViewport> GetViewport() { return Viewport; };

private:

	void SetAssets(USkeletalMesh* SourceMesh, USkeletalMesh* TargetMesh, UIKRetargeter* Retargeter);
	void ShowAssetWarnings();

	void OnFinishedChangingSelectionProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	// export button states
	bool CanExportAnimations() const;
	FReply OnExportAnimations();
	bool CanExportRetargetAssets() const;
	FReply OnExportRetargetAssets();
	
	// output current status to the message area
	EBatchRetargetUIState GetCurrentState() const;
	EVisibility GetWarningVisibility() const;
	FText GetWarningText() const;

	// necessary data collected from UI to run retarget.
	FIKRetargetBatchOperationContext BatchContext;

	void ApplyOverrideSetToPreview();

	// ui elements
	TSharedPtr<SRetargetPoseViewport> Viewport;
	TSharedPtr<SRetargetExporterAssetBrowser> AssetBrowser;
	TSharedPtr<SIKRigOutputLog> LogView;
	FIKRigLogger Log;
	TObjectPtr<UBatchRetargetSettings> Settings;
	FProceduralRetargetAssets ProceduralAssets;

public:

	static void ShowWindow(TArray<UObject*> InAnimAssets);
	static TSharedPtr<SWindow> Window;
	
	static const FName LogName;
	static const FText LogLabel;
};