// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Framework/Commands/UICommandList.h"

class SRetargetOpStack;
struct FIKRetargetOpBase;
class FIKRetargetEditorController;
class SRetargetOpList;
struct FRetargetPropertyOverrideNode;

// SListView item-type for a single retarget op
class FRetargetOpStackElement
{
public:

	static TSharedRef<ITableRow> MakeListRowWidget(
		const TSharedRef<STableViewBase>& InOwnerTable,
        TSharedRef<FRetargetOpStackElement> InStackElement,
        TSharedPtr<SRetargetOpList> InOpListWidget);

	static TSharedRef<FRetargetOpStackElement> Make(
		const int32 InOpIndex,
		const bool InCanHaveChildren,
		const TSharedPtr<SRetargetOpList>& InOpListWidget)
	{
		FRetargetOpStackElement* NewElement = new FRetargetOpStackElement(InOpIndex, InCanHaveChildren, InOpListWidget);
		return MakeShareable(NewElement);
	}

	FName GetName() const;
	
	const UScriptStruct* GetType() const;
	
	int32 GetIndexInStack() const { return IndexInStack; };
	
	bool GetCanHaveChildren() const { return bCanHaveChildren; };
	
	const TArray<FRetargetOpStackElement*>& GetChildren() const { return ChildElements; };

	void AddChild(FRetargetOpStackElement* InChild) { ChildElements.Add(InChild); };
	
	const FRetargetOpStackElement* GetParent() const { return ParentElement; };
	
	void SetParent(FRetargetOpStackElement* InParentElement) { ParentElement = InParentElement; };

	TWeakPtr<SRetargetOpList> GetOpList() const { return OpListWidget; };

	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;
	
private:

	int32 IndexInStack = INDEX_NONE;
	bool bCanHaveChildren = false;
	TArray<FRetargetOpStackElement*> ChildElements;
	FRetargetOpStackElement* ParentElement = nullptr;
	TWeakPtr<SRetargetOpList> OpListWidget;
	
	// hidden constructors, always use Make above
	FRetargetOpStackElement() = default;
	FRetargetOpStackElement(
		const int32 InIndexInStack,
		const bool InHasChildren,
		const TSharedPtr<SRetargetOpList>& InOpListWidget) :
		IndexInStack(InIndexInStack),
		bCanHaveChildren(InHasChildren),
		OpListWidget(InOpListWidget){}
};

// meta data about an op type
// used to create filtered menus for creating ops with compatible child types in parent-op menus
struct FIKRetargetOpMetaData
{
	FName NiceName;
	const UScriptStruct* OpType = nullptr;
	const UScriptStruct* ParentType = nullptr;
	bool bIsSingleton = false;
};

// an SListView customized for retarget ops
// Supports:
// - slow double-click rename of list items
// - drag/drop reordering w/ nested ops
// - building a menu of compatible ops to add to the list
// - deleting ops
class SRetargetOpList : public SListView<TSharedPtr<FRetargetOpStackElement>>
{
public:
	
	SLATE_BEGIN_ARGS(SRetargetOpList) {}
		SLATE_ARGUMENT(TWeakPtr<FIKRetargetEditorController>, InEditorController)
		SLATE_ARGUMENT(TWeakPtr<FRetargetOpStackElement>, InParentElement)
	SLATE_END_ARGS()

	virtual ~SRetargetOpList() = default;

	void Construct(const FArguments& InArgs);
	
	TSharedRef<ITableRow> MakeListRowWidget(TSharedPtr<FRetargetOpStackElement> InElement, const TSharedRef<STableViewBase>& OwnerTable);
	
	bool IsEnabled() const;
	
	// drag and drop operations
	FReply OnDragDetected(
		const FGeometry& MyGeometry,
		const FPointerEvent& MouseEvent);
	TOptional<EItemDropZone> OnCanAcceptDrop(
		const FDragDropEvent& DragDropEvent,
		EItemDropZone DropZone,
		TSharedPtr<FRetargetOpStackElement> TargetElement);
	FReply OnAcceptDrop(
		const FDragDropEvent& DragDropEvent,
		EItemDropZone DropZone,
		TSharedPtr<FRetargetOpStackElement> TargetElement);
	// END drag and drop operations
	
	// slow-click renaming
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	void OnSelectionChanged(TSharedPtr<FRetargetOpStackElement> InItem, ESelectInfo::Type SelectInfo) const;
	void OnItemClicked(TSharedPtr<FRetargetOpStackElement> InItem);
	void RequestRenameSelectedOp() const;
	// END slow-click renaming

	// add new op menu
	TSharedRef<SWidget> CreateAddNewOpMenu();
	// menu command callback for adding a new op
	void AddNewRetargetOp(const UScriptStruct* ScriptStruct);
	// END op stack menu
	
	// delete op from stack
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	void DeleteRetargetOp(TSharedPtr<FRetargetOpStackElement> OpToDelete);

	// must be called after refresh
	void RefreshAndRestore();

	// the elements contained in this list
	TArray< TSharedPtr<FRetargetOpStackElement> > Elements;

	// editor controller
	TWeakPtr<FIKRetargetEditorController> EditorController;
	
	// the type of op that is the parent of this list of ops
	// if this is null, then it's assumed to be the top-level op stack
	TWeakPtr<FRetargetOpStackElement> ParentElement;

private:
	
	// slow double-click rename state
	uint32 LastClickCycles = 0;
	TWeakPtr<FRetargetOpStackElement> LastSelectedElement;
	
	// a map of op types to the parent op type (cached at startup)
	TArray<FIKRetargetOpMetaData> AllOpsMetaData;
	void CacheOpTypeMetaData();
};

// builds an UI element representing a single op
class SRetargetOpItem : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRetargetOpItem) {}
	SLATE_ARGUMENT(TWeakPtr<SRetargetOpList>, InOpListWidget)
	SLATE_ARGUMENT(TWeakPtr<FRetargetOpStackElement>, InStackElement)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// RENAMING op
	void OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const;
	FText GetName() const;
	// ENd renaming op

	void GenerateRightClickMenu(FMenuBuilder& InMenuBuilder);

	static float OpHorizontalPadding;
	static float OpVerticalPadding;
	
private:

	FIKRetargetEditorController* GetEditorController() const;

	void MakeOverrideSubMenu(FMenuBuilder& SubMenuBuilder, TSharedPtr<FRetargetPropertyOverrideNode> Node);
	
	bool IsOpEnabled() const;
	
	FIKRetargetOpBase* GetRetargetOp() const;

	TWeakPtr<SRetargetOpList> ListView;
	TWeakPtr<FRetargetOpStackElement> Element;
	TSharedPtr<SInlineEditableTextBlock> EditNameWidget;
};

// an SRetargetOpList item representing a single op with children op beneath it
// (contains an SRetgargetOpList to house children ops)
class SParentRetargetOpItem : public STableRow<TSharedPtr<FRetargetOpStackElement>>
{
public:

	SLATE_BEGIN_ARGS(SParentRetargetOpItem) {}
	SLATE_ARGUMENT(TWeakPtr<FRetargetOpStackElement>, InStackElement)
	SLATE_ARGUMENT(TWeakPtr<SRetargetOpList>, InOpListWidget)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);
	
	void RefreshListView() const;

private:
	
	TWeakPtr<FRetargetOpStackElement> ParentStackElement;
	TWeakPtr<SRetargetOpList> OpListWidget;
	TSharedPtr<SRetargetOpList> ChildrenListView;
};

// an SRetargetOpList item representing a single op with no children
class SRetargetOpSingleItem : public STableRow<TSharedPtr<FRetargetOpStackElement>>
{
public:
	
	SLATE_BEGIN_ARGS(SRetargetOpSingleItem) {}
	SLATE_ARGUMENT(TWeakPtr<FRetargetOpStackElement>, InStackElement)
	SLATE_ARGUMENT(TWeakPtr<SRetargetOpList>, InOpListWidget)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTable);

	bool GetWarningMessage(FText& Message) const;

	bool IsOpEnabled() const;

private:
	
	FIKRetargetOpBase* GetRetargetOp() const;
	
	TWeakPtr<FRetargetOpStackElement> StackElement;
	TWeakPtr<SRetargetOpList> OpListWidget;
};

class FRetargetOpStackDragDropOp : public FDecoratedDragDropOp
{
public:
	
	DRAG_DROP_OPERATOR_TYPE(FRetargetOpStackDragDropOp, FDecoratedDragDropOp)

	static TSharedRef<FRetargetOpStackDragDropOp> New(TWeakPtr<FRetargetOpStackElement> InElement);
	
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	
	static int32 GetIndexToMoveTo(
		const TSharedPtr<FRetargetOpStackElement>& InDraggedElement,
		const TSharedPtr<FRetargetOpStackElement>& InTargetElement,
		const EItemDropZone InDropZone);
	
	TWeakPtr<FRetargetOpStackElement> Element;
};

// top level view of a stack of ik retargeter ops
// shows the main menu for adding ops to the retargeting stack, and a list view of top-level ops
class SRetargetOpStack : public SCompoundWidget, public FEditorUndoClient
{
public:
	
	SLATE_BEGIN_ARGS(SRetargetOpStack) {}
	SLATE_ARGUMENT(TWeakPtr<FIKRetargetEditorController>, InEditorController)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FIKRetargetEditorController>& InEditorController);
	
	void RefreshStackView() const;

private:
	
	TWeakPtr<FIKRetargetEditorController> EditorController;
	TSharedPtr<SRetargetOpList> ListView;
};
