// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetTypeActions_Base.h"

class FAssetTypeActions_IKRigDefinition : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_IKRigDefinition", "IK Rig"); }
	virtual FColor GetTypeColor() const override { return FColor(255, 215, 0); }
	virtual UClass* GetSupportedClass() const override;
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual const TArray<FText>& GetSubMenus() const override;
	virtual UThumbnailInfo* GetThumbnailInfo(UObject* Asset) const override;
	// END IAssetTypeActions Implementation

	static void ExtendSkeletalMeshMenuToMakeIKRig();
	static void CreateNewIKRigFromSkeletalMesh(UObject* InSelectedObject);
};
