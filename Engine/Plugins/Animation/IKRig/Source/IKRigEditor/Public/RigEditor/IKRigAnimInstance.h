// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimPreviewInstance.h"
#include "Rig/IKRigProcessor.h"
#include "AnimNodes/AnimNode_IKRig.h"

#include "IKRigAnimInstance.generated.h"

class UIKRigDefinition;

UCLASS(transient, NotBlueprintType, NotBlueprintable)
class UIKRigAnimInstance : public UAnimPreviewInstance
{
	GENERATED_UCLASS_BODY()

public:

	void SetIKRigAsset(UIKRigDefinition* IKRigAsset);

	void SetProcessorNeedsInitialized();

	FIKRigProcessor* GetCurrentlyRunningProcessor();

protected:
	virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;

	UPROPERTY(Transient)
	FAnimNode_IKRig IKRigNode;
};
