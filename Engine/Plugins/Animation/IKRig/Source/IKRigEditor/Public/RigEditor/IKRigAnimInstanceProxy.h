// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimPreviewInstance.h"
#include "Animation/AnimNode_LinkedInputPose.h"
#include "AnimNodes/AnimNode_IKRig.h"

#include "IKRigAnimInstanceProxy.generated.h"

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
struct FIKRigAnimInstanceProxy : public FAnimPreviewInstanceProxy
{
	GENERATED_BODY()

public:
	
	FIKRigAnimInstanceProxy() = default;
	FIKRigAnimInstanceProxy(UAnimInstance* InAnimInstance, FAnimNode_IKRig* IKRigNode);
	virtual ~FIKRigAnimInstanceProxy() override = default;

	/** FAnimPreviewInstanceProxy interface */
	virtual void Initialize(UAnimInstance* InAnimInstance) override;
	virtual bool Evaluate(FPoseContext& Output) override;
	virtual void UpdateAnimationNode(const FAnimationUpdateContext& InContext) override;
	/** END FAnimPreviewInstanceProxy interface */

	/** FAnimInstanceProxy interface */
	/** Called when the anim instance is being initialized. These nodes can be provided */
	virtual FAnimNode_Base* GetCustomRootNode() override;
	virtual void GetCustomNodes(TArray<FAnimNode_Base*>& OutNodes) override;
	/** END FAnimInstanceProxy interface */

	void SetIKRigAsset(UIKRigDefinition* InIKRigAsset);

	FAnimNode_IKRig* IKRigNode;
};
