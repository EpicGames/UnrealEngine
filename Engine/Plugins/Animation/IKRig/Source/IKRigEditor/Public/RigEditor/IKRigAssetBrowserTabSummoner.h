// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FIKRigEditorToolkit;

struct FIKRigAssetBrowserTabSummoner : public FWorkflowTabFactory
{
public:
	static const FName TabID;
	
	FIKRigAssetBrowserTabSummoner(const TSharedRef<FIKRigEditorToolkit>& InIKRetargetEditor);
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual TSharedPtr<SToolTip> CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const override;

protected:
	TWeakPtr<FIKRigEditorToolkit> IKRigEditor;
};
