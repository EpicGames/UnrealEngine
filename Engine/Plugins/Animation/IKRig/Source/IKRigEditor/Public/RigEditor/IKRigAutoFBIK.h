// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Rig/IKRigDefinition.h"

class UIKRigController;

// what was the outcome of trying to automatically setup FBIK?
enum class EAutoFBIKResult
{
	AllOk,
	MissingMesh,
	UnknownSkeletonType,
	MissingChains,
	MissingPelvisBone
};


// the results of auto characterizing an input skeletal mesh
struct FAutoFBIKResults
{
	FAutoFBIKResults() : Outcome(EAutoFBIKResult::AllOk) {};
	
	EAutoFBIKResult Outcome;
	TArray<FName> MissingChains;
};

// given an IK Rig, will automatically generate an FBIK setup for use with retargeting
struct FAutoFBIKCreator
{
	FAutoFBIKCreator() = default;

	// call this function with any IK Rig controller to automatically create a FBIK setup
	void CreateFBIKSetup(const UIKRigController& IKRigController, FAutoFBIKResults& Results) const;
};
