// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "IKRigEditorStyle.h"

class FIKRigCommands : public TCommands<FIKRigCommands>
{
public:
	FIKRigCommands() : TCommands<FIKRigCommands>
	(
		"IKRig",
		NSLOCTEXT("Contexts", "IKRig", "IK Rig"),
		NAME_None, // "MainFrame"
		FIKRigEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}
	
	// reset whole system to initial state
	TSharedPtr< FUICommandInfo > Reset;

	// automatically generate retarget chains
	TSharedPtr< FUICommandInfo > AutoRetargetChains;

	// automatically setup full body ik
	TSharedPtr< FUICommandInfo > AutoSetupFBIK;

	// show settings of the asset in the details panel
	TSharedPtr< FUICommandInfo > ShowAssetSettings;

	// initialize commands
	virtual void RegisterCommands() override;
};
