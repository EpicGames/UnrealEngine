// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IKRigStructViewer.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/Object.h"

#include "IKRigController.generated.h"

#define UE_API IKRIGEDITOR_API

class UIKRigStructViewer;
struct FAutoCharacterizeResults;
struct FAutoCharacterizer;
struct FAutoFBIKCreator;
struct FAutoFBIKResults;
struct FRetargetDefinition;
struct FIKRigInputSkeleton;
struct FReferenceSkeleton;
struct FIKRigSkeleton;
struct FIKRigSolverBase;
struct FIKRigBoneSettingsBase;
struct FIKRigGoalSettingsBase;
struct FInstancedStruct;
class UIKRigDefinition;
class UIKRigEffectorGoal;
class USkeletalMesh;
class USkeleton;
struct FBoneChain;


// A singleton (per-asset) class used to make modifications to a UIKRigDefinition asset
//
// All modifications to an IKRigDefinition must go through this controller.
//
// Editors can subscribe to the callbacks on this controller to be notified of changes that require reinitialization
// of a running IK Rig processor instance.
// The API here is split into public/scripting sections which are accessible from Blueprint/Python and sections that
// are only relevant to editors in C++.
UCLASS(MinimalAPI, BlueprintType, hidecategories = UObject)
class UIKRigController : public UObject
{
	GENERATED_BODY()

public:

	UE_API UIKRigController();

	//
	// GENERAL PUBLIC/SCRIPTING API
	//
	
	// Use this to get the controller for the given IKRig
	UFUNCTION(BlueprintCallable, Category="IK Rig", meta = (DisplayName = "Get IK Rig Controller"))
	static UE_API UIKRigController* GetController(const UIKRigDefinition* InIKRigDefinition);
	
	// Sets the preview mesh to use. Loads the hierarchy into the asset's IKRigSkeleton.
	// Returns true if the mesh was able to be set. False if it was incompatible for any reason. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig", meta = (DisplayName = "Set IK Rig Skeletal Mesh"))
	UE_API bool SetSkeletalMesh(USkeletalMesh* SkeletalMesh) const;

	// Get the skeletal mesh this asset is initialized with 
	UFUNCTION(BlueprintCallable, Category="IK Rig", meta = (DisplayName = "Get IK Rig Skeletal Mesh"))
	UE_API USkeletalMesh* GetSkeletalMesh() const;

	// Returns true if the provided skeletal mesh could be used with this IK Rig.
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API bool IsSkeletalMeshCompatible(USkeletalMesh* SkeletalMeshToCheck) const;

	//
	// GENERAL C++ ONLY API
	//
	
	// Get the asset this controller controls.
	// Warning: Do not use for general editing of the data model.
	// If you need to make modifications to the asset, do so through the API provided by this controller. 
	UE_API UIKRigDefinition* GetAsset() const;
	UE_API TObjectPtr<UIKRigDefinition>& GetAssetPtr();  
	
	// force all currently connected processors to reinitialize using latest asset state 
	UE_API void BroadcastNeedsReinitialized() const;
	
	//
	// SKELETON C++ ONLY API
	//
	
	// Get read-access to the IKRig skeleton representation 
	UE_API const FIKRigSkeleton& GetIKRigSkeleton() const;

	//
	// SOLVERS PUBLIC/SCRIPTING API
	//

	// Add a new solver of the given type to the bottom of the stack. Returns the stack index.
	// The type here is the full package path of the UStruct type. For example: /Script/IKRig.FullBodyIKSolver
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API int32 AddSolver(const FString InIKRigSolverType) const;

	// Remove the solver at the given stack index. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool RemoveSolver(const int32 SolverIndex) const;

	// Get the number of solvers in the stack. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API int32 GetNumSolvers() const;

	// Move the solver at the given index to the target index. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool MoveSolverInStack(int32 SolverToMoveIndex, int32 TargetSolverIndex) const;

	// Set enabled/disabled status of the given solver. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetSolverEnabled(int32 SolverIndex, bool bIsEnabled) const;

	// Get enabled status of the given solver. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API bool GetSolverEnabled(int32 SolverIndex) const;

	// Set the root bone on a given solver. (not all solvers support root bones, checks CanSetStartBone() first) 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetStartBone(const FName RootBoneName, int32 SolverIndex) const;

	// Get the name of the root bone on a given solver. (not all solvers support start bones, checks CanSetStartBone() first) 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API FName GetStartBone(int32 SolverIndex) const;

	// Set the end bone on a given solver. (not all solvers require extra end bones, checks CanSetEndBone() first) 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetEndBone(const FName EndBoneName, int32 SolverIndex) const;

	// Get the name of the end bone on a given solver. (not all solvers require extra end bones, checks CanSetEndBone() first) 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API FName GetEndBone(int32 SolverIndex) const;

	// Get a controller to customize the settings for a given solver in the stack 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API UIKRigSolverControllerBase* GetSolverController(int32 SolverIndex);

	// Get the index for a given solver in the stack
	// The input here is a solver controller (as returned by GetSolverController())
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API int32 GetIndexOfSolver(UIKRigSolverControllerBase* InController);

	//
	// SOLVERS C++ ONLY API
	//

	// Add a solver to the IK Rig asset's solver stack
	UE_API int32 AddSolver(UScriptStruct* InIKRigSolverType) const;

	// Get access to the given solver. 
	UE_API FIKRigSolverBase* GetSolverAtIndex(int32 Index) const;

	// Get access to the instanced struct for a given solver. 
	UE_API FInstancedStruct* GetSolverStructAtIndex(int32 Index) const;

	// Get access to the given solver. 
	UE_API int32 GetIndexOfSolver(FIKRigSolverBase* Solver) const;
	
	// Get read-only access to the array of solvers. 
	UE_API const TArray<FIKRigSolverBase*> GetSolverArray() const;

	// Get unique label for a given solver. Returns dash separated index and name like so, "0 - SolverName". 
	UE_API FString GetSolverUniqueName(int32 SolverIndex) const;

	//
	// GOALS PUBLIC/SCRIPTING API
	//
	
	// Add a new Goal associated with the given Bone. GoalName must be unique. Bones can have multiple Goals (rare). 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API FName AddNewGoal(const FName GoalName, const FName BoneName) const;

	// Remove the Goal by name. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool RemoveGoal(const FName GoalName) const;

	// Rename a Goal. Returns new name, which may be different after being sanitized. Returns NAME_None if this fails.
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API FName RenameGoal(const FName OldName, const FName PotentialNewName) const;

	// The the Bone that the given Goal should be parented to / associated with. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetGoalBone(const FName GoalName, const FName NewBoneName) const;

	// The the Bone associated with the given Goal. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API FName GetBoneForGoal(const FName GoalName) const;

	// Get the Goal associated with the given Bone (may be NAME_None) 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API FName GetGoalNameForBone(const FName BoneName) const;

	// Connect the given Goal to the given Solver. This creates an "Effector" with settings specific to this Solver.
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool ConnectGoalToSolver(const FName GoalName, int32 SolverIndex) const;

	// Disconnect the given Goal from the given Solver. This removes the Effector that associates the Goal with the Solver.
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool DisconnectGoalFromSolver(const FName GoalToRemove, int32 SolverIndex) const;

	// Returns true if the given Goal is connected to the given Solver. False otherwise. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API bool IsGoalConnectedToSolver(const FName GoalName, int32 SolverIndex) const;

	// Returns true if the given Goal is connected to ANY solver. False otherwise. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API bool IsGoalConnectedToAnySolver(const FName GoalName) const;

	// Get read-write access to the Goal with the given name.
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API UIKRigEffectorGoal* GetGoal(const FName GoalName) const;

	// Get access to the list of Goals. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API const TArray<UIKRigEffectorGoal*>& GetAllGoals() const;

	// Get the UObject for the settings associated with the given Goal in the given Solver.
	// Solvers can define their own per-Goal settings depending on their needs. These are termed "Effectors".
	UE_DEPRECATED(5.6, "In 5.6+ each solver provides it's own controller. Use GetSolverController(SolverIndex).GetGoalSettings(GoalName) instead.")
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API UObject* GetGoalSettingsForSolver(const FName GoalName, int32 SolverIndex) const;

	// returns true if the goal has custom settings for the given solver
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API bool DoesSolverHaveCustomGoalSettings(const FName GoalName, int32 SolverIndex) const;
	
	//
	// GOALS C++ ONLY API
	//
	
	// Get the index of the given Goal in the list of Goals. 
	UE_API int32 GetGoalIndex(const FName InGoalName, const ENameCase CompareMethod = ENameCase::IgnoreCase) const;

	// Get the global-space transform of the given Goal. This may be set by the user in the editor, or at runtime. 
	UE_API FTransform GetGoalCurrentTransform(const FName GoalName) const;

	// Set the Goal to the given transform. 
	UE_API void SetGoalCurrentTransform(const FName GoalName, const FTransform& Transform) const;

	// Reset all Goals back to their initial transforms. 
	UE_API void ResetGoalTransforms() const;

	// Reset initial goal transform 
	UE_API void ResetInitialGoalTransforms() const;

	// Ensure that the given name adheres to required standards for Goal names (no special characters etc..)
	static UE_API void SanitizeGoalName(FString& InOutName);

	// Add a suffix as needed to ensure the Goal name is unique 
	UE_API FName GetUniqueGoalName(const FName NameToMakeUnique) const;

	// Modify a Goal for a transaction. Returns true if Goal found.
	UE_API bool ModifyGoal(const FName GoalName) const;

	//
	// BONES PUBLIC/SCRIPTING API
	//

	// Include/exclude a bone from all the solvers. All bones are included by default. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetBoneExcluded(const FName BoneName, const bool bExclude) const;

	// Returns true if the given Bone is excluded, false otherwise. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API bool GetBoneExcluded(const FName BoneName) const;
	
	// Add settings to the given Bone/Solver. Does nothing if Bone already has settings in this Solver.
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool AddBoneSetting(const FName BoneName, int32 SolverIndex) const;

	// Remove settings for the given Bone/Solver. Does nothing if Bone doesn't have setting in this Solver.
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool RemoveBoneSetting(const FName BoneName, int32 SolverIndex) const;

	// Get the global-space retarget pose transform of the given Bone.
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API FTransform GetRefPoseTransformOfBone(const FName BoneName) const;

	// Get the generic (Solver-specific) Bone settings UObject for this Bone in the given Solver.
	UE_DEPRECATED(5.6, "In 5.6+ each solver provides it's own controller. Use GetSolverController(SolverIndex).GetBoneSettings(BoneName) instead.")
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API UObject* GetBoneSettings(const FName BoneName, int32 SolverIndex) const;

	//
	// BONES C++ ONLY API
	//

	// Get the generic (Solver-specific) Bone settings UStruct for this Bone in the given Solver.
	UE_API FIKRigBoneSettingsBase* GetSettingsForBone(const FName BoneName, int32 SolverIndex) const;

	// Returns true if this Bone can have settings in the given Solver. 
	UE_API bool CanAddBoneSetting(const FName BoneName, int32 SolverIndex, FText* OutError = nullptr) const;

	// Returns true if settings for this Bone can be removed from the given Solver. 
	UE_API bool CanRemoveBoneSetting(const FName BoneName, int32 SolverIndex) const;

	// Returns true if the given Bone has any settings in any Solver. 
	UE_API bool DoesBoneHaveSettings(const FName BoneName) const;

	//
	// RETARGETING PUBLIC/SCRIPTING API
	//

	// Add a new chain with the given Chain and Bone names. Returns newly created chain name (uniquified).
	// Note: only the ChainName is required here, all else can be set later.
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API FName AddRetargetChain(const FName ChainName, const FName StartBoneName, const FName EndBoneName, const FName GoalName) const;

	// Remove a Chain with the given name. Returns true if a Chain was removed. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool RemoveRetargetChain(const FName ChainName) const;

	// Renamed the given Chain. Returns the new name (same as old if unsuccessful). 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API FName RenameRetargetChain(const FName ChainName, const FName NewChainName) const;

	// Set the Start Bone for the given Chain. Returns true if operation was successful. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetRetargetChainStartBone(const FName ChainName, const FName StartBoneName) const;

	// Set the End Bone for the given Chain. Returns true if operation was successful. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetRetargetChainEndBone(const FName ChainName, const FName EndBoneName) const;

	// Set the Goal for the given Chain. Returns true if operation was successful. 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetRetargetChainGoal(const FName ChainName, const FName GoalName) const;

	// Get the Goal name for the given Chain. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API FName GetRetargetChainGoal(const FName ChainName) const;

	// Get the End Bone name for the given Chain. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
    UE_API FName GetRetargetChainStartBone(const FName ChainName) const;

	// Get the Start Bone name for the given Chain. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
    UE_API FName GetRetargetChainEndBone(const FName ChainName) const;

	// Get read-only access to the list of Chains. 
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API const TArray<FBoneChain>& GetRetargetChains() const;

	// Set the Pelvis Bone of the retargeting (can only be one). 
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig", DisplayName="SetPelvisBone")
	UE_API bool SetRetargetRoot(const FName RootBoneName) const;

	// Get the name of the Pelvis Bone of the retargeting (can only be one). 
	UFUNCTION(BlueprintCallable, Category="IK Rig", DisplayName="GetPelvisBone")
	UE_API FName GetRetargetRoot() const;

	// Set the bone used to store root motion for the skeleton (could be different from the root of the hierarchy).
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="IK Rig")
	UE_API bool SetRootMotionBone(const FName RootBoneName) const;

	// Get the bone used to store root motion for the skeleton (could be different from the root of the hierarchy).
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API FName GetRootMotionBone() const;

	// Analyse the skeleton to see if it matches a known template and automatically generates all retarget chains and sets the pelvis
	// Returns true if a matching skeletal template was found and the retarget definition for it was applied.
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API bool ApplyAutoGeneratedRetargetDefinition();

	// Analyse the skeleton to see if it matches a known template and automatically generates a full body IK setup
	// Returns true if a matching skeletal template was found and the FBIK setup for it was applied.
	UFUNCTION(BlueprintCallable, Category="IK Rig")
	UE_API bool ApplyAutoFBIK();
	
	//
	// RETARGETING C++ ONLY API
	//

	// Replace the entire retarget definition (includes all bone chains and the pelvis setting)
	UE_API void SetRetargetDefinition(const FRetargetDefinition& RetargetDefinition) const;

	// Auto generates a retarget definition and returns the results
	UE_API void AutoGenerateRetargetDefinition(FAutoCharacterizeResults& Results) const;

	// Auto generates an FBIK setup for the current skeletal mesh
	UE_API void AutoGenerateFBIK(FAutoFBIKResults& Results) const;

	// Get read-only access to characterizer
	UE_API const FAutoCharacterizer& GetAutoCharacterizer() const;
	
	// Add a Chain with the given BoneChain settings. Returns newly created chain name.
	UE_API FName AddRetargetChainInternal(const FBoneChain& BoneChain) const;
	
	// Get read-only access to a single retarget chain with the given name 
	UE_API const FBoneChain* GetRetargetChainByName(const FName ChainName) const;
	
	// Get the name of the retarget chain that contains the given Bone. Returns NAME_None if Bone not in a Chain. 
	UE_API FName GetRetargetChainFromBone(const FName BoneName, const FIKRigSkeleton* OptionalSkeleton) const;

	// Get the name of the retarget chain that contains the given Goal. Returns NAME_None if Goal not in a Chain. 
	UE_API FName GetRetargetChainFromGoal(const FName GoalName) const;

	// Sorts the Chains from Root to tip based on the Start Bone of each Chain. 
	UE_API void SortRetargetChains() const;

	// Make unique name for a retargeting bone chain. Adds a numbered suffix to make it unique.
	UE_API FName GetUniqueRetargetChainName(FName NameToMakeUnique) const;

	// Returns true if this is a valid chain. Produces array of bone indices between start and end (inclusive).
	// Optionally provide a runtime skeleton from an IKRigProcessor to get indices for a running instance (otherwise uses stored hierarchy in asset)
	UE_API bool ValidateChain( const FName ChainName, const FIKRigSkeleton* OptionalSkeleton, TSet<int32>& OutChainIndices) const;

	// returns an array of wrappers to display structs in the details panel
	// NOTE: these are recycled by the editor using this controller (assumes one editor per asset)
	UE_API TArray<UIKRigStructWrapperBase*> GetStructWrappers(const int32 InNum, TSubclassOf<UIKRigStructWrapperBase> InType);

private:

	// Called whenever the rig is modified in such a way that would require re-initialization by dependent systems.
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnIKRigNeedsInitialized, UIKRigDefinition*);
	FOnIKRigNeedsInitialized IKRigNeedsInitialized;

	// Called whenever a retarget chain is added.
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRetargetChainAdded, UIKRigDefinition*);
	FOnRetargetChainAdded RetargetChainAdded;

	// Called whenever a retarget chain is removed.
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRetargetChainRemoved, UIKRigDefinition*, const FName /*chain name*/);
	FOnRetargetChainRemoved RetargetChainRemoved;

	// Called whenever a retarget chain is renamed.
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnRetargetChainRenamed, UIKRigDefinition*, FName /*old name*/, FName /*new name*/);
	FOnRetargetChainRenamed RetargetChainRenamed;

public:
	
	FOnIKRigNeedsInitialized& OnIKRigNeedsInitialized(){ return IKRigNeedsInitialized; };
	FOnRetargetChainAdded& OnRetargetChainAdded(){ return RetargetChainAdded; };
	FOnRetargetChainRemoved& OnRetargetChainRemoved(){ return RetargetChainRemoved; };
	FOnRetargetChainRenamed& OnRetargetChainRenamed(){ return RetargetChainRenamed; };

private:

	// auto characterizer
	TUniquePtr<FAutoCharacterizer> AutoCharacterizer;

	// auto fbik generator
	TUniquePtr<FAutoFBIKCreator> AutoFBIKCreator;

	// prevent reinitializing from inner operations
	mutable int32 ReinitializeScopeCounter = 0;
	mutable bool bGoalsChangedInScope = false;

	// The actual IKRigDefinition asset that this Controller modifies. 
	UPROPERTY(transient)
	TObjectPtr<UIKRigDefinition> Asset = nullptr;

	// a pool of objects inheriting from UIKRigStructWrapperBase for editing various IK Rig struct types in a details view
	// NOTE: pool contains a bucket per settings type
	UPROPERTY(transient)
	FIKRigStructWrapperPool StructViewerPool;

	// Static global map of assets to their associated controller
	static UE_API TMap<UIKRigDefinition*, UIKRigController*> Controllers;

	// broadcast changes within the asset goals array
	UE_API void BroadcastGoalsChange() const;
	
	friend class UIKRigDefinition;
	friend struct FScopedReinitializeIKRig;
};

struct FScopedReinitializeIKRig
{
	FScopedReinitializeIKRig(const UIKRigController *InController, const bool bGoalsChanged=false)
	{
		InController->ReinitializeScopeCounter++;
		InController->bGoalsChangedInScope = bGoalsChanged ? true : InController->bGoalsChangedInScope;
		Controller = InController;
	}
	~FScopedReinitializeIKRig()
	{
		if (--Controller->ReinitializeScopeCounter == 0)
		{
			Controller->BroadcastNeedsReinitialized();
			
			if (Controller->bGoalsChangedInScope)
			{
				Controller->bGoalsChangedInScope = false;
				Controller->BroadcastGoalsChange();
			}
		}
	};

	const UIKRigController* Controller;
};

#undef UE_API
