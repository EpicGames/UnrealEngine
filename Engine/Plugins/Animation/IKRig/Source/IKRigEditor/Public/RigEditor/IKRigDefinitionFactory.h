// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "IKRigDefinitionFactory.generated.h"

#define UE_API IKRIGEDITOR_API

class SWindow;

UCLASS(MinimalAPI, BlueprintType, hidecategories=Object)
class UIKRigDefinitionFactory : public UFactory
{
	GENERATED_BODY()

public:

	UE_API UIKRigDefinitionFactory();

	// UFactory interface
	UE_API virtual UObject* FactoryCreateNew(
        UClass* Class,
        UObject* InParent,
        FName InName,
        EObjectFlags InFlags,
        UObject* Context,
        FFeedbackContext* Warn) override;
	UE_API virtual FText GetDisplayName() const override;
	UE_API virtual uint32 GetMenuCategories() const override;
	UE_API virtual FText GetToolTip() const override;
	UE_API virtual FString GetDefaultNewAssetName() const override;
	UE_API virtual bool ShouldShowInNewMenu() const override;
	// END UFactory interface

	/**
	 * Create a new IK Rig rig asset at a specified location in your project's content folder.
	 * @param InDesiredPackagePath The package path to where the new IK Rig will be placed. (ie "/Game/MyIKRigs/")
	 * @param InAssetName The name of the new asset (ie "IK_MyNewRig")
	 */
	UFUNCTION(BlueprintCallable, Category = "IK Rig", DisplayName="Create New IK Rig Asset")
	static UE_API UIKRigDefinition* CreateNewIKRigAsset(const FString& InPackagePath, const FString& InAssetName);
};

#undef UE_API
