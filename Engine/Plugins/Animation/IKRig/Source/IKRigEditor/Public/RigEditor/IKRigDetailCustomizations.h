// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "ScopedTransaction.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"
#include "RigEditor/IKRigEditorController.h"


class FIKRigGenericDetailCustomization : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FIKRigGenericDetailCustomization);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	/** Labels and transform types used by the transform widget */
	static const TArray<FText>& GetButtonLabels();
	static const TArray<EIKRigTransformType::Type>& GetTransformTypes();

	/** Per class customization */
	template<typename ClassToCustomize>
	void CustomizeDetailsForClass(IDetailLayoutBuilder& DetailBuilder, const TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized) {}

	/** Template specializations */
	template<>
	void CustomizeDetailsForClass<UIKRigBoneDetails>(IDetailLayoutBuilder& DetailBuilder, const TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized);
	template<>
	void CustomizeDetailsForClass<UIKRigEffectorGoal>(IDetailLayoutBuilder& DetailBuilder, const TArray<TWeakObjectPtr<UObject>>& ObjectsBeingCustomized);

	/** Transaction to handle value changed. It's being effective once the value committed */
	TSharedPtr<FScopedTransaction> ValueChangedTransaction;
};