// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Rig/IKRigDefinition.h"
#include "IPersonaEditMode.h"
#include "SkeletalDebugRendering.h"

class FIKRigEditorController;
class UIKRigEffectorGoal;
class FIKRigEditorToolkit;
class FIKRigPreviewScene;
struct FIKRigProcessor;
struct FGizmoState;

class FIKRigEditMode : public IPersonaEditMode
{
	
public:
	
	static FName ModeName;
	
	FIKRigEditMode();

	/** glue for all the editor parts to communicate */
	void SetEditorController(const TSharedPtr<FIKRigEditorController> InEditorController) { EditorController = InEditorController; };

	/** IPersonaEditMode interface */
	virtual bool GetCameraTarget(FSphere& OutTarget) const override;
	virtual class IPersonaPreviewScene& GetAnimPreviewScene() const override;
	virtual void GetOnScreenDebugInfo(TArray<FText>& OutDebugInfo) const override;
	/** END IPersonaEditMode interface */

	/** FEdMode interface */
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;
	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return true; }
	virtual bool AllowWidgetMove() override;
	virtual bool ShouldDrawWidget() const override;
	virtual bool UsesTransformWidget() const override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	virtual FVector GetWidgetLocation() const override;
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool InputKey(FEditorViewportClient* ViewportClient,FViewport* Viewport, FKey Key,EInputEvent Event) override;
	virtual bool BeginTransform(const FGizmoState& InState) override;
	virtual bool EndTransform(const FGizmoState& InState) override;
	/** END FEdMode interface */

private:
	
	void RenderGoals(FPrimitiveDrawInterface* PDI);
	void RenderBones(FPrimitiveDrawInterface* PDI);
	void GetBoneColors(
		FIKRigEditorController* Controller,
		const FIKRigProcessor& Processor,
		const FReferenceSkeleton& RefSkeleton,
		bool bUseMultiColorAsDefaultColor,
		TArray<FLinearColor>& OutBoneColors) const;

	bool HandleBeginTransform() const;
	bool HandleEndTransform() const;
	
	/** The hosting app */
	TWeakPtr<FIKRigEditorController> EditorController;
};
