// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rig/Solvers/IKRigSolverBase.h"
#include "SIKRigRetargetChainList.h"
#include "Templates/SubclassOf.h"
#include "Animation/AnimationAsset.h"
#include "SAdvancedTransformInputBox.h"

#include "IKRigEditorController.generated.h"

#define UE_API IKRIGEDITOR_API

class SIKRigAssetBrowser;
class SIKRigOutputLog;
class UIKRigAnimInstance;
class FIKRigEditorToolkit;
class SIKRigSolverStack;
class SIKRigHierarchy;
class UIKRigController;
class FSolverStackElement;
class FIKRigTreeElement;
class UAnimInstance;
class UDebugSkelMeshComponent;
class UIKRigStructViewer;
class UIKRigStructWrapperBase;
struct FIKRigProcessor;

enum EIKRigSelectionType : int8
{
	Hierarchy,
	SolverStack,
	RetargetChains,
};

UCLASS(MinimalAPI, config = Engine, hidecategories = UObject)
class UIKRigBoneDetails : public UObject
{
	GENERATED_BODY()

public:

	// todo update bone info automatically using something else
	void SetBone(const FName& BoneName)
	{
		SelectedBone = BoneName;
	};

	UPROPERTY(VisibleAnywhere, Category = "Selection")
	FName SelectedBone;
	
	UPROPERTY(VisibleAnywhere, Category = "Bone Transforms")
	FTransform CurrentTransform;

	UPROPERTY(VisibleAnywhere, Category = "Bone Transforms")
	FTransform ReferenceTransform;

	UPROPERTY()
	TWeakObjectPtr<UAnimInstance> AnimInstancePtr;

	UPROPERTY()
	TWeakObjectPtr<UIKRigDefinition> AssetPtr;

#if WITH_EDITOR

	UE_API TOptional<FTransform> GetTransform(EIKRigTransformType::Type TransformType) const;
	UE_API bool IsComponentRelative(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType) const;
	UE_API void OnComponentRelativeChanged(ESlateTransformComponent::Type Component, bool bIsRelative, EIKRigTransformType::Type TransformType);
	UE_API void OnCopyToClipboard(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType) const;
	UE_API void OnPasteFromClipboard(ESlateTransformComponent::Type Component, EIKRigTransformType::Type TransformType);

	template <typename DataType>
	void GetContentFromData(const DataType& InData, FString& Content) const
	{
		TBaseStructure<DataType>::Get()->ExportText(Content, &InData, &InData, nullptr, PPF_None, nullptr);
	}

#endif
	
private:
	
	static UE_API bool CurrentTransformRelative[3];
	static UE_API bool ReferenceTransformRelative[3];
};

enum class EChainSide
{
	Left,
	Right,
	Center
};

// use string matching and skeletal analysis to suggest a reasonable default name for retarget chains 
struct FRetargetChainAnalyzer
{
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;

	FRetargetChainAnalyzer()
	{
		TextFilter = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);
	}
	
	void AssignBestGuessName(FBoneChain& Chain, const FIKRigSkeleton& IKRigSkeleton);
	
	static FName GetDefaultChainName();

	EChainSide GetSideOfChain(const TArray<int32>& BoneIndices, const FIKRigSkeleton& IKRigSkeleton) const;
};

// a home for cross-widget communication to synchronize state across all tabs and viewport 
class FIKRigEditorController : public TSharedFromThis<FIKRigEditorController>, FGCObject
{
public:

	// initialize the editor controller to an instance of the IK Rig editor 
	void Initialize(TSharedPtr<FIKRigEditorToolkit> Toolkit, const UIKRigDefinition* Asset);
	// cleanup when editor closed 
	void Close() const;

	// get the currently active processor running the IK Rig in the editor 
	FIKRigProcessor* GetIKRigProcessor() const;
	// get the currently running IKRig skeleton (if there is a running processor) 
	const FIKRigSkeleton* GetCurrentIKRigSkeleton() const;

	// callback when IK Rig requires re-initialization 
	void HandleIKRigNeedsInitialized(UIKRigDefinition* ModifiedIKRig) const;

	// delete key pressed (in either viewport or tree view)
	void HandleDeleteSelectedElements();

	// create goals 
	void AddNewGoals(const TArray<FName>& GoalNames, const TArray<FName>& BoneNames);
	// clear all selected objects 
	void ClearSelection() const;
	// callback when goal is selected in the viewport 
	void HandleGoalSelectedInViewport(const FName& GoalName, bool bReplace) const;
	// callback when bone is selected in the viewport 
	void HandleBoneSelectedInViewport(const FName& BoneName, bool bReplace) const;
	// reset all goals to initial transforms 
	void Reset() const;
	// refresh just the skeleton tree view 
	void RefreshTreeView() const;
	// clear the output log 
	void ClearOutputLog() const;
	// automatically generates retarget chains 
	void AutoGenerateRetargetChains() const;
	// automatically generates IK setup 
	void AutoGenerateFBIK() const;

	// return list of those solvers in the stack that are selected by user 
	void GetSelectedSolvers(TArray<TSharedPtr<FSolverStackElement> >& OutSelectedSolvers) const;
	// get index of the first selected solver, return INDEX_None if nothing selected 
	int32 GetSelectedSolverIndex();
	// get names of all goals that are selected 
	void GetSelectedGoalNames(TArray<FName>& OutGoalNames) const;
	// return the number of selected goals 
	int32 GetNumSelectedGoals() const;
	// get names of all bones that are selected 
	void GetSelectedBoneNames(TArray<FName>& OutBoneNames) const;
	// get all bones that are selected 
	void GetSelectedBones(TArray<TSharedPtr<FIKRigTreeElement>>& OutBoneItems) const;
	// returns true if Goal is currently selected 
	bool IsGoalSelected(const FName& GoalName) const;
	// is anything selected in the skeleton view? 
	bool DoesSkeletonHaveSelectedItems() const;
	// get name of the selected retargeting chain 
	TArray<FName> GetSelectedChains() const;
	// get chains selected in skeleton view 
	void GetChainsSelectedInSkeletonView(TArray<FBoneChain>& InOutChains);
	
	// create new retarget chains from selected bones (or single empty chain if no selection) 
	void CreateNewRetargetChains();
	
	// show single transform of bone in details view 
	void ShowDetailsForBone(const FName BoneName) const { ShowDetailsForBones({BoneName}); }
	void ShowDetailsForBones(const TArray<FName>& BoneNames) const;
	// show BONE settings in details view 
	void ShowDetailsForBoneSettings(const TArray<TSharedPtr<FIKRigTreeElement>>& InItems) const;
	// show GOAL settings in details view 
	void ShowDetailsForGoals(const TArray<FName>& GoalName) const;
	// show EFFECTOR settings in details view 
	void ShowDetailsForGoalSettings(const TArray<TSharedPtr<FIKRigTreeElement>>& InItems) const;
	// show SOLVER settings in details view 
	void ShowDetailsForSolvers(const TArray<int32>& InSolverIndices) const;
	// show nothing in details view 
	void ShowAssetDetails() const;
	// show details for items 
	void ShowDetailsForElements(const TArray<TSharedPtr<FIKRigTreeElement>>& InItems) const;
	// show the currently configured IKRigStructWrapper objects in the details tab
	void ShowStructWrappersInDetailsView(TArray<UIKRigStructWrapperBase*> InStructWrappers) const;
	// callback when detail is edited 
	void OnFinishedChangingDetails(const FPropertyChangedEvent& PropertyChangedEvent) const;
	// returns true if the supplied UObject is being shown in the details panel
	bool IsObjectInDetailsView(const UObject* Object) const;
	
	// set details tab view 
	void SetDetailsView(const TSharedPtr<class IDetailsView>& InDetailsView);
	// set skeleton tab view 
	void SetHierarchyView(const TSharedPtr<SIKRigHierarchy>& InSkeletonView){ SkeletonView = InSkeletonView; };
	// set solver stack tab view 
	void SetSolverStackView(const TSharedPtr<SIKRigSolverStack>& InSolverStackView){ SolverStackView = InSolverStackView; };
	// set retargeting tab view 
	void SetRetargetingView(const TSharedPtr<SIKRigRetargetChainList>& InRetargetingView){ RetargetingView = InRetargetingView; };
	// set output log view 
	void SetOutputLogView(const TSharedPtr<SIKRigOutputLog>& InOutputLogView){ OutputLogView = InOutputLogView; };

	// right after importing a skeleton, we ask user what solver they want to use 
	bool PromptToAddDefaultSolver() const;
	// show user the new retarget chain they are about to create (provides option to edit name) 
	FName PromptToAddNewRetargetChain(FBoneChain& BoneChain) const;
	// right after creating a goal, we ask user if they want it assigned to a retarget chain 
	void PromptToAssignGoalToChain(FName NewGoalName) const;

	// play preview animation on running anim instance in editor (before IK) 
	void PlayAnimationAsset(UAnimationAsset* AssetToPlay);
	
	// all modifications to the data model should go through this controller
	TObjectPtr<UIKRigController> AssetController;

	// viewport skeletal mesh 
	UDebugSkelMeshComponent* SkelMeshComponent;

	// viewport anim instance 
	TObjectPtr<UIKRigAnimInstance> AnimInstance;

	// the persona toolkit 
	TWeakPtr<FIKRigEditorToolkit> EditorToolkit;

	// used to analyze a retarget chain 
	FRetargetChainAnalyzer ChainAnalyzer;

	// UI and viewport selection state 
	bool bManipulatingGoals = false;

	// record which part of the UI was last selected 
	EIKRigSelectionType GetLastSelectedType() const;
	void SetLastSelectedType(EIKRigSelectionType SelectionType);
	// END selection type 

	// FGCObject interface 
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObjects(BoneDetails);
		Collector.AddReferencedObject(AnimInstance);
		Collector.AddReferencedObject(AssetController);
	};
	virtual FString GetReferencerName() const override { return "IKRigEditorController"; };
	// END FGCObject interface 

	// UIKRigBoneDetails factory *
	TObjectPtr<UIKRigBoneDetails> CreateBoneDetails(const TSharedPtr<FIKRigTreeElement const>& InItem) const;
	
private:

	TArray<TSharedPtr<FIKRigTreeElement>> FilterByFirstItemType(
		const TArray<TSharedPtr<FIKRigTreeElement>>& InItems,
		TFunctionRef<TSubclassOf<UObject>(const TSharedPtr<FIKRigTreeElement>&)> GetTypeOfItem) const;

	// asset properties tab 
	TSharedPtr<IDetailsView> DetailsView;

	// the skeleton tree view 
	TSharedPtr<SIKRigHierarchy> SkeletonView;
	
	// the solver stack view 
	TSharedPtr<SIKRigSolverStack> SolverStackView;

	// the solver stack view 
	TSharedPtr<SIKRigRetargetChainList> RetargetingView;

	// asset browser view 
	TSharedPtr<SIKRigAssetBrowser> AssetBrowserView;

	// output log view 
	TSharedPtr<SIKRigOutputLog> OutputLogView;

	// when prompting user to assign a skeletal mesh 
	TSharedPtr<SWindow> MeshPickerWindow;

	EIKRigSelectionType LastSelectedType;

	mutable TArray<TObjectPtr<UIKRigBoneDetails>> BoneDetails;

	// remove delegate when closing editor 
	FDelegateHandle ReinitializeDelegateHandle;

	friend struct FIKRigOutputLogTabSummoner;
	friend class SIKRigAssetBrowser;
};

struct FIKRigSolverTypeAndName
{
	FText NiceName;
	UScriptStruct* SolverType;
};

#undef UE_API
