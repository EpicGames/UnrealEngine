// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateTypes.h"

class FIKRigEditorStyle	final : public FSlateStyleSet
{
public:
	
	FIKRigEditorStyle() : FSlateStyleSet("IKRigEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		const FVector2D Icon64x64(64.0f, 64.0f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Animation/IKRig/Content"));
		
		Set("IKRig.Tree.Bone", new IMAGE_BRUSH("Slate/Bone_16x", Icon16x16));
		Set("IKRig.Tree.BoneWithSettings", new IMAGE_BRUSH("Slate/BoneWithSettings_16x", Icon16x16));
		Set("IKRig.Tree.Goal", new IMAGE_BRUSH("Slate/Goal_16x", Icon16x16));
		Set("IKRig.Tree.Effector", new IMAGE_BRUSH("Slate/Effector_16x", Icon16x16));
		Set("IKRig.TabIcon", new IMAGE_BRUSH("Slate/Tab_16x", Icon16x16));
		Set("IKRig.Solver", new IMAGE_BRUSH("Slate/Solver_16x", Icon16x16));
		Set("IKRig.DragSolver", new IMAGE_BRUSH("Slate/DragSolver", FVector2D(6, 15)));
		Set("IKRig.Reset", new IMAGE_BRUSH("Slate/Reset", Icon40x40));
		Set("IKRig.Reset.Small", new IMAGE_BRUSH("Slate/Reset", Icon20x20));

		Set("ClassIcon.IKRigDefinition", new IMAGE_BRUSH_SVG("Slate/IKRig", Icon16x16));
		Set("ClassThumbnail.IKRigDefinition", new IMAGE_BRUSH_SVG("Slate/IKRig_64", Icon64x64));

		Set("IKRig.SolverStack", new IMAGE_BRUSH_SVG("Slate/SolverStack", Icon64x64));
		Set("IKRig.IKRetargeting", new IMAGE_BRUSH_SVG("Slate/IKRetargeting", Icon64x64));
		Set("IKRig.Hierarchy", new IMAGE_BRUSH_SVG("Slate/Hierarchy", Icon64x64));

		Set("IKRig.AssetSettings", new IMAGE_BRUSH_SVG("Slate/AssetSettings", Icon64x64));
		Set("IKRig.AutoRetarget", new IMAGE_BRUSH_SVG("Slate/AutoChainRetarget", Icon64x64));
		Set("IKRig.AutoIK", new IMAGE_BRUSH_SVG("Slate/AutoFullBodyIK", Icon64x64));
		
		FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("SkeletonTree.NormalFont");
		Set( "IKRig.Tree.NormalText", FTextBlockStyle(NormalText));
		Set( "IKRig.Tree.ItalicText", FTextBlockStyle(NormalText).SetFont(DEFAULT_FONT("Italic", 10)));

		SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
		Set( "IKRig.Viewport.Border", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f, FLinearColor(1.0f,1.0f,1.0f,1.0f) ) );
		
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FIKRigEditorStyle& Get()
	{
		static FIKRigEditorStyle Inst;
		return Inst;
	}
	
	~FIKRigEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};