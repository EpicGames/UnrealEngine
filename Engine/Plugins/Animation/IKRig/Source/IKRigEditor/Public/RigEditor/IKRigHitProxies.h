// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HitProxies.h"
#include "UObject/NameTypes.h"

struct HIKRigEditorBoneProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	FName BoneName;

	HIKRigEditorBoneProxy(const FName& InBoneName);

	virtual EMouseCursor::Type GetMouseCursor() override;
	virtual bool AlwaysAllowsTranslucentPrimitives() const override;
};

struct HIKRigEditorGoalProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	FName GoalName;
	
	HIKRigEditorGoalProxy(const FName& InGoalName);

	virtual EMouseCursor::Type GetMouseCursor() override;
	virtual bool AlwaysAllowsTranslucentPrimitives() const override;
};