// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorkflowOrientedApp/ApplicationMode.h"
#include "WorkflowOrientedApp/WorkflowCentricApplication.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FIKRigEditorToolkit;

class FIKRigMode : public FApplicationMode
{
	
public:
	
	FIKRigMode(TSharedRef<class FWorkflowCentricApplication> InHostingApp, TSharedRef<class IPersonaPreviewScene> InPreviewScene);

	/** FApplicationMode interface */
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	/** END FApplicationMode interface */

protected:
	
	/** The hosting app */
	TWeakPtr<FIKRigEditorToolkit> IKRigEditorPtr;

	/** The tab factories we support */
	FWorkflowAllowedTabSet TabFactories;
};
