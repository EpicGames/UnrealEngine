// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "IKRigEditorStyle.h"

class FIKRigSkeletonCommands : public TCommands<FIKRigSkeletonCommands>
{
public:
	FIKRigSkeletonCommands() : TCommands<FIKRigSkeletonCommands>
	(
		"IKRigSkeleton",
		NSLOCTEXT("Contexts", "IKRigSkeleton", "IK Rig Skeleton"),
		NAME_None,
		FIKRigEditorStyle::Get().GetStyleSetName() // Icon Style Set
	)
	{}
	
	/** create new goal on selected bone */
	TSharedPtr< FUICommandInfo > NewGoal;
	/** delete selected element */
	TSharedPtr< FUICommandInfo > DeleteElement;
	/** add goal to selected solvers */
	TSharedPtr< FUICommandInfo > ConnectGoalToSolver;
	/** remove goal to selected solvers */
	TSharedPtr< FUICommandInfo > DisconnectGoalFromSolver;

	/** set root bone on selected solvers */
	TSharedPtr< FUICommandInfo > SetStartBoneOnSolver;

	/** set end bone on selected solvers (Pole Solvers only)*/
	TSharedPtr< FUICommandInfo > SetEndBoneOnSolver;
	
	/** add bone setting to bone */
	TSharedPtr< FUICommandInfo > AddBoneSettings;
	/** remove bone setting from bone */
	TSharedPtr< FUICommandInfo > RemoveBoneSettings;
	
	/** remove bone from solve */
	TSharedPtr< FUICommandInfo > ExcludeBone;
	/** add bone to solve */
	TSharedPtr< FUICommandInfo > IncludeBone;

	/** create retarget chain */
	TSharedPtr< FUICommandInfo > NewRetargetChain;
	/** set the pelvis bone */
	TSharedPtr< FUICommandInfo > SetPelvis;
	/** clear the pelvis bone */
	TSharedPtr< FUICommandInfo > ClearPelvis;

	/** rename selected goal */
	TSharedPtr< FUICommandInfo > RenameGoal;

	/** reload the hierarchy */
	TSharedPtr< FUICommandInfo > RefreshHierarchy;
	
	/** initialize commands */
	virtual void RegisterCommands() override;
};
