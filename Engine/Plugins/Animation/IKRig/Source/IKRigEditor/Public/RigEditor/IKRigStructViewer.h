// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "ScopedTransaction.h"
#include "Interfaces/Interface_BoneReferenceSkeletonProvider.h"
#include "RigEditor/IKRigEditorController.h"
#include "UObject/SavePackage.h"

#include "IKRigStructViewer.generated.h"


class UIKRetargeterController;

struct FIKRigStructToView
{
	// the type that corresponds to the struct memory returned by the StructMemoryProvider
	const UScriptStruct* Type = nullptr;
	// provides the memory address of the struct to edit (refreshed after undo/redo)
	// NOTE: we can't just pass in raw pointers to the struct memory because these can be destroyed after a transaction.
	// So instead we pass in TFunction callbacks that get the latest memory locations when the details panel is refreshed.
	TFunction<uint8*()> MemoryProvider;
	// a UObject that owns the struct (this is what will be transacted when the property is edited)
	TWeakObjectPtr<UObject> Owner = nullptr;
	// a unique identifier that callbacks can use to know what struct was modified
	FName UniqueName;

	void Reset()
	{
		Type = nullptr;
		MemoryProvider.Reset();
		Owner = nullptr;
		UniqueName = NAME_None;
	}
	
	bool IsValid() const
	{
		if (!ensure(MemoryProvider.IsSet()))
		{
			return false;
		}
		
		uint8* Memory = MemoryProvider();
		if (!ensure(Memory != nullptr))
		{
			return false;
		}
		
		return Type && Owner.IsValid() && UniqueName != NAME_None;
	}
};

// a thin wrapper around UStruct data to display in a details panel
// this is a generic wrapper that works for any struct
// it is intended to work with FIKRigStructViewerCustomization which simply puts the entire struct in the details panel
// if you need customization, you need to work with UIKRigStructWrapperBase which allows customized derived classes
UCLASS(Blueprintable)
class UIKRigStructViewer : public UObject, public IBoneReferenceSkeletonProvider
{
	GENERATED_BODY()

public:
	
	/**
	 * Configures an instance of a struct to display in the details panel with undo/redo support.
	 * @param InStructToView the struct to view (see FStructToView for details) */
	void SetStructToView(const FIKRigStructToView& InStructToView)
	{
		StructToView = InStructToView;
	}

	virtual bool IsValid() const
	{
		return StructToView.IsValid();
	}

	void Reset()
	{
		StructToView.Reset();
	}

	TSharedPtr<FStructOnScope>& GetStructOnScope()
	{
		uint8* Memory = StructToView.MemoryProvider();
		StructOnScope = MakeShared<FStructOnScope>(StructToView.Type, Memory);
		return StructOnScope;
	}

	FName GetTypeName() const
	{
		// try to get the "nice name" from metadata
		FString TypeName = StructToView.Type->GetMetaData(TEXT("DisplayName"));

		// if no "DisplayName" metadata is found, fall back to the struct name
		if (TypeName.IsEmpty())
		{
			TypeName = StructToView.Type->GetName();
		}
		
		return FName(TypeName);
	}

	UObject* GetStructOwner() const
	{
		return StructToView.Owner.Get();
	};

	void TriggerReinitializationIfNeeded(const FPropertyChangedEvent& InEvent);

	void SetupPropertyEditingCallbacks(const TSharedPtr<IPropertyHandle>& InProperty);

	// IBoneReferenceSkeletonProvider overrides.
	virtual USkeleton* GetSkeleton(bool& bInvalidSkeletonIsError, const IPropertyHandle* PropertyHandle) override;
	// ~END IBoneReferenceSkeletonProvider overrides.

protected:

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnNeedsReinitialized, const FName& /*UniqueStructName*/, const FPropertyChangedEvent& /*Modified Property*/);
	FOnNeedsReinitialized NeedsReinitializedDelegate;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPreEditProperty, const FName& /*UniqueStructName*/, const FProperty* /*PropertyAboutToChange*/);
	FOnPreEditProperty PreEditPropertyDelegate;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPostEditProperty, const FName& /*UniqueStructName*/, const FPropertyChangedEvent& /*Modified Property*/);
	FOnPostEditProperty PostEditPropertyDelegate;
	
	// a wrapper of the struct sent to the details panel
	TSharedPtr<FStructOnScope> StructOnScope;
	// the data needed to display and edit an instance of a struct in memory
	FIKRigStructToView StructToView;

public:
	
	// attach a delegate to be notified whenever a property is edited that requires the system to be reinitialized
	// NOTE: This happens in a few hard-coded special cases or for any property marked with the meta tag: ReinitializeOnEdit
	FOnNeedsReinitialized& OnNeedsReinitialized(){ return NeedsReinitializedDelegate; };
	
	// attach a delegate to be notified BEFORE a property is edited
	FOnPreEditProperty& OnPreEditProperty(){ return PreEditPropertyDelegate; };
	
	// attach a delegate to be notified AFTER a property is edited
	FOnPostEditProperty& OnPostEditProperty(){ return PostEditPropertyDelegate; };	
};

class FIKRigStructViewerCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// END IDetailCustomization interface

	static TArray<IDetailPropertyRow*> AddAllPropertiesToCategoryGroups(
		const TSharedPtr<FStructOnScope>& StructData,
		IDetailLayoutBuilder& InDetailBuilder);
};

// this is meant to be subclassed by a type that contains a UProperty of a struct to be edited
// similar to UIKRigStructViewer but supports multi-struct editing and greater customization
UCLASS(BlueprintType)
class UIKRigStructWrapperBase : public UIKRigStructViewer
{
	GENERATED_BODY()

public:
	
	void Initialize(FIKRigStructToView& InStructToWrap, const FName& InWrapperPropertyName);

	void InitializeWithRetargeter(FIKRigStructToView& InStructToWrap, const FName& InWrapperPropertyName, UIKRetargeterController* InRetargeterController);

	virtual bool IsValid() const override;

	FName GetWrapperPropertyName() const { return WrapperPropertyName; };

	bool IsPropertyHidden(const FName& InPropertyName) const { return PropertiesToHide.Contains(InPropertyName); };

	void SetPropertyHidden(const FName& InPropertyName, bool bHidden);

	void UpdateWrappedStructWithLatestValues();

	void UpdateWrapperStructWithLatestValues();
	
	virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent ) override;

	// NOTE: How to add multi-edit support to IK Rig settings structs.
	// IK Rig settings structs can supply a "UIWrapper" meta data string with the path to a custom wrapper class
	// to be displayed in the details panel when editing that struct.
	//
	// In order to ensure that edits to the wrapper are pushed back to the actual struct-to-be-edited,
	// the wrapper must inherit from UIKRigStructWrapperBase. This stores a pointer to the original memory and
	// triggers callbacks when properties are edited to push those values back to the struct being edited.
	//
	// By default, only a single struct under a UProperty named "Settings" will be displayed in the details panel.
	// See example subclasses of UIKRigStructWrapperBase.
	static TSubclassOf<UIKRigStructWrapperBase> GetStructWrapperFromMetaData(const UScriptStruct* S)
	{
		const FString Path = S->GetMetaData(TEXT("UIWrapper"));
		if (Path.IsEmpty())
		{
			return nullptr;
		}
	
		const FSoftClassPath ClassPath(Path);
		TSoftClassPtr<UIKRigStructWrapperBase> SoftClass(ClassPath);
		return SoftClass.LoadSynchronous();
	}
	
private:
	
	TArray<FName> PropertiesToHide;
	FName WrapperPropertyName;
	FProperty* WrapperProperty;
};

class FIKRigStructWrapperCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// END IDetailCustomization interface
};

// NOTE: this is a dummy wrapper around TArray<TObjectPtr<>> that is necessary because we cannot store TArray<rawpointer> as a TMap value in the pool
USTRUCT()
struct FIKRigStructWrapperBucket
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	TArray<TObjectPtr<UIKRigStructWrapperBase>> Items;
};

USTRUCT()
struct FIKRigStructWrapperPool
{
	GENERATED_BODY()

public:
	// call once to give the pool an owning UObject to prevent GC, otherwise uses GetTransientPackage()
	void Init(UObject* InOuter) { CreationOuter = InOuter; }

	// get (or lazily instantiate) Num wrappers of the requested subclass.
	// NOTE: buckets will grow, but never shrink
	TArray<UIKRigStructWrapperBase*> GetStructWrappers(int32 Num, TSubclassOf<UIKRigStructWrapperBase> InType)
	{
		TArray<UIKRigStructWrapperBase*> Out;
		if (Num <= 0 || !InType)
		{
			return Out;
		}

		FIKRigStructWrapperBucket& Bucket = Pool.FindOrAdd(InType);

		// grow, but never shrink (preserves indices; no compaction)
		if (Bucket.Items.Num() < Num)
		{
			Bucket.Items.SetNum(Num); // new entries default to nullptr
		}

		Out.Reserve(Num);

		UObject* Outer = CreationOuter.IsValid() ? CreationOuter.Get() : CastChecked<UObject>(GetTransientPackage());

		for (int32 InstanceIndex = 0; InstanceIndex < Num; ++InstanceIndex)
		{
			// Create if missing or of the wrong type (defensive in case bucket was reused)
			if (!IsValid(Bucket.Items[InstanceIndex]) || Bucket.Items[InstanceIndex]->GetClass() != InType)
			{
				Bucket.Items[InstanceIndex] = NewObject<UIKRigStructWrapperBase>(Outer, InType, NAME_None, RF_Transient);
			}

			UIKRigStructWrapperBase* Instance = Bucket.Items[InstanceIndex];
			Instance->Reset();
			Out.Add(Instance);
		}

		return Out;
	}

private:
	// used as the Outer for all pooled objects
	UPROPERTY(Transient)
	TWeakObjectPtr<UObject> CreationOuter;

	// one bucket per subclass; values are GC-tracked via TObjectPtr
	UPROPERTY(Transient)
	TMap<TSubclassOf<UIKRigStructWrapperBase>, FIKRigStructWrapperBucket> Pool;
};