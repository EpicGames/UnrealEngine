// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IKRigStructViewer.h"

#include "Rig/Solvers/IKRigBodyMoverSolver.h"
#include "Rig/Solvers/IKRigFullBodyIK.h"
#include "Rig/Solvers/IKRigLimbSolver.h"
#include "Rig/Solvers/IKRigPoleSolver.h"
#include "Rig/Solvers/IKRigSetTransform.h"
#include "Rig/Solvers/IKRigStretchLimb.h"

#include "IKRigStructWrappers.generated.h"


//
// BODY MOVER Struct Wrappers
//

UCLASS()
class UBodyMoverSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Body Mover Settings")
	FIKRigBodyMoverSettings Settings;
};

UCLASS()
class UBodyMoverGoalSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Body Mover Goal Settings")
	FIKRigBodyMoverGoalSettings Settings;
};

//
// FBIK Struct Wrappers
//

UCLASS()
class UFBIKSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Full-Body IK Settings")
	FIKRigFBIKSettings Settings;
};

UCLASS()
class UFBIKGoalSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "FBIK Goal Settings")
	FIKRigFBIKGoalSettings Settings;
};

UCLASS()
class UFBIKBoneSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "FBIK Bone Settings")
	FIKRigFBIKBoneSettings Settings;
};

//
// Limb Solver Struct Wrappers
//

UCLASS()
class ULimbSolverSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Limb Solver Settings")
	FIKRigLimbSolverSettings Settings;
};

//
// Pole Solver Struct Wrappers
//

UCLASS()
class UPoleSolverSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Pole Solver Settings")
	FIKRigPoleSolverSettings Settings;
};

//
// SetTransform Solver Struct Wrappers
//

UCLASS()
class USetTransformSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Set Transform Settings")
	FIKRigSetTransformSettings Settings;
};

//
// Stretch Limb Solver Struct Wrappers
//

UCLASS()
class UStretchLimbSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Stretch Limb Settings")
	FIKRigStretchLimbSettings Settings;
};

UCLASS()
class UStretchLimbBoneSettingsWrapper : public UIKRigStructWrapperBase
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category = "Stretch Limb Bone Settings")
	FIKRigStretchLimbBoneSettings Settings;
};
