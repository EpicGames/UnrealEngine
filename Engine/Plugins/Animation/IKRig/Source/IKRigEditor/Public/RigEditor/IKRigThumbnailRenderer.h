// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "ThumbnailRendering/SkeletalMeshThumbnailRenderer.h"
#include "IKRigThumbnailRenderer.generated.h"

class FCanvas;
class FRenderTarget;

// this thumbnail renderer displays a given IK Rig in the asset icon
UCLASS(config=Editor, MinimalAPI)
class UIKRigThumbnailRenderer : public USkeletalMeshThumbnailRenderer
{
	GENERATED_BODY()

	// Begin UThumbnailRenderer Object
	IKRIGEDITOR_API virtual bool CanVisualizeAsset(UObject* Object) override;
	IKRIGEDITOR_API virtual void Draw(UObject* Object, int32 X, int32 Y, uint32 Width, uint32 Height, FRenderTarget*, FCanvas* Canvas, bool bAdditionalViewFamily) override;
	IKRIGEDITOR_API virtual EThumbnailRenderFrequency GetThumbnailRenderFrequency(UObject* Object) const override;
	// End UThumbnailRenderer Object

private:
	USkeletalMesh* GetPreviewMeshFromRig(UObject* Object) const;
};

