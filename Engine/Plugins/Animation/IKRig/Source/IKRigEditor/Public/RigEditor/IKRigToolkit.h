// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IHasPersonaToolkit.h"
#include "IKRigController.h"
#include "IKRigEditorController.h"
#include "IKRigMode.h"
#include "IPersonaPreviewScene.h"
#include "PersonaAssetEditorToolkit.h"
#include "SIKRigHierarchy.h"

namespace IKRigEditorModes
{
	extern const FName IKRigEditorMode;
}

class FIKRigEditorToolkit :
	public FPersonaAssetEditorToolkit,
	public IHasPersonaToolkit,
	public FSelfRegisteringEditorUndoClient,
	public FTickableEditorObject
{
public:

	FIKRigEditorToolkit();
	virtual ~FIKRigEditorToolkit() override;

	void InitAssetEditor(
		const EToolkitMode::Type Mode,
		const TSharedPtr< IToolkitHost >& InitToolkitHost,
		UIKRigDefinition* IKRigAsset);

	/** FAssetEditorToolkit interface */
	virtual void OnClose() override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager) override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	/** END FAssetEditorToolkit interface */

	//** FTickableEditorObject Interface
	virtual void Tick(float DeltaTime) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override;
	//~ END FTickableEditorObject Interface

	/** IHasPersonaToolkit interface */
	virtual TSharedRef<IPersonaToolkit> GetPersonaToolkit() const override { return PersonaToolkit.ToSharedRef(); }
	/** END IHasPersonaToolkit interface */

	/** FSelfRegisteringEditorUndoClient interface */
	virtual void PostUndo( bool bSuccess );
	virtual void PostRedo(bool bSuccess);
	virtual bool MatchesContext( const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts ) const override;
	/** END FSelfRegisteringEditorUndoClient interface */

	TSharedRef<FIKRigEditorController> GetController() const {return EditorController;};

	/** tab creation callbacks */
	void HandlePreviewSceneCreated(const TSharedRef<IPersonaPreviewScene>& InPersonaPreviewScene);
	void HandleDetailsCreated(const TSharedRef<class IDetailsView>& InDetailsView) const;
	void HandleViewportCreated(const TSharedRef<class IPersonaViewport>& InViewport);
	void HandleOnPreviewSceneSettingsCustomized(IDetailLayoutBuilder& DetailBuilder) const;
	/** END preview scene setup */

private:

	/** toolbar */
	void BindCommands();
	void ExtendToolbar();
	void FillToolbar(FToolBarBuilder& ToolbarBuilder);
	/** END toolbar */
	
	/** centralized management of selection across skeleton view and viewport */
	TSharedRef<FIKRigEditorController> EditorController;

	/** Preview scene to be supplied by IHasPersonaToolkit::GetPersonaToolkit */
	TSharedPtr<IPersonaToolkit> PersonaToolkit;
};
