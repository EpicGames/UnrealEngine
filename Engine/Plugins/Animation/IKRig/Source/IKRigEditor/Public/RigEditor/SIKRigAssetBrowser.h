// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ContentBrowserDelegates.h"
#include "Widgets/Layout/SBox.h"


class FIKRigEditorController;

class SIKRigAssetBrowser : public SBox
{
public:
	SLATE_BEGIN_ARGS(SIKRigAssetBrowser) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FIKRigEditorController> InEditorController);

private:
	//~ Begin SWidget
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget

	void ExecuteFindInContentBrowserAction();
	bool CanExecuteFindInContentBrowserAction();

	void RefreshView();
	void OnPathChange(const FString& NewPath);
	void OnAssetDoubleClicked(const FAssetData& AssetData);
	bool OnShouldFilterAsset(const struct FAssetData& AssetData);
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets) const;
	
	TWeakPtr<FIKRigEditorController> EditorController;
	TSharedPtr<SBox> AssetBrowserBox;
	TSharedPtr<FUICommandList> CommandList;
	/** Used to get the currently selected assets */
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;

	friend FIKRigEditorController;
};
