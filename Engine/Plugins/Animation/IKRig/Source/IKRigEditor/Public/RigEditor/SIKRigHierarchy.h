// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "SBaseHierarchyTreeView.h"
#include "SIKRigRetargetChainList.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "SIKRigSolverStack.h"

#include "SIKRigHierarchy.generated.h"

class FIKRigEditorController;
class SIKRigHierarchy;
class FIKRigEditorToolkit;
class USkeletalMesh;
class UIKRigBoneDetails;

enum class EIKRigTreeElementType { BONE, GOAL, GOAL_SETTINGS, BONE_SETTINGS };

class FIKRigTreeElement : public TSharedFromThis<FIKRigTreeElement>
{
public:
	
	FIKRigTreeElement(
		const FText& InKey,
		EIKRigTreeElementType InType,
		const TSharedRef<FIKRigEditorController>& InEditorController);

	FText Key;
	EIKRigTreeElementType ElementType;

	/** determines if this item is filtered out of the view */
	bool bIsHidden = false;
	TSharedPtr<FIKRigTreeElement> UnFilteredParent;
	
	TSharedPtr<FIKRigTreeElement> Parent;
	TArray<TSharedPtr<FIKRigTreeElement>> Children;

	/** effector meta-data (if it is an effector) */
	FName EffectorGoalName = NAME_None;
	int32 EffectorIndex = INDEX_NONE;

	/** bone setting meta-data (if it is a bone setting) */
	FName BoneSettingBoneName = NAME_None;
	int32 BoneSettingsSolverIndex = INDEX_NONE;

	/** name of bone if it is one */
	FName BoneName = NAME_None;
	
	/** name of goal if it is one */
	FName GoalName = NAME_None;

	/** delegate for when the context menu requests a rename */
	void RequestRename();
	DECLARE_DELEGATE(FOnRenameRequested);
	FOnRenameRequested OnRenameRequested;

	/** get the underlying object */
	void GetStruct(const UScriptStruct*& OutType, uint8*& OutMemory) const;

	/** get the name of the retarget chain this element belongs to (or NAME_None if not in a chain) */
	FName GetChainName() const;

private:
	/** centralized editor controls */
	TWeakPtr<FIKRigEditorController> EditorController;

	/** on demand bone details object */	
	mutable TObjectPtr<UIKRigBoneDetails> OptionalBoneDetails = nullptr;
};

class SIKRigHierarchyItem : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SIKRigHierarchyItem) {}
	SLATE_ARGUMENT(TWeakPtr<FIKRigEditorController>, EditorController)
	SLATE_ARGUMENT(TWeakPtr<FIKRigTreeElement>, TreeElement)
	SLATE_ARGUMENT(TWeakPtr<SIKRigHierarchy>, HierarchyView)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
private:

	bool OnVerifyNameChanged(const FText& InText, FText& OutErrorMessage) const;
	void OnNameCommitted(const FText& InText, ETextCommit::Type InCommitType) const;
	FText GetName() const;
	
	TWeakPtr<FIKRigTreeElement> WeakTreeElement;
	TWeakPtr<FIKRigEditorController> EditorController;
	TWeakPtr<SIKRigHierarchy> HierarchyView;
};

class FIKRigSkeletonDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FIKRigSkeletonDragDropOp, FDecoratedDragDropOp)
	static TSharedRef<FIKRigSkeletonDragDropOp> New(TWeakPtr<FIKRigTreeElement> InElement);
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	TWeakPtr<FIKRigTreeElement> Element;
};

class SIKRigSkeletonRow : public SMultiColumnTableRow<TSharedPtr<FIKRigTreeElement>>
{
public:

	SLATE_BEGIN_ARGS(SIKRigSkeletonRow) {}
	SLATE_ARGUMENT(TSharedPtr<FIKRigEditorController>, EditorController)
	SLATE_ARGUMENT(TSharedPtr<FIKRigTreeElement>, TreeElement)
	SLATE_ARGUMENT(TSharedPtr<SIKRigHierarchy>, HierarchyView)
	SLATE_END_ARGS()

	void Construct(
		const FArguments& InArgs,
		const TSharedRef<STableViewBase>& InOwnerTable)
	{
		OwnerTable = InOwnerTable;
		WeakTreeElement = InArgs._TreeElement;
		EditorController = InArgs._EditorController;
		HierarchyView = InArgs._HierarchyView;
		
		SMultiColumnTableRow<TSharedPtr<FIKRigTreeElement>>::Construct(
			FSuperRowType::FArguments()
			.OnDragDetected(this, &SIKRigSkeletonRow::HandleDragDetected)
			.OnCanAcceptDrop(this, &SIKRigSkeletonRow::HandleCanAcceptDrop)
			.OnAcceptDrop(this, &SIKRigSkeletonRow::HandleAcceptDrop)
			,InOwnerTable);
	}

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the list view. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	/** End SMultiColumnTableRow */

	/** drag and drop */
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FIKRigTreeElement> TargetItem);
	FReply HandleAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FIKRigTreeElement> TargetItem);
	/** END drag and drop */

private:

	TSharedPtr<STableViewBase> OwnerTable;
	TWeakPtr<FIKRigTreeElement> WeakTreeElement;
	TWeakPtr<FIKRigEditorController> EditorController;
	TWeakPtr<SIKRigHierarchy> HierarchyView;
};

struct FIKRigHierarchyFilterOptions
{
	bool bShowAll = true;
	bool bShowOnlyBones = false;
	bool bShowOnlyGoals = false;
	bool bShowOnlyBoneSettings = false;
	bool bHideUnaffectedBones = false;

	void ResetShowOptions()
	{
		bShowAll = false;
		bShowOnlyBones = false;
		bShowOnlyGoals = false;
		bShowOnlyBoneSettings = false;
	}
};

USTRUCT()
struct FIKRigHierarchyImportSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Hierachy Import")
	TObjectPtr<USkeletalMesh> Mesh = nullptr;
};

typedef SBaseHierarchyTreeView<FIKRigTreeElement> SIKRigSkeletonTreeView;

class SIKRigHierarchy : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SIKRigHierarchy) {}
	SLATE_END_ARGS()

    ~SIKRigHierarchy() = default;

	void Construct(const FArguments& InArgs, TSharedRef<FIKRigEditorController> InEditorController);

	/** selection state queries */
	static bool IsBoneInSelection(TArray<TSharedPtr<FIKRigTreeElement>>& SelectedBoneItems, const FName& BoneName);
	void GetSelectedBones(TArray<TSharedPtr<FIKRigTreeElement>>& OutBoneItems) const;
	void GetSelectedBoneNames(TArray<FName>& OutSelectedBoneNames) const;
	void GetSelectedGoals(TArray<TSharedPtr<FIKRigTreeElement>>& OutSelectedGoals) const;
	int32 GetNumSelectedGoals();
	void GetSelectedGoalNames(TArray<FName>& OutSelectedGoalNames) const;
	bool IsGoalSelected(const FName& GoalName);
	void AddSelectedItemFromViewport(
		const FName& ItemName,
		EIKRigTreeElementType ItemType,
		const bool bReplace);
	void GetSelectedBoneChains(TArray<FBoneChain>& OutChains) const;
	TArray<TSharedPtr<FIKRigTreeElement>> GetSelectedItems() const;
	bool HasSelectedItems() const;
	/** END selection state queries */

	/** determine if the element is connected to the selected solver */
	bool IsElementConnectedToSolver(TWeakPtr<FIKRigTreeElement> TreeElement, int32 SolverIndex);
	/** determine if the element is connected to ANY solver */
	bool IsElementConnectedToAnySolver(TWeakPtr<FIKRigTreeElement> TreeElement);
	/** determine if the element is an excluded bone*/
	bool IsElementExcludedBone(TWeakPtr<FIKRigTreeElement> TreeElement);

	/** return true if a skeleton hierarchy has been loaded from an asset*/
	bool IsSkeletonLoaded() const;
	/** import skeletal mesh into the asset */
	FReply OnImportSkeletonClicked() const;

private:
	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	/** END SWidget interface */
	
	/** Bind commands that this widget handles */
	void BindCommands();

	/** directly, programmatically add item to selection */
	void AddItemToSelection(const TSharedPtr<FIKRigTreeElement>& InItem);
	/** directly, programmatically add item to selection */
	void RemoveItemFromSelection(const TSharedPtr<FIKRigTreeElement>& InItem);
	/** directly, programmatically replace item in selection */
	void ReplaceItemInSelection(const FText& OldName, const FText& NewName);

	/** creating / renaming / deleting goals */
	void HandleNewGoal() const;
	bool CanAddNewGoal() const;
	void HandleRenameGoal() const;
	bool CanRenameGoal() const;
	void HandleDeleteElements();
	bool CanDeleteElement() const;
	/** END creating / renaming / deleting goals */

	/** connecting/disconnecting goals to solvers */
	void HandleConnectGoalToSolver();
	void HandleDisconnectGoalFromSolver();
	bool CanConnectGoalToSolvers() const;
	bool CanDisconnectGoalFromSolvers() const;
	void ConnectSelectedGoalsToSelectedSolvers(bool bConnect);
	int32 GetNumSelectedGoalToSolverConnections(bool bCountOnlyConnected) const;
	/** END connecting/disconnecting goals to solvers */

	/** setting root bone */
	void HandleSetRootBoneOnSolvers();
	bool CanSetRootBoneOnSolvers();
	/** END setting root bone */

	/** setting end bone */
	void HandleSetEndBoneOnSolvers();
	bool CanSetEndBoneOnSolvers() const;
	bool HasEndBoneCompatibleSolverSelected() const;
	/** END setting end bone */

	/** per-bone settings */
	void HandleAddBoneSettings();
	bool CanAddBoneSettings();
	void HandleRemoveBoneSettings();
	bool CanRemoveBoneSettings();
	/** END per-bone settings */
	
	/** exclude/include bones */
	void HandleExcludeBone();
	bool CanExcludeBone();
	void HandleIncludeBone();
	bool CanIncludeBone();
	/** END exclude/include bones */

	/** retarget chains */
	void HandleNewRetargetChain();
	bool CanAddNewRetargetChain();
	void HandleSetRetargetRoot();
	bool CanSetPelvis();
	void HandleClearPelvis();
	bool CanClearPelvis();
	/** END retarget chains */

	/** refresh the bone hierarchy */
	bool CanRefreshHierarchy();
	void HandleRefreshHierarchy();

	/** centralized editor controls (facilitate cross-communication between multiple UI elements)*/
	TWeakPtr<FIKRigEditorController> EditorController;
	
	/** command list we bind to */
	TSharedPtr<FUICommandList> CommandList;
	
	/** tree view widget */
	TSharedPtr<SIKRigSkeletonTreeView> TreeView;
	TArray<TSharedPtr<FIKRigTreeElement>> RootElements;
	TArray<TSharedPtr<FIKRigTreeElement>> AllElements;

	/** filtering the tree with search box */
	TSharedRef<SWidget> CreateFilterMenuWidget();
	void OnFilterTextChanged(const FText& SearchText);
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;
	FIKRigHierarchyFilterOptions FilterOptions;
	
	/** tree view callbacks */
	void RefreshTreeView(bool IsInitialSetup=false);
	void HandleGetChildrenForTree(TSharedPtr<FIKRigTreeElement> InItem, TArray<TSharedPtr<FIKRigTreeElement>>& OutChildren);
	void OnSelectionChanged(TSharedPtr<FIKRigTreeElement> Selection, ESelectInfo::Type SelectInfo);
	TSharedRef< SWidget > CreateAddNewMenu();
	TSharedPtr< SWidget > CreateContextMenu();
	void OnItemClicked(TSharedPtr<FIKRigTreeElement> InItem);
	void OnItemDoubleClicked(TSharedPtr<FIKRigTreeElement> InItem);
	void OnSetExpansionRecursive(TSharedPtr<FIKRigTreeElement> InItem, bool bShouldBeExpanded);
	void SetExpansionRecursive(TSharedPtr<FIKRigTreeElement> InElement, bool bTowardsParent, bool bShouldBeExpanded);
	void FillContextMenu(FMenuBuilder& MenuBuilder);
	/** END tree view callbacks */

	FIKRigHierarchyImportSettings ImportSettings;

	friend SIKRigHierarchyItem;
	friend FIKRigEditorController;
	friend SIKRigSolverStack;
	friend SIKRigRetargetChainList;
};
