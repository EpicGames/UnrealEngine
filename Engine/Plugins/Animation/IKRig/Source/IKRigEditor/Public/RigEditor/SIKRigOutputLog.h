// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBox.h"

struct FIKRigLogger;
class IMessageLogListing;
class FIKRetargetEditorController;

class SIKRigOutputLog : public SBox
{
	
public:
	
	SLATE_BEGIN_ARGS(SIKRigOutputLog) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const FName& InLogName);

	void ClearLog() const;
	
private:
	
	/** the output log */
	TSharedPtr<SWidget> OutputLogWidget;
	TSharedPtr<IMessageLogListing> MessageLogListing;
};