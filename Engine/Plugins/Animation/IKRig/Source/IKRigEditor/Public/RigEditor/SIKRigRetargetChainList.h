// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "Rig/IKRigDefinition.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Framework/Commands/UICommandList.h"
#include "Misc/TextFilterExpressionEvaluator.h"

#include "SIKRigRetargetChainList.generated.h"

class UIKRigEffectorGoal;
class FIKRigEditorController;
class SIKRigRetargetChainList;
class FIKRigEditorToolkit;
struct FBoneChain;
class SEditableTextBox;

class FRetargetChainElement
{
public:

	TSharedRef<ITableRow> MakeListRowWidget(
		const TSharedRef<STableViewBase>& InOwnerTable,
        TSharedRef<FRetargetChainElement> InStackElement,
        TSharedPtr<SIKRigRetargetChainList> InChainList);

	static TSharedRef<FRetargetChainElement> Make(const FName& InChainName)
	{
		return MakeShareable(new FRetargetChainElement(InChainName));
	}

	FName ChainName;

private:
	
	/** Hidden constructor, always use Make above */
	FRetargetChainElement(const FName& InChainName) : ChainName(InChainName) {}

	/** Hidden constructor, always use Make above */
	FRetargetChainElement() {}
};


class SIKRigRetargetChainRow : public SMultiColumnTableRow<TSharedPtr<FRetargetChainElement>>
{
public:
	
	void Construct(
        const FArguments& InArgs,
        const TSharedRef<STableViewBase>& InOwnerTableView,
        TSharedRef<FRetargetChainElement> InChainElement,
        TSharedPtr<SIKRigRetargetChainList> InChainList);

	/** Overridden from SMultiColumnTableRow.  Generates a widget for this column of the table row. */
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	const FReferenceSkeleton& GetReferenceSkeleton() const;

	TSharedRef<SWidget> MakeGoalComboEntryWidget(TSharedPtr<FString> InItem) const;
	
	void OnStartBoneComboSelectionChanged(FName InName) const;
	void OnEndBoneComboSelectionChanged(FName InName) const;
	void OnGoalComboSelectionChanged(TSharedPtr<FString> InGoalName, ESelectInfo::Type SelectInfo);
	
	FName GetStartBoneName(bool& bMultipleValues) const;
	FName GetEndBoneName(bool& bMultipleValues) const;
	FText GetGoalName() const;

	void OnRenameChain(const FText& InText, ETextCommit::Type ) const;
	
	TArray<TSharedPtr<FString>> GoalOptions;
	TWeakPtr<FRetargetChainElement> ChainElement;
	TWeakPtr<SIKRigRetargetChainList> ChainList;

	friend SIKRigRetargetChainList;
};

USTRUCT()
struct FIKRigRetargetChainSettings
{
	GENERATED_BODY()

	FIKRigRetargetChainSettings(){};
	
	FIKRigRetargetChainSettings(FName InChainName, const FName& InStartBone, const FName& InEndBone)
		: ChainName(InChainName)
		, StartBone(InStartBone)
		, EndBone(InEndBone)
	{}

	UPROPERTY(EditAnywhere, Category = "Bone Chain")
	FName ChainName;

	UPROPERTY(VisibleAnywhere, Category = "Bone Chain")
	FName StartBone;

	UPROPERTY(VisibleAnywhere, Category = "Bone Chain")
	FName EndBone;
};

struct FChainFilterOptions
{
	bool bHideSingleBoneChains = false;
	bool bShowOnlyIKChains = false;
	bool bShowOnlyMissingBoneChains = false;
};

typedef SListView< TSharedPtr<FRetargetChainElement> > SRetargetChainListViewType;

class SIKRigRetargetChainList : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SIKRigRetargetChainList) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FIKRigEditorController> InEditorController);

	TArray<FName> GetSelectedChains() const;

private:
	
	/** menu for adding new solver commands */
	TSharedPtr<FUICommandList> CommandList;
	
	/** editor controller */
	TWeakPtr<FIKRigEditorController> EditorController;

	/** pelvis / root bone widgets */
	void OnPelvisBoneComboSelectionChanged(FName InName);
	void OnRootBoneComboSelectionChanged(FName InName);
	FName GetPelvisBoneName(bool& bMultipleValues) const;
	FName GetRootBoneName(bool& bMultipleValues) const;
	const FReferenceSkeleton& GetReferenceSkeleton() const;
	
	/** list view */
	TSharedPtr<SRetargetChainListViewType> ListView;
	TArray< TSharedPtr<FRetargetChainElement> > ListViewItems;
	/** END list view */

	/** filtering the list with search box */
	TSharedRef<SWidget> CreateFilterMenuWidget();
	void OnFilterTextChanged(const FText& SearchText);
	TSharedPtr<FTextFilterExpressionEvaluator> TextFilter;
	FChainFilterOptions ChainFilterOptions;

	/** callback when "Add New Chain" clicked */
	bool IsAddChainEnabled() const;
	
	/** when a chain is clicked on in the stack view */
	void OnItemClicked(TSharedPtr<FRetargetChainElement> InItem);

	/** list view generate row callback */
	TSharedRef<ITableRow> MakeListRowWidget(TSharedPtr<FRetargetChainElement> InElement, const TSharedRef<STableViewBase>& OwnerTable);

	/** call to refresh the list view */
	void RefreshView();

	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	/** END SWidget interface */

	/** list view OnContextMenuOpening callback */
	TSharedPtr< SWidget > CreateContextMenu();
	
	/** sort chain list callback */
	void SortChainList();

	/** mirror the selected chains (right-click menu callback) */
	void MirrorSelectedChains() const;

	friend SIKRigRetargetChainRow;
	friend FIKRigEditorController;
};
