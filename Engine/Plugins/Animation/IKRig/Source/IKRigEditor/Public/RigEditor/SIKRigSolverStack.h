// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Framework/Commands/UICommandList.h"

struct FIKRigSolverBase;
class FIKRigEditorController;
class SIKRigSolverStack;
class FIKRigEditorToolkit;


class FSolverStackElement
{
public:

	TSharedRef<ITableRow> MakeListRowWidget(
		const TSharedRef<STableViewBase>& InOwnerTable,
        TSharedRef<FSolverStackElement> InStackElement,
        TSharedPtr<SIKRigSolverStack> InSolverStack);

	static TSharedRef<FSolverStackElement> Make(FText DisplayName, int32 SolverIndex)
	{
		return MakeShareable(new FSolverStackElement(DisplayName, SolverIndex));
	}
	
	FText DisplayName;
	int32 IndexInStack;

private:
	/** Hidden constructor, always use Make above */
	FSolverStackElement(FText InDisplayName, int32 SolverIndex)
        : DisplayName(InDisplayName), IndexInStack(SolverIndex)
	{
	}

	/** Hidden constructor, always use Make above */
	FSolverStackElement() {}
};

class SIKRigSolverStackItem : public STableRow<TSharedPtr<FSolverStackElement>>
{
public:
	
	void Construct(
        const FArguments& InArgs,
        const TSharedRef<STableViewBase>& OwnerTable,
        TSharedRef<FSolverStackElement> InStackElement,
        TSharedPtr<SIKRigSolverStack> InSolverStack);

	bool GetWarningMessage(FText& Message) const;

	bool IsSolverEnabled() const;

private:
	const FIKRigSolverBase* GetSolver(const bool bFromAsset) const;
	TWeakPtr<FSolverStackElement> StackElement;
	TWeakPtr<SIKRigSolverStack> SolverStack;
};

class FIKRigSolverStackDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FIKRigSolverStackDragDropOp, FDecoratedDragDropOp)
    static TSharedRef<FIKRigSolverStackDragDropOp> New(TWeakPtr<FSolverStackElement> InElement);
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	TWeakPtr<FSolverStackElement> Element;
};

struct FIKRigSolverMetaData
{
	UScriptStruct* ScriptStruct;
	FText NiceName;
};

typedef SListView< TSharedPtr<FSolverStackElement> > SSolverStackListViewType;
class SIKRigSolverStack : public SCompoundWidget, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SIKRigSolverStack) {}
	SLATE_END_ARGS()

    ~SIKRigSolverStack();

	void Construct(const FArguments& InArgs, TSharedRef<FIKRigEditorController> InEditorController);

private:
	/** menu for adding new solver commands */
	TSharedPtr<FUICommandList> CommandList;
	
	/** editor controller */
	TWeakPtr<FIKRigEditorController> EditorController;
	
	/** the solver stack list view */
	TSharedPtr<SSolverStackListViewType> ListView;
	TArray< TSharedPtr<FSolverStackElement> > ListViewItems;

	/** solver stack menu */
	TSharedRef<SWidget> CreateAddNewMenuWidget();
	void BuildAddNewMenu(FMenuBuilder& MenuBuilder);
	bool IsAddSolverEnabled() const;
	/** END solver stack menu */

	/** all type info about all available solver types (cached at startup) */
	TArray<FIKRigSolverMetaData> AllSolversMetaData;
	void CacheSolverMetaData();
	
	/** menu command callback for adding a new solver */
	void AddNewSolver(UScriptStruct* SolverType);
	/** delete solver from stack */
	void DeleteSolver(TSharedPtr<FSolverStackElement> SolverToDelete);
	/** when a solver is selected on in the stack view */
	void OnSelectionChanged(TSharedPtr<FSolverStackElement> InItem, ESelectInfo::Type SelectInfo);
	/** when a solver is clicked on in the stack view */
	void OnItemClicked(TSharedPtr<FSolverStackElement> InItem);
	void ShowDetailsForItems(const TArray<TSharedPtr<FSolverStackElement>>& InItems); 

	/** list view generate row callback */
	TSharedRef<ITableRow> MakeListRowWidget(TSharedPtr<FSolverStackElement> InElement, const TSharedRef<STableViewBase>& OwnerTable);

	/** call to refresh the stack view */
	void RefreshStackView();

	/** SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	/** END SWidget interface */
	
	/** drag and drop operations */
    FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
    TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FSolverStackElement> TargetItem);
    FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone DropZone, TSharedPtr<FSolverStackElement> TargetItem);
	/** END drag and drop operations */

	friend SIKRigSolverStackItem;
	friend FIKRigEditorController;
};
