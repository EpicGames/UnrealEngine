// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/STreeView.h"

template<class ItemType>
class SBaseHierarchyTreeView : public STreeView<TSharedPtr<ItemType>>
{
	typedef STreeView<TSharedPtr<ItemType>> BaseView;
	using TSparseItemMap = typename BaseView::TSparseItemMap;
	using TItemSet = typename BaseView::TItemSet;
	
public:

	virtual ~SBaseHierarchyTreeView() = default;

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override
	{
		FReply Reply = STreeView<TSharedPtr<ItemType>>::OnFocusReceived(MyGeometry, InFocusEvent);
		LastClickCycles = FPlatformTime::Cycles();
		return Reply;
	}

	/** Save a snapshot of items expansion and selection state */
	void SaveAndClearState()
	{
		SaveAndClearSparseItemInfos();
		SaveAndClearSelection();
	}
	
	/** Restore items expansion and selection state from the saved snapshot after tree reconstruction */
	void RestoreState(const TSharedPtr<ItemType>& ItemPtr)
	{
		this->RestoreSparseItemInfos(ItemPtr);
		this->RestoreSelection(ItemPtr);
	}

	/** slow double-click rename state*/
	uint32 LastClickCycles = 0;
	TWeakPtr<ItemType> LastSelected;

private:

	/** Save a snapshot of the internal map that tracks item expansion before tree reconstruction */
	void SaveAndClearSparseItemInfos()
	{
		OldSparseItemInfos = BaseView::SparseItemInfos;
		BaseView::ClearExpandedItems();
	}

	/** Restore the expansion infos map from the saved snapshot after tree reconstruction */
	void RestoreSparseItemInfos(const TSharedPtr<ItemType>& ItemPtr)
	{
		for (const TTuple<TSharedPtr<ItemType>, FSparseItemInfo>& Pair : OldSparseItemInfos)
		{
			if (Pair.Key->Key.EqualTo(ItemPtr->Key))
			{
				// the SparseItemInfos now reference the new element, but keep the same expansion state
				BaseView::SparseItemInfos.Add(ItemPtr, Pair.Value);
				return;
			}
		}

		// set default state as expanded if not found
		BaseView::SparseItemInfos.Add(ItemPtr, FSparseItemInfo(true, false));
	}
	
	/** Save a snapshot of the internal set that tracks item selection before tree reconstruction */
	void SaveAndClearSelection()
	{
		OldSelectedItems = BaseView::SelectedItems;
		BaseView::ClearSelection();
	}
	
	/** Restore the selection from the saved snapshot after tree reconstruction */
	void RestoreSelection(const TSharedPtr<ItemType>& ItemPtr)
	{
		for (const TSharedPtr<ItemType>& OldItem : OldSelectedItems)
		{
			if (OldItem->Key.EqualTo(ItemPtr->Key))
			{
				// select the new element
				BaseView::SetItemSelection(ItemPtr, true, ESelectInfo::Direct);
				return;
			}
		}
	}
	
	/** A temporary snapshot of the SparseItemInfos in STreeView, used during SIKRetargetHierarchy::RefreshTreeView() */
	TSparseItemMap OldSparseItemInfos;

	/** A temporary snapshot of the SelectedItems in SListView, used during SIKRetargetHierarchy::RefreshTreeView() */
	TItemSet OldSelectedItems;
};
