// Copyright Epic Games, Inc. All Rights Reserved.

#include "IKRigUAFModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Retargeter/IKRetargeter.h"
#include "RigVMCore/RigVMRegistry.h"

void FIKRigUAFModule::StartupModule()
{
}

void FIKRigUAFModule::ShutdownModule()
{
	IModuleInterface::ShutdownModule();
}

IMPLEMENT_MODULE(FIKRigUAFModule, IKRigUAF)