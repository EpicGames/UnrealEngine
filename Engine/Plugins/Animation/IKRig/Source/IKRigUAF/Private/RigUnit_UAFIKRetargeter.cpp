// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigUnit_UAFIKRetargeter.h"

#include "Engine/SkeletalMesh.h"
#include "AnimationRuntime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_UAFIKRetargeter)

DEFINE_STAT(STAT_AnimNext_RigUnit_IKRetargeter);

FRigUnit_UAFIKRetargeter_Execute()
{
	SCOPE_CYCLE_COUNTER(STAT_AnimNext_RigUnit_IKRetargeter);

	using namespace UE::UAF;

	// validate prerequisites
	if( !IsValid(IKRetargetAsset) ||
		!SourcePose.LODPose.IsValid() ||
		!TargetAnimGraphRefPose.ReferencePose.IsValid())
	{
		return;
	}

	// get source skeletal mesh
	const UE::UAF::FReferencePose& SourceRefPose = SourcePose.LODPose.GetRefPose();
	const USkeletalMesh* SourceMesh = SourceRefPose.SkeletalMesh.Get();
	
	// get target skeletal mesh
	const UE::UAF::FReferencePose& TargetRefPose = TargetAnimGraphRefPose.ReferencePose.GetRef<UE::UAF::FReferencePose>();
	const USkeletalMesh* TargetMesh = TargetRefPose.SkeletalMesh.Get();

	// collect settings to retarget with starting with asset settings and override/merge with custom profile
	WorkData.Profile.FillProfileWithAssetSettings(IKRetargetAsset);
	WorkData.Profile.MergeWithOtherProfile(CustomRetargetProfile);
		
	// ensure the processor is initialized with the current inputs
	FIKRetargetProcessor& Processor = WorkData.Processor;
	if (!Processor.WasInitializedWithTheseAssets(SourceMesh, TargetMesh, IKRetargetAsset))
	{
		// initialize retarget processor with source and target skeletal meshes
		FRetargetInitParameters Params;
		Params.SourceSkeletalMesh = SourceMesh;
		Params.TargetSkeletalMesh = TargetMesh;
		Params.RetargeterAsset = IKRetargetAsset;
		Params.CustomProfile = &WorkData.Profile;
		Params.bSuppressWarnings = false;
		Processor.Initialize(Params);
	}

	// bail out if not initialized
	if (!Processor.IsInitialized())
	{
		return;
	}

	// copy the full local ref pose of the source
	const FReferenceSkeleton& SourceRefSkeleton = SourceMesh->GetRefSkeleton();
	WorkData.SourceMeshPoseLocal = SourceRefSkeleton.GetRefBonePose();
	// override all bones in the current LOD with the current local input pose
	const int32 NumBonesInSourceLOD = SourcePose.LODPose.GetNumBones();
	for (int32 SourceLODBoneIndex = 0; SourceLODBoneIndex < NumBonesInSourceLOD; ++SourceLODBoneIndex)
	{
		const int32 MeshBoneIndex = SourceRefPose.GetMeshBoneIndexFromLODBoneIndex(SourceLODBoneIndex);
		WorkData.SourceMeshPoseLocal[MeshBoneIndex] = SourcePose.LODPose.LocalTransforms[SourceLODBoneIndex];
	}

	// convert to component space
	FAnimationRuntime::FillUpComponentSpaceTransforms(SourceRefSkeleton, WorkData.SourceMeshPoseLocal, WorkData.SourceMeshPoseGlobal);
	
	// run the retargeter
	FRetargetRunParameters Params;
	Params.SourceGlobalPose = &WorkData.SourceMeshPoseGlobal;
	Params.Profile = &WorkData.Profile;
	Params.OverrideSetsToApply = OverrideSetNames;
	Params.Variables = nullptr; // TODO, add support for dynamic variables input pins in UAF
	Params.DeltaTime = ExecuteContext.GetDeltaTime();
	Params.LOD = TargetRefPose.GetSourceLODLevel();
	const TArray<FTransform>& TargetGlobalPose = Processor.RunRetargeter(Params);

	// fill the target pose with a ref pose at the appropriate LOD level
	TargetPose.LODPose.PrepareForLOD(TargetRefPose, TargetRefPose.GetSourceLODLevel(), true/*bSetRefPose*/, false/*bAdditive*/);

	// copy output pose back to target lod after converting back to local space
	const TArrayView<const FBoneIndexType>& LODToMeshBoneMap = TargetPose.LODPose.GetLODBoneIndexToMeshBoneIndexMap();
	const TArrayView<const FBoneIndexType>& MeshToParentMeshBoneMap = TargetPose.LODPose.GetMeshBoneIndexToParentMeshBoneIndexMap();
	const int32 NumBonesInTargetLOD = TargetPose.LODPose.GetNumBones();
	for (int32 TargetLODBoneIndex = 0; TargetLODBoneIndex < NumBonesInTargetLOD; ++TargetLODBoneIndex)
	{
		const FBoneIndexType ChildMeshBoneIndex = LODToMeshBoneMap[TargetLODBoneIndex];
		const FBoneIndexType ParentMeshBoneIndex = MeshToParentMeshBoneMap[ChildMeshBoneIndex];
		const FTransform& ChildGlobal = TargetGlobalPose[ChildMeshBoneIndex];
		const FTransform& ParentGlobal = TargetGlobalPose.IsValidIndex(ParentMeshBoneIndex) ? TargetGlobalPose[ParentMeshBoneIndex] : FTransform::Identity;
		TargetPose.LODPose.LocalTransforms[TargetLODBoneIndex] = (ChildGlobal * ParentGlobal.Inverse());
	}
}