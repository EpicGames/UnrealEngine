// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Retargeter/IKRetargeter.h"
#include "AnimNextExecuteContext.h"
#include "Graph/AnimNext_LODPose.h"
#include "Graph/RigUnit_AnimNextBase.h"
#include "RemapPoseData.h"
#include "AnimNextStats.h"
#include "Retargeter/IKRetargetProcessor.h"

#include "RigUnit_UAFIKRetargeter.generated.h"

DECLARE_CYCLE_STAT_EXTERN(TEXT("UAF RigUnit: IK Retargeter Total"), STAT_AnimNext_RigUnit_IKRetargeter, STATGROUP_AnimNext, IKRIGUAF_API);

USTRUCT(BlueprintType)
struct FIKRetargetWorkData
{
	GENERATED_BODY()
 
	// the runtime processor used to run the retarget and generate new poses
	FIKRetargetProcessor Processor;

	// scratch space for adapting the LOD'd pose to the full input pose
	TArray<FTransform> SourceMeshPoseLocal;
	TArray<FTransform> SourceMeshPoseGlobal;

	// reusable profile to pass to the retargeter (merged with input profile)
	FRetargetProfile Profile;
};

// IK Retargeting UAF Graph node.
USTRUCT(MinimalAPI, meta=(DisplayName="IK Retargeter", Category="Animation Graph", NodeColor="1, 0, 1", Keywords="Retarget, IK"))
struct FRigUnit_UAFIKRetargeter : public FRigUnit_AnimNextBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	void Execute();

	// The source input pose to retarget.
	UPROPERTY(EditAnywhere, DisplayName = "Source Pose", Category = "Graph", meta = (Input))
	FAnimNextGraphLODPose SourcePose;

	// The ref pose of the target skeletal mesh.
	UPROPERTY(EditAnywhere, DisplayName = "Target Ref Pose", Category = "Graph", meta = (Input))
	FAnimNextGraphReferencePose TargetAnimGraphRefPose;

	// An IK Retarget asset defining the retargeting operations to run.
	UPROPERTY(EditAnywhere, DisplayName = "IK Retarget Asset", Category = "Graph", meta = (Input))
	TObjectPtr<UIKRetargeter> IKRetargetAsset;

	// Connect a custom retarget profile to modify the retargeter's settings at runtime.
	UPROPERTY(EditAnywhere, DisplayName = "Custom Profile", Category = "Graph", meta = (Input))
	FRetargetProfile CustomRetargetProfile;

	// An optional list of property override sets to apply to op settings
	// NOTE: these names must match the names of override sets stored in the retarget asset
	UPROPERTY(EditAnywhere, DisplayName = "Custom Override Sets", Category = "Graph", meta = (Input))
	TArray<FName> OverrideSetNames;

	// The output pose for the target skeletal mesh.
	UPROPERTY(EditAnywhere, DisplayName = "Retargeted Pose", Category = "Graph", meta = (Output))
	FAnimNextGraphLODPose TargetPose;
	
	UPROPERTY(EditAnywhere, DisplayName = "Execute", Category = "BeginExecution", meta = (Input, Output))
	FAnimNextExecuteContext ExecuteContext;

	UPROPERTY(meta = (Hidden))
	FIKRetargetWorkData WorkData;
};
